[CoreRedirects]
; 5.3
+StructRedirects=(OldName="StaticMeshInstanceVisualizationMeshDesc",NewName="/Script/MassRepresentation.MassStaticMeshInstanceVisualizationMeshDesc")
+StructRedirects=(OldName="ISMCSharedData",NewName="/Script/MassRepresentation.MassISMCSharedData")