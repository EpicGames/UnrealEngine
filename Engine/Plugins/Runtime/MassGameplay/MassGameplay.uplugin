{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.4",
	"FriendlyName": "MassGameplay",
	"Description": "Implementation of large-scale agent simulation based on MassEntity",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MassCommon",
			"Type": "Runtime"
		},
		{
			"Name": "MassActors",
			"Type": "Runtime"
		},
		{
			"Name": "MassEQS",
			"Type": "Runtime"
		},
		{
			"Name": "MassSpawner",
			"Type": "Runtime"
		},
		{
			"Name": "MassSmartObjects",
			"Type": "Runtime"
		},
		{
			"Name": "MassSimulation",
			"Type": "Runtime"
		},
		{
			"Name": "MassLOD",
			"Type": "Runtime"
		},
		{
			"Name": "MassMovement",
			"Type": "Runtime"
		},
		{
			"Name": "MassCharacterTrajectory",
			"Type": "Runtime"
		},
		{
			"Name": "MassReplication",
			"Type": "Runtime"
		},
		{
			"Name": "MassRepresentation",
			"Type": "Runtime"
		},
		{
			"Name": "MassGameplayDebug",
			"Type": "Runtime"
		},
		{
			"Name": "MassGameplayEditor",
			"Type": "Editor"
		},
		{
			"Name": "MassMovementEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "MassGameplayExternalTraits",
			"Type": "Runtime"
		},
		{
			"Name": "MassGameplayTestSuite",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ZoneGraph",
			"Enabled": true
		},
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "ZoneGraphAnnotations",
			"Enabled": true
		},
		{
			"Name": "SmartObjects",
			"Enabled": true
		},
		{
			"Name": "StateTree",
			"Enabled": true
		},
		{
			"Name": "DataValidation",
			"Enabled": true
		}
	]
}