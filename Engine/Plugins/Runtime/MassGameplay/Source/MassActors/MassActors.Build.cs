// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassActors : ModuleRules
	{
		public MassActors(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"MassCore",
					"MassEntity",
					"AIModule",
					"MassCommon",
					"MassMovement",
					"MassSpawner",
					"MassSimulation",
					"MassReplication",
					"MassSignals",
					"NetCore",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
