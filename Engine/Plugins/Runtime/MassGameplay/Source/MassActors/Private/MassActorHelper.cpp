// Copyright Epic Games, Inc.All Rights Reserved.

#include "MassActorHelper.h"
#include "MassActorTypes.h"
#include "MassActorSubsystem.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "Engine/World.h"

namespace UE::MassActor
{
	bool AddEntityTagToActor(const AActor& Actor, const UScriptStruct& TagType)
	{
		UWorld* World = Actor.GetWorld();
		UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(World);
		UMassActorSubsystem* MassActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(World);
		if (EntitySubsystem && MassActorSubsystem)
		{
			FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
			const FMassEntityHandle AgentHandle = MassActorSubsystem->GetEntityHandleFromActor(&Actor);
			if (AgentHandle.IsValid())
			{
				EntityManager.AddTagToEntity(AgentHandle, &TagType);
				return true;
			}
			UE_LOGF(LogMassActor, Warning, "Failed to add tag %ls to actor %ls due to it not having an associated entity"
				, *TagType.GetName(), *Actor.GetName());
		}
		else
		{
			UE_LOGF(LogMassActor, Warning, "Failed to add tag %ls to actor %ls due to missing: %ls%ls"
				, *TagType.GetName(), *Actor.GetName(), EntitySubsystem ? TEXT("EntitySubsystem, ") : TEXT("")
				, MassActorSubsystem ? TEXT("MassActorSubsystem, ") : TEXT(""));
		}
		return false;
	}

	bool RemoveEntityTagFromActor(const AActor& Actor, const UScriptStruct& TagType)
	{
		UWorld* World = Actor.GetWorld();
		UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(World);
		UMassActorSubsystem* MassActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(World);
		if (EntitySubsystem && MassActorSubsystem)
		{
			const FMassEntityHandle AgentHandle = MassActorSubsystem->GetEntityHandleFromActor(&Actor);
			if (AgentHandle.IsValid())
			{
				FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
				EntityManager.RemoveTagFromEntity(AgentHandle, &TagType);
				return true;
			}
			UE_LOGF(LogMassActor, Warning, "Failed to remove tag %ls from actor %ls due to it not having an associated entity"
				, *TagType.GetName(), *Actor.GetName());
		}
		else
		{
			UE_LOGF(LogMassActor, Warning, "Failed to remove tag %ls from actor %ls due to missing: %ls%ls"
				, *TagType.GetName(), *Actor.GetName(), EntitySubsystem ? TEXT("EntitySubsystem, ") : TEXT("")
				, MassActorSubsystem ? TEXT("MassActorSubsystem, ") : TEXT(""));
		}
		return false;
	}
} // namespace UE::MassActor
