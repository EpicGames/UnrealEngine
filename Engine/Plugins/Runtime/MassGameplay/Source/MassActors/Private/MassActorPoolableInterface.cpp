// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassActorPoolableInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassActorPoolableInterface)


UMassActorPoolableInterface::UMassActorPoolableInterface(FObjectInitializer const& InOI)
	: UInterface(InOI)
{
}
