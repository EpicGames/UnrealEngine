// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassActorSpawnerSubsystem.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "MassActorPoolableInterface.h"
#include "MassActorTypes.h"
#include "MassAgentComponent.h"
#include "MassEntitySubsystem.h"
#include "MassSimulationSettings.h"
#include "MassSimulationSubsystem.h"
#include "MassSpawner.h"
#include "MassSpawnerInternalTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassActorSpawnerSubsystem)

CSV_DEFINE_CATEGORY(MassActors, true);

namespace UE::Mass::Actors
{
	bool bUseActorPooling = true;
	bool bUseGUIDToNameActors = false;
	namespace
	{
		FAutoConsoleVariableRef CVars[] = 
		{
			{TEXT("ai.mass.actorpooling"), bUseActorPooling, TEXT("Use Actor Pooling. DEPRECATED. uSE "), ECVF_Scalability}
			, {TEXT("mass.spawning.actorpooling"), bUseActorPooling, TEXT("Use Actor Pooling"), ECVF_Scalability}
			, {TEXT("mass.spawning.UseGUIDToNameActors"), bUseGUIDToNameActors, TEXT("Whether to include spawn request's GUID in the spawned actor's name. Using this can result in some actors being reused if their name matches")}
		};
	}
	
}

//----------------------------------------------------------------------//
// UMassActorSpawnerSubsystem 
//----------------------------------------------------------------------//
void UMassActorSpawnerSubsystem::RetryActorSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle)
{
	check(SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle));
	const int32 Index = SpawnRequestHandle.GetIndex();
	check(SpawnRequests.IsValidIndex(Index));
	FMassActorSpawnRequest& SpawnRequest = SpawnRequests[SpawnRequestHandle.GetIndex()].GetMutable<FMassActorSpawnRequest>();
	if (ensureMsgf(SpawnRequest.SpawnStatus == ESpawnRequestStatus::Failed, TEXT("Can only retry failed spawn requests")))
	{
		UWorld* World = GetWorld();
		check(World);

		SpawnRequest.SpawnStatus = ESpawnRequestStatus::RetryPending;
		SpawnRequest.SerialNumber = RequestSerialNumberCounter.fetch_add(1);
		SpawnRequest.RequestedTime = World->GetTimeSeconds();
	}
}

bool UMassActorSpawnerSubsystem::RemoveActorSpawnRequest(FMassActorSpawnRequestHandle& SpawnRequestHandle)
{
	if (!ensureMsgf(SpawnRequestHandleManager.RemoveHandle(SpawnRequestHandle), TEXT("Invalid spawn request handle")))
	{
		return false;
	}

	check(SpawnRequests.IsValidIndex(SpawnRequestHandle.GetIndex()));
	FMassActorSpawnRequest& SpawnRequest = SpawnRequests[SpawnRequestHandle.GetIndex()].GetMutable<FMassActorSpawnRequest>();
	check(SpawnRequest.SpawnStatus != ESpawnRequestStatus::Processing);
	SpawnRequestHandle.Invalidate();
	SpawnRequest.Reset();
	return true;
}

void UMassActorSpawnerSubsystem::ConditionalDestroyActor(UWorld& World, AActor& ActorToDestroy)
{
	UWorld* ActorsWorld = ActorToDestroy.GetWorld();

	// we directly call DestroyActors only if they're tied to the current world.
	// Otherwise we rely on engine mechanics to get rid of them.
	if (ActorsWorld == &World)
	{
		World.DestroyActor(&ActorToDestroy);
	}
	else
	{
		ensureMsgf(ActorToDestroy.HasActorBegunPlay() == false, TEXT("Failed to destroy %s due to world mismatch, while the actor is still being valid (as indicated by HasActorBegunPlay() == true)")
			, *ActorToDestroy.GetName());
	}
}

void UMassActorSpawnerSubsystem::DestroyActor(AActor* Actor, bool bImmediate /*= false*/)
{
	check(Actor);

	// We need to unregister immediately MassAgentComponent as it will become out of sync with mass
	if (UMassAgentComponent* AgentComp = Actor->FindComponentByClass<UMassAgentComponent>())
	{
		// All we want here it to unregister with the subsysem, not unregister the component as we want to keep it for futher usage if we put the actor in the pool.
		AgentComp->UnregisterWithAgentSubsystem();
	}

	if(bImmediate)
	{
		if (!ReleaseActorToPool(Actor))
		{
			// Couldn't release actor back to pool, so destroy it
			UWorld* World = GetWorld();
			check(World);
			ConditionalDestroyActor(*World, *Actor);

			--NumActorSpawned;
		}
	}
	else
	{
		ActorsToDestroy.Add(Actor);
	}
}

bool UMassActorSpawnerSubsystem::ReleaseActorToPool(AActor* Actor)
{
	if (!IsActorPoolingEnabled())
	{
		return false;
	}

	const bool bIsPoolableActor = Actor->Implements<UMassActorPoolableInterface>();
	if (bIsPoolableActor && IMassActorPoolableInterface::Execute_CanBePooled(Actor))
	{
		UMassAgentComponent* AgentComp = Actor->FindComponentByClass<UMassAgentComponent>();
		if (!AgentComp || AgentComp->IsReadyForPooling())
		{
			IMassActorPoolableInterface::Execute_PrepareForPooling(Actor);
			Actor->SetActorHiddenInGame(true);
			if (AgentComp)
			{
				AgentComp->UnregisterWithAgentSubsystem();
			}

			auto& Pool = PooledActors.FindOrAdd(Actor->GetClass());
			checkf(Pool.Find(Actor) == INDEX_NONE, TEXT("Actor%s is already in the pool"), *AActor::GetDebugName(Actor));
			Pool.Add(Actor);
			++NumActorPooled;
			return true;
		}
	}
	return false;
}

FMassActorSpawnRequestHandle UMassActorSpawnerSubsystem::RequestActorSpawnInternal(const FConstStructView SpawnRequestView)
{
	// The handle manager has a freelist of the release indexes, so it can return us a index that we previously used.
	const FMassActorSpawnRequestHandle SpawnRequestHandle = SpawnRequestHandleManager.GetNextHandle();
	const int32 Index = SpawnRequestHandle.GetIndex();

	// Check if we need to grow the array, otherwise it is a previously released index that was returned.
	if (!SpawnRequests.IsValidIndex(Index))
	{
		checkf(SpawnRequests.Num() == Index, TEXT("This case should only be when we need to grow the array of one element."));
		SpawnRequests.Emplace(SpawnRequestView);
	}
	else
	{
		SpawnRequests[Index] = SpawnRequestView;
	}

	UWorld* World = GetWorld();
	check(World);

	// Initialize the spawn request status
	FMassActorSpawnRequest& SpawnRequest = GetMutableSpawnRequest<FMassActorSpawnRequest>(SpawnRequestHandle);
	SpawnRequest.SpawnStatus = ESpawnRequestStatus::Pending;
	SpawnRequest.SerialNumber = RequestSerialNumberCounter.fetch_add(1);
	SpawnRequest.RequestedTime = World->GetTimeSeconds();

	return SpawnRequestHandle;
}

// @todo investigate whether storing requests in a sorted array would improve overall perf - if AllHandles was sorted we
// wouldn't need to do any tests other than just checking if a thing is valid and not completed (i.e. pending or retry-pending).
FMassActorSpawnRequestHandle UMassActorSpawnerSubsystem::GetNextRequestToSpawn(int32& InOutHandleIndex) const
{
	const TArray<FMassActorSpawnRequestHandle>& AllHandles = SpawnRequestHandleManager.GetHandles();
	if (AllHandles.Num() == 0)
	{
		InOutHandleIndex = INDEX_NONE;
		return FMassActorSpawnRequestHandle();
	}

	FMassActorSpawnRequestHandle BestSpawnRequestHandle;
	float BestPriority = MAX_FLT;
	bool bBestIsPending = false;
	uint32 BestSerialNumber = MAX_uint32;
	int32 BestIndex = INDEX_NONE;
		
	int32 HandleIndex = InOutHandleIndex != INDEX_NONE ? ((InOutHandleIndex + 1) % AllHandles.Num()) : 0;
	const int32 IterationsLimit = (InOutHandleIndex == INDEX_NONE) ? AllHandles.Num() : (AllHandles.Num() - 1);
	
	for (int32 IterationIndex = 0; IterationIndex < IterationsLimit; ++IterationIndex, HandleIndex = (HandleIndex + 1) % AllHandles.Num())
	{
		const FMassActorSpawnRequestHandle SpawnRequestHandle = AllHandles[HandleIndex];
		if (!SpawnRequestHandle.IsValid())
		{
			continue;
		}
		const FMassActorSpawnRequest& SpawnRequest = GetSpawnRequest<FMassActorSpawnRequest>(SpawnRequestHandle);
		if (SpawnRequest.SpawnStatus == ESpawnRequestStatus::Pending)
		{
			if (!bBestIsPending ||
				SpawnRequest.Priority < BestPriority ||
				(SpawnRequest.Priority == BestPriority && SpawnRequest.SerialNumber < BestSerialNumber))
			{
				BestSpawnRequestHandle = SpawnRequestHandle;
				BestSerialNumber = SpawnRequest.SerialNumber;
				BestPriority = SpawnRequest.Priority;
				bBestIsPending = true;
				BestIndex = HandleIndex;
			}
		}
		else if (!bBestIsPending && SpawnRequest.SpawnStatus == ESpawnRequestStatus::RetryPending)
		{
			// No priority on retries just FIFO
			if (SpawnRequest.SerialNumber < BestSerialNumber)
			{
				BestSpawnRequestHandle = SpawnRequestHandle;
				BestSerialNumber = SpawnRequest.SerialNumber;
				BestIndex = HandleIndex;
			}
		}
	}

	InOutHandleIndex = BestIndex;
	return BestSpawnRequestHandle;
}

ESpawnRequestStatus UMassActorSpawnerSubsystem::SpawnOrRetrieveFromPool(const FSpawnArgs& Args)
{
	if (IsActorPoolingEnabled())
	{
		auto* Pool = PooledActors.Find(Args.Request.Template);

		if (Pool && Pool->Num() > 0)
		{
			AActor* PooledActor = nullptr;
			do
			{
				PooledActor = Pool->Pop(EAllowShrinking::No);
				--NumActorPooled;
			} while (IsValid(PooledActor) == false && Pool->Num() > 0);

			if (IsValid(PooledActor))
			{
				PooledActor->SetActorHiddenInGame(false);
				PooledActor->SetActorTransform(Args.Request.Transform, false, nullptr, ETeleportType::ResetPhysics);

				IMassActorPoolableInterface::Execute_PrepareForGame(PooledActor);

				if (UMassAgentComponent* AgentComp = PooledActor->FindComponentByClass<UMassAgentComponent>())
				{
					// normally this function gets called from UMassAgentComponent::OnRegister. We need to call it manually here
					// since we're bringing this actor our of a pool.
					AgentComp->RegisterWithAgentSubsystem();
				}

				Args.Request.SpawnedActor = PooledActor;
				return ESpawnRequestStatus::Succeeded;
			}
		}
	}

	FActorSpawnParameters ActorSpawnParameters;
	ESpawnRequestStatus SpawnStatus = SpawnActor(Args, ActorSpawnParameters);

	if (SpawnStatus == ESpawnRequestStatus::Succeeded)
	{
		if (IsValidChecked(Args.Request.SpawnedActor))
		{
			++NumActorSpawned;
		}
	}
	else
	{
		UE_VLOG_CAPSULE(this, LogMassActor, Error,
					Args.Request.Transform.GetLocation(),
					Args.Request.Template.GetDefaultObject()->GetSimpleCollisionHalfHeight(),
					Args.Request.Template.GetDefaultObject()->GetSimpleCollisionRadius(),
					Args.Request.Transform.GetRotation(),
					FColor::Red,
					TEXT("Unable to spawn actor for Mass entity [%s]"), *Args.Request.MassAgent.DebugGetDescription());
	}

	return SpawnStatus;
}

TObjectPtr<AActor> UMassActorSpawnerSubsystem::FindActorByName(const FName ActorName, ULevel* OverrideLevel) const
{
	TObjectPtr<AActor> FoundActor;
	check(GetWorld());
	OverrideLevel = OverrideLevel ? OverrideLevel : GetWorld()->GetCurrentLevel();
	
	if (UObject* FoundObject = StaticFindObjectFast(nullptr, OverrideLevel, ActorName))
	{
		FoundActor = Cast<AActor>(FoundObject);
		if (FoundObject)
		{
			if (IsValid(FoundActor) == false)
			{
				FoundActor->ClearGarbage();
			}
		}
	}
	return FoundActor;
}

ESpawnRequestStatus UMassActorSpawnerSubsystem::SpawnActor(const FSpawnArgs& Args, FActorSpawnParameters& InOutSpawnParameters) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassActorSpawnerSubsystem::SpawnActor);

	if (UE::Mass::Actors::bUseGUIDToNameActors && Args.Request.Guid.IsValid())
	{
		// offsetting `D` by 1 since `0` has special meaning for FNames
		InOutSpawnParameters.Name = FName(FString::Printf(TEXT("%s_%ud_%ud_%ud"), *Args.Request.Template->GetName(), Args.Request.Guid.A, Args.Request.Guid.B, Args.Request.Guid.C), Args.Request.Guid.D + 1);
		//InOutSpawnParameters.OverrideLevel = InOutSpawnParameters.OverrideLevel ? OverrideLevel : World->GetCurrentLevel();

		Args.Request.SpawnedActor = FindActorByName(InOutSpawnParameters.Name, InOutSpawnParameters.OverrideLevel ? InOutSpawnParameters.OverrideLevel : Args.World->GetCurrentLevel());
		if (Args.Request.SpawnedActor)
		{
			Args.Request.SpawnedActor->SetActorEnableCollision(true);
			Args.Request.SpawnedActor->SetActorHiddenInGame(false);
			return ESpawnRequestStatus::Succeeded;
		}
	}

	InOutSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	Args.Request.SpawnedActor = Args.World->SpawnActor<AActor>(Args.Request.Template, Args.Request.Transform, InOutSpawnParameters);
	if (IsValid(Args.Request.SpawnedActor))
	{
#if WITH_EDITOR
		const FMassCreatedBySpawner* CreatedBy = Args.EntityManager.GetFragmentDataPtr<FMassCreatedBySpawner>(Args.Request.MassAgent);
		if (CreatedBy && CreatedBy->Spawner)
		{
			const FString& FolderName = CreatedBy->Spawner->GetActorLabel();
			Args.Request.SpawnedActor->SetFolderPath(FName(FolderName));
		}
#endif
		return ESpawnRequestStatus::Succeeded;
	}
	else
	{
		// World->SpawnActor can hand back a non-null but pending-kill actor (e.g. construction script
		// destroyed it, or world is tearing down). Callers - and the assert at MassRepresentationSubsystem
		// case ESpawnRequestStatus::Failed - require the invariant: Failed implies SpawnedActor==nullptr.
		Args.Request.SpawnedActor = nullptr;
		return ESpawnRequestStatus::Failed;
	}
}

void UMassActorSpawnerSubsystem::ProcessPendingSpawningRequest(const double MaxTimeSlicePerTick)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassActorSpawnerSubsystem::ProcessPendingSpawningRequest);

	SpawnRequestHandleManager.ShrinkHandles();

	const double TimeSliceEnd = FPlatformTime::Seconds() + MaxTimeSlicePerTick;

	const int32 IterationsLimit = SpawnRequestHandleManager.CalcNumUsedHandles();
	int32 IterationsCount = 0;

	TNotNull<UWorld*> World = GetWorld();
	FMassEntityManager& EntityManager = World->GetSubsystem<UMassEntitySubsystem>()->GetMutableEntityManager();

	while (FPlatformTime::Seconds() < TimeSliceEnd && IterationsCount++ < IterationsLimit)
	{
		const FMassActorSpawnRequestHandle SpawnRequestHandle = GetNextRequestToSpawn(StartingHandleIndex);

		// getting an invalid handle is fine - it indicates no more handles are there to be considered. 
		if (!SpawnRequestHandle.IsValid() 
			|| !SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle))
		{
			return;
		}

		FStructView SpawnRequestView = SpawnRequests[SpawnRequestHandle.GetIndex()];

		FSpawnArgs Args =
		{
			.World         = World,
			.EntityManager = EntityManager,
			.RequestHandle = SpawnRequestHandle,
			.RequestView   = SpawnRequestView,
			.Request       = SpawnRequestView.Get<FMassActorSpawnRequest>(),
		};

		if (!ensureMsgf(Args.Request.SpawnStatus == ESpawnRequestStatus::Pending ||
						Args.Request.SpawnStatus == ESpawnRequestStatus::RetryPending, TEXT("GetNextRequestToSpawn returned a request that was already processed, need to return only request with pending status.")))
		{
			return;
		}

		ESpawnRequestStatus Result = ProcessSpawnRequestImpl(Args);
		ensureMsgf(Result != ESpawnRequestStatus::None, TEXT("Getting ESpawnRequestStatus::None as a result in this context is unexpected. Needs to be investigated."));
	}
}

ESpawnRequestStatus UMassActorSpawnerSubsystem::ProcessSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle)
{
	if (!SpawnRequestHandle.IsValid()
		|| !SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle))
	{
		return ESpawnRequestStatus::None;
	}

	FStructView SpawnRequestView = SpawnRequests[SpawnRequestHandle.GetIndex()];
	FMassActorSpawnRequest& SpawnRequest = SpawnRequestView.Get<FMassActorSpawnRequest>();

	return ProcessSpawnRequest(SpawnRequestHandle, SpawnRequestView, SpawnRequest);
}

ESpawnRequestStatus UMassActorSpawnerSubsystem::ProcessSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle, FStructView SpawnRequestView, FMassActorSpawnRequest& SpawnRequest)
{
	UWorld* World = GetWorld();
	FSpawnArgs Args =
	{
		.World         = World,
		.EntityManager = World->GetSubsystem<UMassEntitySubsystem>()->GetMutableEntityManager(),
		.RequestHandle = SpawnRequestHandle,
		.RequestView   = SpawnRequestView,
		.Request       = SpawnRequest,
	};
	return ProcessSpawnRequestImpl(Args);
}

ESpawnRequestStatus UMassActorSpawnerSubsystem::ProcessSpawnRequestImpl(const FSpawnArgs& Args)
{
	if (!ensureMsgf(Args.Request.IsFinished() == false, TEXT("Finished spawn requests are not expected to be processed again. Bailing out.")))
	{
		// returning None rather than the actual SpawnRequest.SpawnStatus to indicate the issue has occurred. 
		return ESpawnRequestStatus::None;
	}

	// Do the spawning
	Args.Request.SpawnStatus = ESpawnRequestStatus::Processing;

	// Call the pre spawn delegate on the spawn request
	if (Args.Request.ActorPreSpawnDelegate.IsBound())
	{
		Args.Request.ActorPreSpawnDelegate.Execute(Args.RequestHandle, Args.RequestView);
	}

	Args.Request.SpawnStatus = SpawnOrRetrieveFromPool(Args);

	if (Args.Request.IsFinished())
	{
		if (Args.Request.SpawnStatus == ESpawnRequestStatus::Succeeded && IsValid(Args.Request.SpawnedActor))
		{
			if (UMassAgentComponent* AgentComp = Args.Request.SpawnedActor->FindComponentByClass<UMassAgentComponent>())
			{
				AgentComp->SetPuppetHandle(Args.Request.MassAgent);
			}
		}

		EMassActorSpawnRequestAction PostAction = EMassActorSpawnRequestAction::Remove;

		// Call the post spawn delegate on the spawn request
		if (Args.Request.ActorPostSpawnDelegate.IsBound())
		{
			PostAction = Args.Request.ActorPostSpawnDelegate.Execute(Args.RequestHandle, Args.RequestView);
		}

		if (PostAction == EMassActorSpawnRequestAction::Remove)
		{
			// If notified, remove the spawning request
			ensureMsgf(SpawnRequestHandleManager.RemoveHandle(Args.RequestHandle), TEXT("When providing a delegate, the spawn request gets automatically removed, no need to remove it on your side"));
		}
	}
	else
	{
		// lower priority
		Args.Request.SpawnStatus = ESpawnRequestStatus::RetryPending;
	}

	return Args.Request.SpawnStatus;
}

void UMassActorSpawnerSubsystem::ProcessPendingDestruction(const double MaxTimeSlicePerTick)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassActorSpawnerSubsystem::ProcessPendingDestruction);

	UWorld* World = GetWorld();
	check(World);

	const ENetMode CurrentWorldNetMode = World->GetNetMode();
	const bool bHasToDestroyAllActorsOnServerSide = CurrentWorldNetMode != NM_Client && CurrentWorldNetMode != NM_Standalone;
	const double TimeSliceEnd = FPlatformTime::Seconds() + MaxTimeSlicePerTick;

	{
		// Try release to pool actors or destroy them
		TRACE_CPUPROFILER_EVENT_SCOPE(DestroyActors);
		while ((DeactivatedActorsToDestroy.Num() || ActorsToDestroy.Num()) && 
			   (bHasToDestroyAllActorsOnServerSide || FPlatformTime::Seconds() <= TimeSliceEnd))
		{
			AActor* ActorToDestroy = DeactivatedActorsToDestroy.Num() ? DeactivatedActorsToDestroy.Pop(EAllowShrinking::No) : ActorsToDestroy.Pop(EAllowShrinking::No);
			if (ActorToDestroy && !ReleaseActorToPool(ActorToDestroy))
			{
				// Couldn't release actor back to pool, so destroy it
				ConditionalDestroyActor(*World, *ActorToDestroy);
				--NumActorSpawned;
			}
		}
	}

	if(ActorsToDestroy.Num())
	{
		// Try release to pool remaining actors or deactivate them
		TRACE_CPUPROFILER_EVENT_SCOPE(DeactivateActors);
		for (AActor* ActorToDestroy : ActorsToDestroy)
		{
			if (!ReleaseActorToPool(ActorToDestroy))
			{
				// Couldn't release actor back to pool, do simple deactivate instead
				ActorToDestroy->SetActorEnableCollision(false);
				ActorToDestroy->SetActorHiddenInGame(true);
				ActorToDestroy->SetActorTickEnabled(false);
				DeactivatedActorsToDestroy.Add(ActorToDestroy);
			}
		}
		ActorsToDestroy.Reset();
	}
}

void UMassActorSpawnerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UMassSimulationSubsystem* SimSystem = Collection.InitializeDependency<UMassSimulationSubsystem>();
	check(SimSystem);

	Super::Initialize(Collection);

	SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassActorSpawnerSubsystem::OnPrePhysicsPhaseStarted);
	SimSystem->GetOnProcessingPhaseFinished(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassActorSpawnerSubsystem::OnPrePhysicsPhaseFinished);
}

void UMassActorSpawnerSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (const UWorld* World = GetWorld())
	{
		if (UMassSimulationSubsystem* SimSystem = UWorld::GetSubsystem<UMassSimulationSubsystem>(World))
		{
			SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).RemoveAll(this);
			SimSystem->GetOnProcessingPhaseFinished(EMassProcessingPhase::PrePhysics).RemoveAll(this);
		}
	}
}

void UMassActorSpawnerSubsystem::OnPrePhysicsPhaseStarted(const float DeltaSeconds)
{
	// Spawn actors before processors run so they can operate on deterministic actor state for the frame
	// 
	// Note: MassRepresentationProcessor relies on actor spawns being processed before it runs so it can confirm the
	// spawn and clean up the previous representation  
	const UMassSimulationSettings* SimSettings = GetDefault<UMassSimulationSettings>();
	check(SimSettings);
	ProcessPendingSpawningRequest(SimSettings->DesiredActorSpawningTimeSlicePerTick);

	CSV_CUSTOM_STAT(MassActors, NumSpawned, NumActorSpawned, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(MassActors, NumPooled, NumActorPooled, ECsvCustomStatOp::Accumulate);
}

void UMassActorSpawnerSubsystem::OnPrePhysicsPhaseFinished(const float DeltaSeconds)
{
	// Destroy any actors queued for destruction this frame and hide any we didn't get to within the max processing time
	// 
	// Note: MassRepresentationProcessor relies on actor destruction processing after it runs so it can clean up
	// unwanted actor representations that it has replaced for this frame. It also relies on this running before physics
	// so unwanted representations don't interfere with new physics enabled actors 
	const UMassSimulationSettings* SimSettings = GetDefault<UMassSimulationSettings>();
	check(SimSettings);
	ProcessPendingDestruction(SimSettings->DesiredActorDestructionTimeSlicePerTick);
}

void UMassActorSpawnerSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMassActorSpawnerSubsystem* MASS = Cast<UMassActorSpawnerSubsystem>(InThis);
	if (MASS)
	{
		for (auto It = MASS->PooledActors.CreateIterator(); It; ++It)
		{
			Collector.AddReferencedObjects<AActor>(It.Value());
		}
	}

	Super::AddReferencedObjects(InThis, Collector);
}

void UMassActorSpawnerSubsystem::EnableActorPooling() 
{ 
	bActorPoolingEnabled = true; 
}

void UMassActorSpawnerSubsystem::DisableActorPooling() 
{
	bActorPoolingEnabled = false;

	ReleaseAllResources();

}

bool UMassActorSpawnerSubsystem::IsActorPoolingEnabled()
{
	return UE::Mass::Actors::bUseActorPooling && bActorPoolingEnabled;
}

void UMassActorSpawnerSubsystem::ReleaseAllResources()
{
	if (UWorld* World = GetWorld())
	{
		for (auto It = PooledActors.CreateIterator(); It; ++It)
		{
			auto& ActorArray = It.Value();
			for (int i = 0; i < ActorArray.Num(); i++)
			{
				if (ActorArray[i])
				{
					ConditionalDestroyActor(*World, *ActorArray[i]);
				}
			}
			NumActorSpawned -= ActorArray.Num();
		}
	}
	PooledActors.Empty();

	NumActorPooled = 0;
	CSV_CUSTOM_STAT(MassActors, NumSpawned, NumActorSpawned, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(MassActors, NumPooled, NumActorPooled, ECsvCustomStatOp::Accumulate);
}
