// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassActorSubsystem.h"
#include "MassCommonTypes.h"
#include "MassActorTypes.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassSimulationSubsystem.h"
#include "VisualLogger/VisualLogger.h"

//----------------------------------------------------------------------//
//  FMassActorFragment 
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassActorSubsystem)

void FMassActorFragment::SetAndUpdateHandleMap(const FMassEntityHandle MassAgent, AActor* InActor, const bool bInIsOwnedByMass)
{
	SetNoHandleMapUpdate(MassAgent, InActor, bInIsOwnedByMass);

	if (InActor)
	{
		UWorld* World = InActor->GetWorld();
		check(World);
		if (UMassActorSubsystem* MassActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(World))
		{
			MassActorSubsystem->SetHandleForActor(InActor, MassAgent);
		}
	}
}

void FMassActorFragment::ResetAndUpdateHandleMap(UMassActorSubsystem* CachedActorSubsystem)
{
	if (AActor* ActorPtr = Cast<AActor>(Actor.Get()))
	{
		if (CachedActorSubsystem == nullptr)
		{
			UWorld* World = Actor->GetWorld();
			UE_CLOGF(World == nullptr, LogMassActor, Warning, "%s: got Null while fetching World for actor %ls. Can cause issues down the line. Pass in the optional CachedActorSubsystem parameter to address"
				, __FUNCTION__, *GetNameSafe(ActorPtr));
			CachedActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(World);
		}

		if (CachedActorSubsystem)
		{
			CachedActorSubsystem->RemoveHandleForActor(ActorPtr);
		}
	}

	ResetNoHandleMapUpdate();
}

void FMassActorFragment::SetNoHandleMapUpdate(const FMassEntityHandle MassAgent, AActor* InActor, const bool bInIsOwnedByMass)
{
	check(InActor);
	check(!Actor.IsValid());
	check(MassAgent.IsValid());
	Actor = InActor;
	bIsOwnedByMass = bInIsOwnedByMass;
}

void FMassActorFragment::ResetNoHandleMapUpdate()
{
	Actor.Reset();
	bIsOwnedByMass = false;
}

AActor* FMassActorFragment::GetMutable(EActorAccess Access)
{
	switch (Access)
	{
	case EActorAccess::OnlyWhenAlive:
		return Actor.Get();
	case EActorAccess::IncludePendingKill:
		return Actor.Get(true);
	case EActorAccess::IncludeUnreachable:
		return Actor.GetEvenIfUnreachable();
	default:
		checkf(false, TEXT("Invalid ActorAccess value: %i."), static_cast<int32>(Access));
		return nullptr;
	}
}

const AActor* FMassActorFragment::Get(EActorAccess Access) const
{
	return const_cast<FMassActorFragment*>(this)->GetMutable(Access);
}

//----------------------------------------------------------------------//
//  UMassActorSubsystem 
//----------------------------------------------------------------------//
void UMassActorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	// making sure UMassSimulationSubsystem gets created before the MassActorSubsystem
	Collection.InitializeDependency<UMassSimulationSubsystem>();
	
	if (UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(GetWorld()))
	{
		ActorManager = MakeShareable(new FMassActorManager(EntitySubsystem->GetMutableEntityManager().AsShared()));
	}

	OverrideSubsystemTraits<UMassActorSubsystem>(Collection);
}

void UMassActorSubsystem::Deinitialize()
{
	ActorManager.Reset();
	Super::Deinitialize();
}

//----------------------------------------------------------------------//
//  FMassActorManager
//----------------------------------------------------------------------//
FMassActorManager::FMassActorManager(const TSharedPtr<FMassEntityManager>& InEntityManager, UObject* InOwner)
	: EntityManager(InEntityManager)
	, Owner(InOwner)
{

}

FMassEntityHandle FMassActorManager::GetEntityHandleFromActor(const TObjectKey<const AActor> Actor)
{
	checkSlow(EntityManager);

	UE_MT_SCOPED_READ_ACCESS(ActorHandleMapDetector);
	FMassEntityHandle* Entity = ActorHandleMap.Find(Actor);
	if (Entity == nullptr || EntityManager->IsEntityValid(*Entity) == false)
	{
		return FMassEntityManager::InvalidEntity;
	}

	checkCode
	(
		const TObjectKey<const AActor> ActorKey(GetActorFromHandle(*Entity, FMassActorFragment::EActorAccess::IncludeUnreachable));
		checkf(ActorKey == Actor || ActorKey == TObjectKey<const AActor>()
			, TEXT("Actor associated to the Entity handle should either match or not be set (FMassActorFragment/UMassAgentComponent not used)"));
	);

	return *Entity;
}

AActor* FMassActorManager::GetActorFromHandle(const FMassEntityHandle Handle, FMassActorFragment::EActorAccess Access) const
{
	check(EntityManager);
	FMassActorFragment* Data = EntityManager->GetFragmentDataPtr<FMassActorFragment>(Handle);
	return Data != nullptr ? Data->GetMutable(Access) : nullptr;
}

void FMassActorManager::SetHandleForActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle)
{
	UE_MT_SCOPED_WRITE_ACCESS(ActorHandleMapDetector);
	ActorHandleMap.Add(Actor, Handle);
}

void FMassActorManager::RemoveHandleForActor(const TObjectKey<const AActor> Actor)
{
	UE_MT_SCOPED_WRITE_ACCESS(ActorHandleMapDetector);
	ActorHandleMap.Remove(Actor);
}

void FMassActorManager::DisconnectActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle)
{
	if (Handle.IsValid() == false)
	{
		return;
	}

	FMassEntityHandle FoundEntity;
	{
		UE_MT_SCOPED_WRITE_ACCESS(ActorHandleMapDetector);
		// We're assuming the Handle does match Actor, so we're RemoveAndCopyValue. If if doesn't we'll add it back.
		// The expectation is that this won't happen on a regular basis..
		if (ActorHandleMap.RemoveAndCopyValue(Actor, FoundEntity) == false)
		{
			// the entity doesn't match the actor
			return;
		}
	}

	if (FoundEntity == Handle)
	{
		check(EntityManager);
		if (FMassActorFragment* Data = EntityManager->GetFragmentDataPtr<FMassActorFragment>(Handle))
		{
			Data->ResetAndUpdateHandleMap();
			// Since we're disconnecting the actor, the entity should no longer be registered in the MassActorSubsystem (or MassActorRepresentationProcessor).
			EntityManager->RemoveFragmentFromEntity(Handle, FMassActorFragment::StaticStruct());
		}
	}
	else
	{
		// unexpected mismatch. Add back and notify.
		UE_VLOG_UELOG(Owner.Get(), LogMass, Warning, TEXT("%s: Trying to disconnect actor %s while the Handle given doesn't match the system\'s records")
			, ANSI_TO_TCHAR(__FUNCTION__), *AActor::GetDebugName(Actor.ResolveObjectPtr()));
		SetHandleForActor(Actor, Handle);
	}
}
