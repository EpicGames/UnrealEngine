// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IMassActorsModule.h"


class FMassActorsModule : public IMassActorsModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassActorsModule, MassActors)



void FMassActorsModule::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void FMassActorsModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}



