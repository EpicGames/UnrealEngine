// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAgentComponent.h"
#include "EngineUtils.h"
#include "MassEntityView.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassCommonTypes.h"
#include "MassAgentSubsystem.h"
#include "VisualLogger/VisualLogger.h"
#include "MassActorSubsystem.h"
#include "MassSpawner.h"
#include "Net/UnrealNetwork.h"
#include "MassReplicationFragments.h"
#include "MassMovementFragments.h"
#include "GameFramework/CharacterMovementComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAgentComponent)

namespace UE::Mass::Debug
{
	bool bBreakOnFailedAgentCheck = true;

	namespace
	{
		FAutoConsoleVariableRef CVars[] = 
		{
			{TEXT("mass.debug.BreakOnFailedAgentCheck"), bBreakOnFailedAgentCheck, TEXT("Enabled by default. If disabled, won't cause check failures when AgentComponent is in an unexpected state.")}
		};
	}
}

#define MASSAGENT_CHECK( condition, Format, ... ) \
	UE_CVLOG(!(condition), GetOwner(), LogMass, Error, Format, ##__VA_ARGS__); \
	checkf(condition || !UE::Mass::Debug::bBreakOnFailedAgentCheck, Format, ##__VA_ARGS__);

//----------------------------------------------------------------------//
// UMassAgentComponent
//----------------------------------------------------------------------//
UMassAgentComponent::UMassAgentComponent()
	: EntityConfig(*this)
{
#if WITH_EDITORONLY_DATA
	bAutoRegisterInEditorMode = true;
#endif // WITH_EDITORONLY_DATA
	bAutoRegister = true;
	State = EAgentComponentState::None;
	SetIsReplicatedByDefault(true);
}

#if WITH_EDITOR
void UMassAgentComponent::PostInitProperties() 
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad) || GetOuter() == nullptr || GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return;
	}
	
	if (GetWorld())
	{
		bAutoRegister = bAutoRegisterInEditorMode || GetWorld()->IsGameWorld();
	}
}
void UMassAgentComponent::PostLoad()
{
	Super::PostLoad();
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || GetOuter() == nullptr || GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return;
	}

	if (GetWorld())
	{
		bAutoRegister = bAutoRegisterInEditorMode || GetWorld()->IsGameWorld();
	}
}

void UMassAgentComponent::PostDuplicate(const bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	EntityConfig.PostDuplicate(bDuplicateForPIE);
}

void UMassAgentComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_EntityConfig = GET_MEMBER_NAME_CHECKED(UMassAgentComponent, EntityConfig);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == NAME_EntityConfig)
	{
		EntityConfig.ForceRegenerateGUID();
	}
}
#endif // WITH_EDITOR

void UMassAgentComponent::OnRegister()
{
	Super::OnRegister();

	if (IsRunningCommandlet() || IsRunningCookCommandlet() || GIsCookerLoadingPackage)
	{
		// ignore, we're not doing any registration while cooking or running a commandlet
		return;
	}

	if (GetOuter() == nullptr || GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || HasAnyFlags(RF_ArchetypeObject))
	{
		// we won't try registering a CDO's component with Mass
		ensure(false && "temp, wanna know this happened");
		return;
	}

	UWorld* World = GetWorld();
	if (World == nullptr || World->WorldType == EWorldType::None || World->WorldType == EWorldType::Inactive
#if WITH_EDITOR
		|| World->IsPreviewWorld() || (bAutoRegisterInEditorMode == false && World->IsGameWorld() == false)
#endif // WITH_EDITOR
		)
	{
		// we don't care about preview worlds. Those are transient, temporary worlds like the one created when opening a BP editor.
		return;
	}

	// @todo hook up to pawn possessing stuff, maybe?
	RegisterWithAgentSubsystem();
}

bool UMassAgentComponent::IsReadyForPooling() const
{
	// If we're waiting for puppet initialization, we could have some bad interactions
	if (IsPuppetPendingInitialization())
	{
		return false;
	}

	return true;
}

void UMassAgentComponent::RegisterWithAgentSubsystem()
{
	UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld());
	UE_CVLOG_UELOG(AgentSubsystem == nullptr, GetOwner(), LogMass, Error, TEXT("Unable to find UMassAgentSubsystem instance. Make sure the world is initialized"));
	if (ensureMsgf(AgentSubsystem, TEXT("Unable to find UMassAgentSubsystem instance. Make sure the world is initialized")))
	{
		TemplateID = AgentSubsystem->RegisterAgentComponent(*this);
	}
}

void UMassAgentComponent::UnregisterWithAgentSubsystem()
{
	UWorld* World = GetWorld();
	if (World && World->bIsTearingDown)
	{
		// no need to Unregister during world teardown
		return;
	}
	if (State != EAgentComponentState::None)
	{
		if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(World))
		{
			UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
			AgentSubsystem->ShutdownAgentComponent(*this);
			MASSAGENT_CHECK(State == EAgentComponentState::None ||
				State == EAgentComponentState::PuppetPaused ||
				State == EAgentComponentState::PuppetReplicatedOrphan,
				TEXT("%s is expecting to be in state[None|PuppetPaused|PuppetReplicatedOrphan] state but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
		}
	}
	DebugCheckStateConsistency();
	if (AgentHandle.IsValid())
	{
		ClearEntityHandleInternal();
	}

	State = EAgentComponentState::None;
	TemplateID = FMassEntityTemplateID();
}

void UMassAgentComponent::OnUnregister()
{
	UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
	UnregisterWithAgentSubsystem();

	Super::OnUnregister();
}

void UMassAgentComponent::SetEntityHandle(const FMassEntityHandle NewHandle)
{
	MASSAGENT_CHECK(State == EAgentComponentState::EntityPendingCreation,
		TEXT("%s is expecting to be in state[EntityPendingCreation] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	SetEntityHandleInternal(NewHandle);
	SwitchToState(EAgentComponentState::EntityCreated);
}

void UMassAgentComponent::SetEntityHandleInternal(const FMassEntityHandle NewHandle)
{
	ensureMsgf((AgentHandle.IsValid() && NewHandle.IsValid()) == false, TEXT("Overriding an existing entity ID might result in a dangling entity still affecting the simulation"));
	AgentHandle = NewHandle;

	UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(GetWorld());
#if	UE_REPLICATION_COMPILE_SERVER_CODE
	// Fetch NetID if it exist
	if (EntitySubsystem)
	{
		if (const FMassNetworkIDFragment* NetIDFragment = EntitySubsystem->GetEntityManager().GetFragmentDataPtr<FMassNetworkIDFragment>(AgentHandle))
		{
			if (!IsNetSimulating())
			{
				NetID = NetIDFragment->NetID;
			}
			else
			{
				check(NetID == NetIDFragment->NetID);
			}
		}
	}
#endif // UE_REPLICATION_COMPILE_SERVER_CODE

	if (const UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld()))
	{
		AgentSubsystem->NotifyMassAgentComponentEntityAssociated(*this);
	}

	// Sync up with mass
	if (EntitySubsystem)
	{
		if (IsNetSimulating())
		{
			const FMassEntityView EntityView(EntitySubsystem->GetEntityManager(), AgentHandle);

			// @todo Find a way to add these initialization into either translator initializer or adding new fragments
			// Make sure to fetch the fragment after any release, as that action can move the entity around into new archetype and 
			// by the same fact change the references to the fragments.
			if (FMassActorFragment* ActorInfo = EntityView.GetFragmentDataPtr<FMassActorFragment>())
			{
				checkf(!ActorInfo->IsValid(), TEXT("Expecting ActorInfo fragment to be null"));
				ActorInfo->SetAndUpdateHandleMap(AgentHandle, GetOwner(), !IsNetSimulating()/*bIsOwnedByMass*/);
			}

			// Initialize location of the replicated actor to match the mass replicated one
			if (const FTransformFragment* TransformFragment = EntityView.GetFragmentDataPtr<FTransformFragment>())
			{
				GetOwner()->SetActorTransform(TransformFragment->GetTransform(), /*bSweep*/false, /*OutSweepHitResult*/nullptr, ETeleportType::TeleportPhysics);
			}

			// Initialize velocity of the replicated actor to match the mass replicated one
			if (const FMassVelocityFragment* Velocity = EntityView.GetFragmentDataPtr<FMassVelocityFragment>())
			{
				if (UCharacterMovementComponent* MovementComp = GetOwner()->FindComponentByClass<UCharacterMovementComponent>())
				{
					MovementComp->Velocity = Velocity->Value;
				}
			}
		}
		else if (IsPuppet() == false)
		{
			// Since this component is owned by an actor not owned by Mass (i.e. it's not a "puppet")
			// we need to manually notify UMassActorSubsystem about this entity-handle-to-actor mapping
			if (FMassActorFragment* ActorInfo = EntitySubsystem->GetEntityManager().GetFragmentDataPtr<FMassActorFragment>(AgentHandle))
			{
				checkf(!ActorInfo->IsValid(), TEXT("Expecting ActorInfo fragment to be null"));
				ActorInfo->SetAndUpdateHandleMap(AgentHandle, GetOwner(), !IsNetSimulating()/*bIsOwnedByMass*/);
			}
			// @todo consider adding the fragment with an appropriate initialization, if the fragment's missing
		}
	}
}

void UMassAgentComponent::SetPuppetHandle(const FMassEntityHandle NewHandle)
{
	MASSAGENT_CHECK(State == EAgentComponentState::EntityPendingCreation,
		TEXT("%s is expecting to be in state[EntityPendingCreation] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	DebugCheckStateConsistency();

	checkf(AgentHandle.IsValid() == false, TEXT("Can't set a new puppet handle of top of a regular agent handle. Entities would end up dangling."));
	SetEntityHandleInternal(NewHandle);

	if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld()))
	{
		AgentSubsystem->MakePuppet(*this);
	}
}

void UMassAgentComponent::PuppetInitializationPending()
{
	MASSAGENT_CHECK(State == EAgentComponentState::EntityPendingCreation ||
		State == EAgentComponentState::PuppetPendingReplication ||
		State == EAgentComponentState::PuppetPaused ||
		State == EAgentComponentState::PuppetReplicatedOrphan,
		TEXT("%s is expecting to be in state[EntityPendingCreation|PuppetPendingReplication|PuppetPaused|PuppetReplicatedOrphan] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	SwitchToState(EAgentComponentState::PuppetPendingInitialization);
}

void UMassAgentComponent::PuppetInitializationDone()
{
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPendingInitialization,
		TEXT("%s is expecting to be in state[PuppetPendingInitialization] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	SwitchToState(EAgentComponentState::PuppetInitialized);
}

void UMassAgentComponent::PuppetInitializationAborted()
{
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPendingInitialization,
		TEXT("%s is expecting to be in state[PuppetPendingInitialization] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	SwitchToState(EAgentComponentState::PuppetPaused);
}

void UMassAgentComponent::PuppetReplicationAborted()
{
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPendingReplication,
		TEXT("%s is expecting to be in state[PuppetPendingReplication] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	// No Mass entity exists yet in this state (no handle, no NetID). Just reset.
	SwitchToState(EAgentComponentState::None);
}

void UMassAgentComponent::ClearEntityHandle()
{
	MASSAGENT_CHECK(State == EAgentComponentState::EntityCreated,
		TEXT("%s is expecting to be in state[EntityCreated] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	ClearEntityHandleInternal();
	SwitchToState(EAgentComponentState::None);
}

void UMassAgentComponent::ClearEntityHandleInternal()
{
	if (UWorld* World = GetWorld())
	{
		if (const UMassAgentSubsystem* AgentSubsystem = World->GetSubsystem<UMassAgentSubsystem>())
		{
			AgentSubsystem->NotifyMassAgentComponentEntityDetaching(*this);
		}

		if (const UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>())
		{
			// Sync up with mass
			FMassActorFragment* ActorInfo = EntitySubsystem->GetEntityManager().GetFragmentDataPtr<FMassActorFragment>(AgentHandle);
			if (ActorInfo && (IsNetSimulating() == true || IsPuppet() == false))
			{
				checkf(!ActorInfo->IsValid() || ActorInfo->Get() == GetOwner(), TEXT("[%s] Expecting actor pointer to be the Component\'s owner")
					, IsNetSimulating() ? TEXT("NetSimulating") : TEXT("Non-Puppet"));
				ActorInfo->ResetAndUpdateHandleMap(World->GetSubsystem<UMassActorSubsystem>());
			}
		}
	}
	
	AgentHandle = FMassEntityManager::InvalidEntity;
}

void UMassAgentComponent::PuppetUnregistrationDone()
{	
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPaused ||
			 State == EAgentComponentState::PuppetInitialized,
		TEXT("%s is expecting to be in state[PuppetPaused|PuppetInitialized] state but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	SwitchToState(EAgentComponentState::PuppetPaused);

	PuppetSpecificAddition.Reset();

	// AgentHandle on purpose. It's possible to unregister the AgentComponent just to
	// re-register it soon after in which case having this information stored is beneficial to avoid a need to 
	// re-configure the component as a puppet
}

void UMassAgentComponent::EntityCreationPending()
{
	MASSAGENT_CHECK(State == EAgentComponentState::None,
		TEXT("%s is expecting to be in state[None] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	SwitchToState(EAgentComponentState::EntityPendingCreation);
}

void UMassAgentComponent::EntityCreationAborted()
{
	MASSAGENT_CHECK(State == EAgentComponentState::EntityPendingCreation,
		TEXT("%s is expecting to be in state[EntityPendingCreation] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
	SwitchToState(EAgentComponentState::None);
}

void UMassAgentComponent::SwitchToState(EAgentComponentState NewState)
{
	UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("Entity[%s] %s From:%s To:%s"), *AgentHandle.DebugGetDescription(), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State), *UEnum::GetValueAsString(NewState));
	State = NewState;
	DebugCheckStateConsistency();
}

void UMassAgentComponent::DebugCheckStateConsistency()
{
#if DO_CHECK
	if (GetWorld() && GetWorld()->bIsTearingDown)
	{
		// Inconsistency is expected during teardown
		return;
	}
	switch (State)
	{
		case EAgentComponentState::None:
		case EAgentComponentState::EntityPendingCreation:
			MASSAGENT_CHECK(AgentHandle.IsValid() == false, TEXT("Not expecting a valid mass agent handle in state %s"), *UEnum::GetValueAsString(State));
			break;
		case EAgentComponentState::EntityCreated:
			MASSAGENT_CHECK(AgentHandle.IsValid() == true, TEXT("Expecting a valid mass agent handle in state %s"), *UEnum::GetValueAsString(State));
			break;
		case EAgentComponentState::PuppetPendingInitialization:
		case EAgentComponentState::PuppetInitialized:
		case EAgentComponentState::PuppetPaused:
		{
			const bool bValidAgentHandle = AgentHandle.IsValid();
			MASSAGENT_CHECK(bValidAgentHandle, TEXT("Expecting a valid mass agent handle in state %s"), *UEnum::GetValueAsString(State));
			if (bValidAgentHandle)
			{
				if (const UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(GetWorld()))
				{
					const FMassEntityManager& EntityManager = EntitySubsystem->GetEntityManager();
					const bool bIsValidEntity = EntityManager.IsEntityValid(AgentHandle);
					MASSAGENT_CHECK(bIsValidEntity, TEXT("Exepecting a valid entity in state %s"), *UEnum::GetValueAsString(State))
					if (bIsValidEntity)
					{
						const bool bIsBuiltEntity = EntityManager.IsEntityBuilt(AgentHandle);
						MASSAGENT_CHECK(bIsBuiltEntity, TEXT("Expecting a fully built entity in state %s"), *UEnum::GetValueAsString(State));
						if (bIsBuiltEntity)
						{
							AActor* Owner = GetOwner();
							const AActor* Actor = EntityManager.GetFragmentDataChecked<FMassActorFragment>(AgentHandle).Get();
							MASSAGENT_CHECK(Actor == nullptr || Actor == Owner, TEXT("Mass Actor and Owner mismatched in state %s"), *UEnum::GetValueAsString(State));
						}
					}
				}
			}
			break;
		}
		case EAgentComponentState::PuppetPendingReplication:
			MASSAGENT_CHECK(IsNetSimulating(), TEXT("Expecting to be a replicated none authoritative actor in state %s"), *UEnum::GetValueAsString(State));
			MASSAGENT_CHECK(AgentHandle.IsValid() == false, TEXT("Not expecting a valid mass agent handle in state %s"), *UEnum::GetValueAsString(State));
			MASSAGENT_CHECK(NetID.IsValid() == false, TEXT("Not expecting a valid net id in state %s"), *UEnum::GetValueAsString(State));
			break;
		case EAgentComponentState::PuppetReplicatedOrphan:
			MASSAGENT_CHECK(IsNetSimulating(), TEXT("Expecting to be a replicated none authoritative actor in state %s"), *UEnum::GetValueAsString(State));
			MASSAGENT_CHECK(AgentHandle.IsValid() == false, TEXT("Not expecting a valid mass agent handle in state %s"), *UEnum::GetValueAsString(State));
			MASSAGENT_CHECK(NetID.IsValid() == true, TEXT("Expecting a valid net id in state %s"), *UEnum::GetValueAsString(State));
			break;
		default:
			MASSAGENT_CHECK(false, TEXT("Unsuported agent component state"));
			break;
	}
#endif // DO_CHECK
}

void UMassAgentComponent::SetEntityConfig(const FMassEntityConfig& InEntityConfig)
{
	EntityConfig = InEntityConfig;
	EntityConfig.SetOwner(*this);
}

void UMassAgentComponent::Enable()
{
	if (IsRegistered() == false)
	{
		UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
		RegisterComponent();
	}
}

void UMassAgentComponent::Disable()
{
	if (IsRegistered())
	{
		UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
		UnregisterComponent();
	}
}

/* This method is evil and if it get called before while spawning the AgentHandle is not set yet and some early code was destroying actor after a certain time even if the kill entity did not worked. */
void UMassAgentComponent::KillEntity(const bool bDestroyActor)
{
	UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));

	AActor* Owner = GetOwner();
	UWorld* World = GetWorld();
	if (Owner == nullptr)
	{
		return;
	}

	// Caching the entity and if we have to disconnect actor as the next operation will invalidate that information
	const FMassEntityHandle EntityHandleToDespawn = AgentHandle;
	const bool bDisconnectActor = IsPuppet() && bDestroyActor == false;
	if (State != EAgentComponentState::None)
	{
		if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(World))
		{
			AgentSubsystem->UnregisterAgentComponent(*this);
			MASSAGENT_CHECK(State == EAgentComponentState::None ||
				State == EAgentComponentState::PuppetPaused,
				TEXT("%s is expecting to be in state[None|PuppetPaused] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));
			// We need to clear the entity now as it will be despawned below.
			ClearEntityHandleInternal();
			// Since we cleared the entity handle, need to switch to none state to prevent the call to UnregisterAgentComponent upon actor destruction.
			SwitchToState(EAgentComponentState::None);
		}
	}

	if (bDisconnectActor)
	{
		// break connection between entity and actor so that the actor doesn't get destroyed as part of the entity's 
		// removal. Removal is to be expected for puppet actors.
		if (UMassActorSubsystem* ActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(World))
		{
			ActorSubsystem->DisconnectActor(Owner, EntityHandleToDespawn);
		}
	}

	// @todo temp hack to utilize the same path as Spawners (i.e. using EntityTemplate's deinitialization pipeline). 
	// this will go away once we switch over to system-component based approach or monitors
	if (EntityHandleToDespawn.IsValid())
	{
		for (TActorIterator<AMassSpawner> It(World); It; ++It)
		{
			(*It)->DespawnEntity(EntityHandleToDespawn);
		}
	}
}

void UMassAgentComponent::PausePuppet(const bool bPause)
{
	if (ensureMsgf(IsPuppet(), TEXT("%s can only be called when the the mass agent component acts as a puppet. Current state is %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State))
		&& IsPuppetPaused() != bPause)
	{
		if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld()))
		{
			if (bPause)
			{
				MASSAGENT_CHECK(State == EAgentComponentState::PuppetPendingInitialization ||
					State == EAgentComponentState::PuppetInitialized,
					TEXT("%s(true) is expecting to be in state[PuppetPendingInitialization|PuppetInitialized] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

				AgentSubsystem->UnregisterAgentComponent(*this);
			}
			else
			{
				MASSAGENT_CHECK(State == EAgentComponentState::PuppetPaused,
					TEXT("%s(false) is expecting to be in state[PuppetPaused] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

				AgentSubsystem->RegisterAgentComponent(*this);
			}
		}
	}
	DebugCheckStateConsistency();
}

void UMassAgentComponent::PuppetReplicationPending()
{
	checkf(IsNetSimulating(), TEXT("Expecting a replicated puppet"));
	MASSAGENT_CHECK(State == EAgentComponentState::None,
		TEXT("%s is expecting to be in state[None] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	SwitchToState(EAgentComponentState::PuppetPendingReplication);
}

void UMassAgentComponent::SetReplicatedPuppetHandle(FMassEntityHandle NewHandle)
{
	checkf(IsNetSimulating(), TEXT("Expecting a replicated puppet"));
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPendingReplication ||
		State == EAgentComponentState::PuppetReplicatedOrphan,
		TEXT("%s is expecting to be in state [PuppetPendingReplication|PuppetReplicatedOrphan] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	SetEntityHandleInternal(NewHandle);
}

void UMassAgentComponent::ClearReplicatedPuppetHandle()
{
	checkf(IsNetSimulating(), TEXT("Expecting a replicated puppet"));
	MASSAGENT_CHECK(State == EAgentComponentState::PuppetPaused,
		TEXT("%s is expecting to be in state [PuppetPaused] but is in %s"), ANSI_TO_TCHAR(__FUNCTION__), *UEnum::GetValueAsString(State));

	if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld()))
	{
		UE_VLOG(GetOwner(), LogMass, Verbose, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
		AgentSubsystem->UnregisterAgentComponent(*this);
	}
	ClearEntityHandleInternal();
	SwitchToState(EAgentComponentState::PuppetReplicatedOrphan);
}

void UMassAgentComponent::MakePuppetAReplicatedOrphan()
{
	checkf(IsNetSimulating(), TEXT("Expecting a replicated puppet"));

	SwitchToState(EAgentComponentState::PuppetReplicatedOrphan);
}

void UMassAgentComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams SharedParams;
	DOREPLIFETIME_WITH_PARAMS_FAST(UMassAgentComponent, NetID, SharedParams);
}

void UMassAgentComponent::OnRep_NetID()
{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	if (UMassAgentSubsystem* AgentSubsystem = UWorld::GetSubsystem<UMassAgentSubsystem>(GetWorld()))
	{
		AgentSubsystem->NotifyMassAgentComponentReplicated(*this);
	}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
}
