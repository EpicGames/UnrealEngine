// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAgentSubsystem.h"
#include "MassCommandBuffer.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassCommonTypes.h"
#include "MassSimulationSubsystem.h"
#include "MassSpawnerSubsystem.h"
#include "MassActorTypes.h"
#include "MassAgentComponent.h"
#include "MassEntityTemplateRegistry.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityView.h"
#include "MassReplicationSubsystem.h"
#include "Engine/NetDriver.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAgentSubsystem)

namespace FMassAgentSubsystemHelper
{

inline void InitializeAgentComponentFragments(const UMassAgentComponent& AgentComp, FMassEntityView& EntityView, const EMassTranslationDirection Direction
	, TConstArrayView<FMassEntityTemplateData::FObjectFragmentInitializerFunction> ObjectFragmentInitializers)
{
	AActor* Owner = AgentComp.GetOwner();
	check(Owner);
	for (const FMassEntityTemplateData::FObjectFragmentInitializerFunction& Initializer : ObjectFragmentInitializers)
	{
		Initializer(*Owner, EntityView, Direction);
	}
}

}

//----------------------------------------------------------------------//
// UMassAgentSubsystem 
//----------------------------------------------------------------------//
void UMassAgentSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// making sure UMassSimulationSubsystem gets created before the MassAgentSubsystem
	Collection.InitializeDependency<UMassSimulationSubsystem>();
	Collection.InitializeDependency<UMassSpawnerSubsystem>();
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	Collection.InitializeDependency<UMassReplicationSubsystem>();
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
	
	UWorld* World = GetWorld();
	check(World);

	EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World).AsShared();

	SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(World);

	SimulationSystem = UWorld::GetSubsystem<UMassSimulationSubsystem>(World);
	SimulationSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassAgentSubsystem::OnProcessingPhaseStarted, EMassProcessingPhase::PrePhysics);

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	ReplicationSubsystem = UWorld::GetSubsystem<UMassReplicationSubsystem>(World);
	check(ReplicationSubsystem);

	ReplicationSubsystem->GetOnMassAgentAdded().AddUObject(this, &UMassAgentSubsystem::OnMassAgentAddedToReplication);
	ReplicationSubsystem->GetOnRemovingMassAgent().AddUObject(this, &UMassAgentSubsystem::OnMassAgentRemovedFromReplication);
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
}

void UMassAgentSubsystem::Deinitialize()
{
	EntityManager.Reset();
	Super::Deinitialize();
}

FMassEntityTemplateID UMassAgentSubsystem::RegisterAgentComponent(UMassAgentComponent& AgentComp)
{
	check(EntityManager);
	check(SpawnerSystem);

	if (AgentComp.IsPuppet())
	{
		MakePuppet(AgentComp);
		return AgentComp.GetTemplateID();
	}

	// This path is hit when a regular (non-puppet) actor that owns a UMassAgentComponent gets
	// unregistered (e.g. via EndPlay or RemoveFromWorld) and then registered again (e.g. re-added
	// to the world or re-activated). In that scenario the component still carries a valid
	// TemplateID from its previous registration. We intentionally bail out with a warning rather
	// than re-register, because the old Mass entity handle may still be live and double-registration
	// would create orphaned entity data. Callers that need a fresh registration should first ensure
	// the component has been fully unregistered (TemplateID invalidated) before calling this again.
	if (AgentComp.GetTemplateID().IsValid())
	{
		UE_VLOG_UELOG(this, LogMassActor, Warning, TEXT("UMassAgentSubsystem::RegisterAgentComponent called while the given agent component has already been registered (Owner: %s, entity handle %s, template ID %s)")
			, *GetNameSafe(AgentComp.GetOwner()), *AgentComp.GetEntityHandle().DebugGetDescription(), *AgentComp.GetTemplateID().ToString());
		return FMassEntityTemplateID();
	}

	AActor* AgentActor = AgentComp.GetOwner();
	check(AgentActor);
	UWorld* World = AgentActor->GetWorld();
	check(World);

	const FMassEntityConfig& EntityConfig = AgentComp.GetEntityConfig();
	const FMassEntityTemplate& EntityTemplate = EntityConfig.GetOrCreateEntityTemplate(*World);

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	if (AgentComp.IsNetSimulating())
	{
		AgentComp.PuppetReplicationPending();
	}
	else
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
	{
		FMassAgentInitializationQueue& AgentQueue = PendingAgentEntities.FindOrAdd(EntityTemplate.GetTemplateID());
		// Agent already in the queue! Earlier conditions should have failed or data is inconsistent.
		check(AgentQueue.AgentComponents.Find(&AgentComp) == INDEX_NONE);
		AgentQueue.AgentComponents.Add(&AgentComp);

		UE_VLOG(this, LogMassActor, Verbose, TEXT("%s registered and PENDING entity creation."), *AgentActor->GetName());
   		AgentComp.EntityCreationPending();
	}

	return EntityTemplate.GetTemplateID();
}

void UMassAgentSubsystem::UpdateAgentComponent(const UMassAgentComponent& AgentComp)
{
	check(EntityManager);
	check(SpawnerSystem)

	if (!ensureMsgf(AgentComp.GetEntityHandle().IsValid(), TEXT("Caling %s is valid only for already registered MassAgentComponents"), ANSI_TO_TCHAR(__FUNCTION__)))
	{
		UE_VLOG(this, LogMassActor, Warning, TEXT("%s: called while the given agent component has not been registered yet (Owner: %s)")
			, ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(AgentComp.GetOwner()));
		return;
	}

	AActor* AgentActor = AgentComp.GetOwner();
	check(AgentActor);
	UWorld* World = AgentActor->GetWorld();
	check(World);

	const FMassEntityConfig& EntityConfig = AgentComp.GetEntityConfig();
	const FMassEntityTemplate& EntityTemplate = EntityConfig.GetOrCreateEntityTemplate(*World);

	const FMassEntityHandle Entity = AgentComp.GetEntityHandle();
	const FMassArchetypeHandle CurrentArchetypeHandle = EntityManager->GetArchetypeForEntity(Entity);
	if (CurrentArchetypeHandle == EntityTemplate.GetArchetype())
	{
		UE_VLOG(this, LogMassActor, Log, TEXT("%s called for %s but no archetype changes have been found")
			, ANSI_TO_TCHAR(__FUNCTION__), *AgentActor->GetName());
	}
	else
	{
		// the tricky case: we want to move the entity over to the new archetype and initialize its fragments, 
		// but only the ones that the previous archetype didn't have - "delta initialize"
		ensureMsgf(false, TEXT("Not implemented yet"));
			
		// @todo add override flag so that we always initialize a new when moving between archetypes
	}

	check(AgentComp.GetEntityHandle().IsValid());
}

void UMassAgentSubsystem::UnregisterAgentComponent(UMassAgentComponent& AgentComp)
{
	if (!EntityManager || SpawnerSystem == nullptr || SimulationSystem == nullptr)
	{
		return;
	}

	if (AgentComp.IsPuppetReplicationPending())
	{
		// The actor was destroyed (e.g., by Iris relevancy) before its NetID was replicated
		// via the old bubble system. No Mass entity exists yet, no cleanup needed —
		// just transition to None.
		AgentComp.PuppetReplicationAborted();
	}
	else if (AgentComp.IsPuppet())
	{
		if (AgentComp.IsPuppetPendingInitialization())
		{
			FMassAgentInitializationQueue* PuppetQueue = PendingPuppets.Find(AgentComp.GetTemplateID());
		
			if (ensureMsgf(PuppetQueue, TEXT("Trying to remove a puppet's agent component from initialization queue but there's no such queue")))
			{
				ensureMsgf(PuppetQueue->AgentComponents.Remove(&AgentComp), TEXT("Trying to remove a puppet's agent component from initialization queue while it's not in the queue"));
			}
			AgentComp.PuppetInitializationAborted();
		}
		else if (ensureMsgf(AgentComp.GetEntityHandle().IsValid(), TEXT("Trying to unregister a puppet's agent component while it's neither pending initialization nor having a valid entity handle")) )
		{
			if (AgentComp.GetPuppetSpecificAddition().IsEmpty() == false)
			{
				const FMassEntityHandle Entity = AgentComp.GetEntityHandle();

			    // remove fragments that have been added for the puppet agent
			    if (EntityManager->IsProcessing())
			    {
				    // need to request via command buffer since we can't move entities while processing is happening
					FMassArchetypeCompositionDescriptor Composition = AgentComp.GetPuppetSpecificAddition();
					EntityManager->Defer().PushCommand<FMassDeferredRemoveCommand>([Entity, Composition](FMassEntityManager& InOutEntityManager)
						{
							if (InOutEntityManager.IsEntityValid(Entity) == false)
							{
								return;
							}
							InOutEntityManager.RemoveCompositionFromEntity(Entity, Composition);
						});
			    }
			    else
			    {
					EntityManager->RemoveCompositionFromEntity(Entity, AgentComp.GetPuppetSpecificAddition());
			    }
			}
			AgentComp.PuppetUnregistrationDone();
		}
	}
	else
	{
		if (AgentComp.GetEntityHandle().IsValid())
		{
			// the entity has already been created. Destroy!
			const FMassEntityTemplate* EntityTemplate = nullptr;
			AActor* AgentActor = AgentComp.GetOwner();
			UWorld* World = AgentActor ? AgentActor->GetWorld() : nullptr;
			if (ensure(World))
			{
				const FMassEntityConfig& EntityConfig = AgentComp.GetEntityConfig();
				EntityTemplate = &EntityConfig.GetEntityTemplateChecked(*World);
			}

			FMassEntityHandle Entity = AgentComp.GetEntityHandle();
			// Clearing the entity before it become invalid as the clear contains notifications
			AgentComp.ClearEntityHandle();

			// Destroy the entity
			if (EntityManager->IsProcessing())
			{
				// need to request via command buffer since we can't move entities while processing is happening
				EntityManager->Defer().DestroyEntity(Entity);
			}
			else if (ensure(EntityTemplate))
			{
				SpawnerSystem->DestroyEntities(TArrayView<FMassEntityHandle>(&Entity, 1));
			}
		}
		else if (AgentComp.IsEntityPendingCreation())
		{
			// hasn't been registered yet. Just remove it from the queue		
			FMassAgentInitializationQueue* AgentQueue = PendingAgentEntities.Find(AgentComp.GetTemplateID());
			if (ensureMsgf(AgentQueue, TEXT("Trying to remove an agent component from initialization queue but there's no such queue")))
			{
				ensureMsgf(AgentQueue->AgentComponents.Remove(&AgentComp), TEXT("Trying to remove an agent component from initialization queue it's missing from the queue"));
			}
			AgentComp.EntityCreationAborted();
		}
	}
}

void UMassAgentSubsystem::ShutdownAgentComponent(UMassAgentComponent& AgentComp)
{
	UnregisterAgentComponent(AgentComp);

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	ReplicatedAgentComponents.Remove(AgentComp.GetNetID());
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

}

void UMassAgentSubsystem::MakePuppet(UMassAgentComponent& AgentComp)
{
	if (!ensureMsgf(AgentComp.GetTemplateID().IsValid(), TEXT("%s tried to used %s as puppet but it doesn't have a valid entity template ID"), ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(AgentComp.GetOwner())))
	{
		return;
	}
		
	if (AgentComp.IsEntityPendingCreation())
	{
		FMassAgentInitializationQueue* AgentQueue = PendingAgentEntities.Find(AgentComp.GetTemplateID());
		if (ensureMsgf(AgentQueue, TEXT("Trying to remove an agent component from initialization queue but there's no such queue")))
		{
			ensureMsgf(AgentQueue->AgentComponents.Remove(&AgentComp), TEXT("Trying to remove an agent component from initialization queue it's missing from the queue"));
		}
	}

	FMassAgentInitializationQueue& PuppetQueue = PendingPuppets.FindOrAdd(AgentComp.GetTemplateID());
	// Agent already in the queue! Earlier conditions should have failed or data is inconsistent.
	check(PuppetQueue.AgentComponents.Find(&AgentComp) == INDEX_NONE);
	PuppetQueue.AgentComponents.Add(&AgentComp);

	AgentComp.PuppetInitializationPending();
}

void UMassAgentSubsystem::OnProcessingPhaseStarted(const float DeltaSeconds, const EMassProcessingPhase Phase)
{
	switch (Phase)
	{
	case EMassProcessingPhase::PrePhysics:
		// initialize what needs initialization
		if (PendingAgentEntities.Num() > 0 || PendingPuppets.Num() > 0)
		{
			HandlePendingInitialization();
		}
		break;
	default:
		// unhandled phases, by design, not every phase needs to be handled by the Actor subsystem
		break;
	}
}

void UMassAgentSubsystem::HandlePendingInitialization()
{
	check(EntityManager);
	check(SpawnerSystem);
	check(SimulationSystem);
	check(SimulationSystem->IsDuringMassProcessing() == false);

	{
		// this block is scoped so that CreationContext gets released by the end
		TSharedPtr<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->GetOrMakeCreationContext();

		for (TTuple<FMassEntityTemplateID, FMassAgentInitializationQueue>& Data : PendingAgentEntities)
		{
			const FMassEntityTemplateID EntityTemplateID = Data.Get<0>();
			const TSharedRef<FMassEntityTemplate>* EntityTemplatePtr = SpawnerSystem->GetTemplateRegistryInstance().FindTemplateFromTemplateID(EntityTemplateID);
			check(EntityTemplatePtr);
			const FMassEntityTemplate& EntityTemplate = EntityTemplatePtr->Get();
		
			TArray<TObjectPtr<UMassAgentComponent>>& AgentComponents = Data.Get<1>().AgentComponents;
			const int32 NewEntityCount = AgentComponents.Num();
		
			if (NewEntityCount <= 0)
			{
				// this case is perfectly fine, all agents registered and unregistered before we processed the queue
				continue;
			}
		
			TArray<FMassEntityHandle> Entities;
			CreationContext = SpawnerSystem->SpawnEntities(EntityTemplate, NewEntityCount, Entities);
			check(Entities.Num() == NewEntityCount);

			if (EntityTemplate.GetObjectFragmentInitializers().Num())
			{
				const TConstArrayView<FMassEntityTemplateData::FObjectFragmentInitializerFunction> ObjectFragmentInitializers = EntityTemplate.GetObjectFragmentInitializers();

				for (int AgentIndex = 0; AgentIndex < Entities.Num(); ++AgentIndex)
				{
					FMassEntityView EntityView(EntityTemplate.GetArchetype(), Entities[AgentIndex]);
					FMassAgentSubsystemHelper::InitializeAgentComponentFragments(*AgentComponents[AgentIndex], EntityView, EMassTranslationDirection::ActorToMass, ObjectFragmentInitializers);
				}
			}

			for (int AgentIndex = 0; AgentIndex < Entities.Num(); ++AgentIndex)
			{		
				AgentComponents[AgentIndex]->SetEntityHandle(Entities[AgentIndex]);
			}
		}
	}

	PendingAgentEntities.Reset();

	for (TTuple<FMassEntityTemplateID, FMassAgentInitializationQueue>& Data : PendingPuppets)
	{
		const FMassEntityTemplateID EntityTemplateID = Data.Get<0>();
		const TSharedRef<FMassEntityTemplate>* EntityTemplatePtr = SpawnerSystem->GetTemplateRegistryInstance().FindTemplateFromTemplateID(EntityTemplateID);

		if (!ensure(EntityTemplatePtr))
		{
			// note that this condition is temporary, we'll be switched to a `check` once we set up characters
			continue;
		}
		const FMassEntityTemplate& EntityTemplate = EntityTemplatePtr->Get();

		const FMassArchetypeCompositionDescriptor TemplateDescriptor = EntityTemplate.GetCompositionDescriptor();

		TArray<TObjectPtr<UMassAgentComponent>>& AgentComponents = Data.Get<1>().AgentComponents;

		for (UMassAgentComponent* AgentComp : AgentComponents)
		{
			const FMassEntityHandle PuppetEntity = AgentComp->GetEntityHandle();
			if (!ensureMsgf(PuppetEntity.IsSet(), TEXT("Trying to initialize puppet's fragments while the pupped doesn't have a corresponding Entity identifier set. This should not happen.")))
			{
				continue;
			}

			FMassArchetypeCompositionDescriptor& PuppetDescriptor = AgentComp->GetMutablePuppetSpecificAddition();
			PuppetDescriptor = TemplateDescriptor;
			EntityManager->AddCompositionToEntity_GetDelta(PuppetEntity, PuppetDescriptor);
			
			if (EntityTemplate.GetObjectFragmentInitializers().Num())
			{
				const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(PuppetEntity);
				FMassEntityView EntityView(ArchetypeHandle, PuppetEntity);
				FMassAgentSubsystemHelper::InitializeAgentComponentFragments(*AgentComp, EntityView, EMassTranslationDirection::MassToActor, EntityTemplate.GetObjectFragmentInitializers());
			}

			AgentComp->PuppetInitializationDone();
		}
	}

	PendingPuppets.Reset();
}

void UMassAgentSubsystem::NotifyMassAgentComponentReplicated(UMassAgentComponent& AgentComp)
{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UWorld* World = GetWorld();
	if (World && ensureMsgf(World->IsNetMode(NM_Client), TEXT("%s: Expecting to only be called in network game on the client"), ANSI_TO_TCHAR(__FUNCTION__)))
	{
		check(ReplicationSubsystem);
		check(AgentComp.GetNetID().IsValid());
		check(!ReplicatedAgentComponents.Find(AgentComp.GetNetID()));
		ReplicatedAgentComponents.Add(AgentComp.GetNetID(), &AgentComp);
		const FMassEntityHandle Entity = ReplicationSubsystem->FindEntity(AgentComp.GetNetID());

		// If not found, the NotifyMassAgentAddedToReplication will link it later once replicated.
		if (Entity.IsSet())
		{
			AgentComp.SetReplicatedPuppetHandle(FMassEntityHandle(Entity));
			MakePuppet(AgentComp);
		}
		else
		{
			AgentComp.MakePuppetAReplicatedOrphan();
		}
	}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
}

void UMassAgentSubsystem::NotifyMassAgentComponentEntityAssociated(const UMassAgentComponent& AgentComp) const
{
	OnMassAgentComponentEntityAssociated.Broadcast(AgentComp);
}

void UMassAgentSubsystem::NotifyMassAgentComponentEntityDetaching(const UMassAgentComponent& AgentComp) const
{
	OnMassAgentComponentEntityDetaching.Broadcast(AgentComp);
}

void UMassAgentSubsystem::OnMassAgentAddedToReplication(FMassNetworkID NetID, FMassEntityHandle Entity)
{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UWorld* World = GetWorld();
	if (World && World->IsNetMode(NM_Client))
	{
		if (TObjectPtr<UMassAgentComponent>* AgentComp = ReplicatedAgentComponents.Find(NetID))
		{
			(*AgentComp)->SetReplicatedPuppetHandle(FMassEntityHandle(Entity));
			MakePuppet(**AgentComp);
		}
	}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
}

void UMassAgentSubsystem::OnMassAgentRemovedFromReplication(FMassNetworkID NetID, FMassEntityHandle Entity)
{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UWorld* World = GetWorld();
	if (World && World->IsNetMode(NM_Client))
	{
		if (TObjectPtr<UMassAgentComponent>* AgentComp = ReplicatedAgentComponents.Find(NetID))
		{
			UnregisterAgentComponent(**AgentComp);
			(*AgentComp)->ClearReplicatedPuppetHandle();
		}
	}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
}
