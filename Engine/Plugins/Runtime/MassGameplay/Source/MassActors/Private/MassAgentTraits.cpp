// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAgentTraits.h"
#include "MassCommonTypes.h"
#include "MassMovementFragments.h"
#include "MassEntityTemplate.h"
#include "MassEntityTemplateRegistry.h"
#include "Translators/MassCapsuleComponentTranslators.h"
#include "Translators/MassCharacterMovementTranslators.h"
#include "Translators/MassSceneComponentLocationTranslator.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAgentTraits)

namespace FMassAgentTraitsHelper 
{
	template<typename T>
	T* AsComponent(UObject& Owner)
	{
		T* Component = nullptr;
		if (AActor* AsActor = Cast<AActor>(&Owner))
		{
			Component = AsActor->FindComponentByClass<T>();
		}
		else
		{
			Component = Cast<T>(&Owner);
		}

		UE_CVLOG_UELOG(Component == nullptr, &Owner, LogMass, Error, TEXT("Trying to extract %s from %s failed")
			, *T::StaticClass()->GetName(), *Owner.GetName());

		return Component;
	}
}

//----------------------------------------------------------------------//
//  UMassAgentCapsuleCollisionSyncTrait
//----------------------------------------------------------------------//
void UMassAgentCapsuleCollisionSyncTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FCapsuleComponentWrapperFragment>();
	BuildContext.AddFragment<FAgentRadiusFragment>();
	if (bSyncTransform || BuildContext.IsInspectingData())
	{
		BuildContext.AddFragment<FTransformFragment>();
	}
	
	BuildContext.GetMutableObjectFragmentInitializers().Add([this](UObject& Owner, FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
		{
			if (UCapsuleComponent* CapsuleComponent = FMassAgentTraitsHelper::AsComponent<UCapsuleComponent>(Owner))
			{
				FCapsuleComponentWrapperFragment& CapsuleFragment = EntityView.GetFragmentData<FCapsuleComponentWrapperFragment>();
				CapsuleFragment.Component = CapsuleComponent;

				FAgentRadiusFragment& RadiusFragment = EntityView.GetFragmentData<FAgentRadiusFragment>();
				RadiusFragment.Radius = CapsuleComponent->GetScaledCapsuleRadius();

				if (bSyncTransform)
				{
					FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();
					TransformFragment.GetMutableTransform() = CapsuleComponent->GetComponentTransform();
				}
			}
		});

	if (bSyncTransform || BuildContext.IsInspectingData())
	{
		if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass) || BuildContext.IsInspectingData())
		{
			BuildContext.AddTranslator<UMassCapsuleTransformToMassTranslator>();
		}

		if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor) || BuildContext.IsInspectingData())
		{
			BuildContext.AddTranslator<UMassTransformToActorCapsuleTranslator>();
		}
	}
}

//----------------------------------------------------------------------//
//  UMassAgentMovementSyncTrait
//----------------------------------------------------------------------//
void UMassAgentMovementSyncTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FCharacterMovementComponentWrapperFragment>();
	BuildContext.AddFragment<FMassVelocityFragment>();
	
	BuildContext.GetMutableObjectFragmentInitializers().Add([=](UObject& Owner, FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
		{
			if (UCharacterMovementComponent* MovementComp = FMassAgentTraitsHelper::AsComponent<UCharacterMovementComponent>(Owner))
			{
				FCharacterMovementComponentWrapperFragment& ComponentFragment = EntityView.GetFragmentData<FCharacterMovementComponentWrapperFragment>();
				ComponentFragment.Component = MovementComp;

				FMassVelocityFragment& VelocityFragment = EntityView.GetFragmentData<FMassVelocityFragment>();

				// the entity is the authority
				if (CurrentDirection ==  EMassTranslationDirection::MassToActor)
				{
					MovementComp->bRunPhysicsWithNoController = true;
					MovementComp->SetMovementMode(EMovementMode::MOVE_Walking);
					MovementComp->Velocity = VelocityFragment.Value;
				}
				// actor is the authority
				else
				{
					VelocityFragment.Value = MovementComp->GetLastUpdateVelocity();
				}
			}
		});

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass) || BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassCharacterMovementToMassTranslator>();
	}

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor) || BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassCharacterMovementToActorTranslator>();
	}
}

//----------------------------------------------------------------------//
//  UMassAgentOrientationSyncTrait
//----------------------------------------------------------------------//
void UMassAgentOrientationSyncTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FCharacterMovementComponentWrapperFragment>();

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass) 
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassCharacterOrientationToMassTranslator>();
	}

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor)
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassCharacterOrientationToActorTranslator>();
	}
}

//----------------------------------------------------------------------//
//  UMassAgentFeetLocationSyncTrait
//----------------------------------------------------------------------//
void UMassAgentFeetLocationSyncTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassSceneComponentWrapperFragment>();
	BuildContext.AddFragment<FTransformFragment>();

	BuildContext.GetMutableObjectFragmentInitializers().Add([=](UObject& Owner, FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
		{
			AActor* AsActor = Cast<AActor>(&Owner);
			if (AsActor && AsActor->GetRootComponent())
			{
				USceneComponent* Component = AsActor->GetRootComponent();
				FMassSceneComponentWrapperFragment& ComponentFragment = EntityView.GetFragmentData<FMassSceneComponentWrapperFragment>();
				ComponentFragment.Component = Component;

				FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();

				REDIRECT_OBJECT_TO_VLOG(Component, &Owner);
				UE_VLOG_LOCATION(&Owner, LogMass, Log, Component->GetComponentLocation(), 30, FColor::Yellow, TEXT("Initial component location"));
				UE_VLOG_LOCATION(&Owner, LogMass, Log, TransformFragment.GetTransform().GetLocation(), 30, FColor::Red, TEXT("Initial entity location"));

				// the entity is the authority
				if (CurrentDirection == EMassTranslationDirection::MassToActor)
				{
					// Temporary disabling this as it is already done earlier in the MassRepresentation and we needed to do a sweep to find the floor
					//Component->SetWorldLocation(FeetLocation, /*bSweep*/true, nullptr, ETeleportType::TeleportPhysics);
				}
				// actor is the authority
				else
				{
					TransformFragment.GetMutableTransform().SetLocation(Component->GetComponentTransform().GetLocation() - FVector(0.f, 0.f, Component->Bounds.BoxExtent.Z));
				}
			}
		});

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass)
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassSceneComponentLocationToMassTranslator>();
	}

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor)
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassSceneComponentLocationToActorTranslator>();
	}
}

