// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translators/MassCapsuleComponentTranslators.h"
#include "MassCommonTypes.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "Components/CapsuleComponent.h"


//----------------------------------------------------------------------//
// UMassCapsuleTransformToMassTranslator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCapsuleComponentTranslators)
UMassCapsuleTransformToMassTranslator::UMassCapsuleTransformToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	RequiredTags.Add<FMassCapsuleTransformCopyToMassTag>();
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	// @todo won't be required anymore once component ticking is run through Mass, or Mass has a way to synchronize with that.
	// Same comment to all the component translators
	bRequiresGameThreadExecution = true;
}

void UMassCapsuleTransformToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCapsuleComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassCapsuleTransformToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FCapsuleComponentWrapperFragment> CapsuleComponentList = Context.GetFragmentView<FCapsuleComponentWrapperFragment>();
			const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
			for (int i = 0; i < CapsuleComponentList.Num(); ++i)
			{
				if (const UCapsuleComponent* CapsuleComp = CapsuleComponentList[i].Component.Get())
				{
					LocationList[i].GetMutableTransform() = CapsuleComp->GetComponentTransform();
				}
			}
		});
}

//----------------------------------------------------------------------//
// UMassTransformToActorCapsuleTranslator
//----------------------------------------------------------------------//
UMassTransformToActorCapsuleTranslator::UMassTransformToActorCapsuleTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	RequiredTags.Add<FMassCapsuleTransformCopyToActorTag>();
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	bRequiresGameThreadExecution = true;
}

void UMassTransformToActorCapsuleTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCapsuleComponentWrapperFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.RequireMutatingWorldAccess(); // due to mutating World by setting component transform
}

void UMassTransformToActorCapsuleTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TArrayView<FCapsuleComponentWrapperFragment> CapsuleComponentList = Context.GetMutableFragmentView<FCapsuleComponentWrapperFragment>();
			const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
			for (int i = 0; i < CapsuleComponentList.Num(); ++i)
			{
				if (UCapsuleComponent* CapsuleComp = CapsuleComponentList[i].Component.Get())
				{
					CapsuleComp->SetWorldTransform(LocationList[i].GetTransform(), /*bSweep=*/false);
				}
			}
		});
}
