// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translators/MassCharacterMovementTranslators.h"
#include "Logging/LogMacros.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityManager.h"
#include "MassCommonTypes.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassSpawnerTypes.h"

//----------------------------------------------------------------------//
//  UMassCharacterMovementToMassTranslator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCharacterMovementTranslators)
UMassCharacterMovementToMassTranslator::UMassCharacterMovementToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassCharacterMovementCopyToMassTag>();
	bRequiresGameThreadExecution = true;
}

void UMassCharacterMovementToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCharacterMovementComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassCharacterMovementToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FCharacterMovementComponentWrapperFragment> ComponentList = Context.GetFragmentView<FCharacterMovementComponentWrapperFragment>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const UCharacterMovementComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				LocationList[EntityIt].GetMutableTransform().SetLocation(AsMovementComponent->GetActorNavLocation());
				
				VelocityList[EntityIt].Value = AsMovementComponent->Velocity;
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassCharacterMovementToActorTranslator
//----------------------------------------------------------------------//
UMassCharacterMovementToActorTranslator::UMassCharacterMovementToActorTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	RequiredTags.Add<FMassCharacterMovementCopyToActorTag>();
	bRequiresGameThreadExecution = true;
}

void UMassCharacterMovementToActorTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCharacterMovementComponentWrapperFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassCharacterMovementToActorTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TArrayView<FCharacterMovementComponentWrapperFragment> ComponentList = Context.GetMutableFragmentView<FCharacterMovementComponentWrapperFragment>();
		const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (UCharacterMovementComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				AsMovementComponent->RequestDirectMove(VelocityList[EntityIt].Value, /*bForceMaxSpeed=*/false);
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassCharacterOrientationToMassTranslator
//----------------------------------------------------------------------//
UMassCharacterOrientationToMassTranslator::UMassCharacterOrientationToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassCharacterOrientationCopyToMassTag>();
	bRequiresGameThreadExecution = true;
}

void UMassCharacterOrientationToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCharacterMovementComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassCharacterOrientationToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FCharacterMovementComponentWrapperFragment> ComponentList = Context.GetFragmentView<FCharacterMovementComponentWrapperFragment>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const UCharacterMovementComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				if (AsMovementComponent->UpdatedComponent != nullptr)
				{
					LocationList[EntityIt].GetMutableTransform().SetRotation(AsMovementComponent->UpdatedComponent->GetComponentTransform().GetRotation());
				}
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassCharacterOrientationToActorTranslator
//----------------------------------------------------------------------//
UMassCharacterOrientationToActorTranslator::UMassCharacterOrientationToActorTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	RequiredTags.Add<FMassCharacterOrientationCopyToActorTag>();
	bRequiresGameThreadExecution = true;
}

void UMassCharacterOrientationToActorTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FCharacterMovementComponentWrapperFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.RequireMutatingWorldAccess(); // due to mutating World by setting component rotation
}

void UMassCharacterOrientationToActorTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TArrayView<FCharacterMovementComponentWrapperFragment> ComponentList = Context.GetMutableFragmentView<FCharacterMovementComponentWrapperFragment>();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (UCharacterMovementComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				if (AsMovementComponent->UpdatedComponent != nullptr)
				{
					const FTransformFragment& Transform = TransformList[EntityIt];
					AsMovementComponent->bOrientRotationToMovement = false;
					AsMovementComponent->UpdatedComponent->SetWorldRotation(Transform.GetTransform().GetRotation());
				}
			}
		}
	});
}
