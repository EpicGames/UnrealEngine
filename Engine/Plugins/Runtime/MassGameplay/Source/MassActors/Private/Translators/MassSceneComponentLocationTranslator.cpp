// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translators/MassSceneComponentLocationTranslator.h"
#include "MassCommonTypes.h"
#include "Components/SceneComponent.h"
#include "MassExecutionContext.h"

//----------------------------------------------------------------------//
//  UMassSceneComponentLocationToMassTranslator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSceneComponentLocationTranslator)
UMassSceneComponentLocationToMassTranslator::UMassSceneComponentLocationToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassSceneComponentLocationCopyToMassTag>();
	bRequiresGameThreadExecution = true;
}

void UMassSceneComponentLocationToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FMassSceneComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassSceneComponentLocationToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FMassSceneComponentWrapperFragment> ComponentList = Context.GetFragmentView<FMassSceneComponentWrapperFragment>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const USceneComponent* AsComponent = ComponentList[EntityIt].Component.Get())
			{
				LocationList[EntityIt].GetMutableTransform().SetLocation(AsComponent->GetComponentTransform().GetLocation() - FVector(0.f, 0.f, AsComponent->Bounds.BoxExtent.Z));
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassSceneComponentLocationToActorTranslator
//----------------------------------------------------------------------//
UMassSceneComponentLocationToActorTranslator::UMassSceneComponentLocationToActorTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	RequiredTags.Add<FMassSceneComponentLocationCopyToActorTag>();
	bRequiresGameThreadExecution = true;
}

void UMassSceneComponentLocationToActorTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FMassSceneComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.RequireMutatingWorldAccess(); // due to mutating World by setting actor's location
}

void UMassSceneComponentLocationToActorTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassSceneComponentWrapperFragment> ComponentList = Context.GetFragmentView<FMassSceneComponentWrapperFragment>();
			const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (USceneComponent* AsComponent = ComponentList[EntityIt].Component.Get())
				{
					AsComponent->SetWorldLocation(LocationList[EntityIt].GetTransform().GetLocation() + FVector(0.f, 0.f, AsComponent->Bounds.BoxExtent.Z));
				}
			}
		});
}
