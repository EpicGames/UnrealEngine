// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translators/MassSceneComponentVelocityTranslator.h"
#include "MassCommonTypes.h"
#include "Components/SceneComponent.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "Translators/MassSceneComponentLocationTranslator.h"

//----------------------------------------------------------------------//
//  UMassSceneComponentVelocityToMassTranslator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSceneComponentVelocityTranslator)
UMassSceneComponentVelocityToMassTranslator::UMassSceneComponentVelocityToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassSceneComponentVelocityCopyToMassTag>();
}

void UMassSceneComponentVelocityToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FMassSceneComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassSceneComponentVelocityToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FMassSceneComponentWrapperFragment> ComponentList = Context.GetFragmentView<FMassSceneComponentWrapperFragment>();
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const USceneComponent* AsComponent = ComponentList[EntityIt].Component.Get())
			{
				VelocityList[EntityIt].Value = AsComponent->GetComponentVelocity();
			}
		}
	});
}
