// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translators/MassTranslators_BehaviorTree.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "MassEntityManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassTranslators_BehaviorTree)

UMassTranslator_BehaviorTree::UMassTranslator_BehaviorTree()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = false;
}

void UMassTranslator_BehaviorTree::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FDataFragment_BehaviorTreeComponentWrapper>(EMassFragmentAccess::ReadWrite);
}
