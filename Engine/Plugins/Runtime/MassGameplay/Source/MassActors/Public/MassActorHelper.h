// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "MassEntityConcepts.h"

class UScriptStruct;
class AActor;

namespace UE::MassActor
{
	/** 
	 * Finds the entity associated with Actor and adds TagType to it,
	 * @return true if successful, false otherwise (see log for details)
	 */
	bool MASSACTORS_API AddEntityTagToActor(const AActor& Actor, const UScriptStruct& TagType);

	/**
	 * Finds the entity associated with Actor and adds TagType to it,
	 * @return true if successful, false otherwise (see log for details)
	 */
	template<typename TagType>
	bool AddEntityTagToActor(const AActor& Actor)
	{
		static_assert(UE::Mass::CTag<TagType>, "Given struct doesn't represent a valid tag type.");
		return AddEntityTagToActor(Actor, *TagType::StaticStruct());
	}

	/**
	 * Finds the entity associated with Actor and remove TagType from its composition
	 * @return true if successful, false otherwise (see log for details)
	 */
	bool MASSACTORS_API RemoveEntityTagFromActor(const AActor& Actor, const UScriptStruct& TagType);

	/**
	 * Finds the entity associated with Actor and remove TagType from its composition
	 * @return true if successful, false otherwise (see log for details)
	 */
	template<typename TagType>
	bool RemoveEntityTagFromActor(const AActor& Actor)
	{
		static_assert(UE::Mass::CTag<TagType>, "Given struct doesn't represent a valid tag type.");
		return RemoveEntityTagFromActor(Actor, *TagType::StaticStruct());
	}
} // namespace UE::MassActor
