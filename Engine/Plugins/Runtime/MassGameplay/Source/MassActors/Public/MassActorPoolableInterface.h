// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "MassActorPoolableInterface.generated.h"

#define UE_API MASSACTORS_API

UINTERFACE(MinimalAPI, Blueprintable)
class UMassActorPoolableInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IMassActorPoolableInterface : public IInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Mass|Actor Pooling")
	UE_API bool CanBePooled();

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Mass|Actor Pooling")
	UE_API void PrepareForPooling();

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category="Mass|Actor Pooling")
	UE_API void PrepareForGame();
};

#undef UE_API
