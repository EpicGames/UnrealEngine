// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "IndexedHandle.h"
#include "Delegates/Delegate.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectKey.h"
#include "MassSubsystemBase.h"
#include "Mass/EntityHandle.h"
#include "StructUtils/StructView.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassActorSpawnerSubsystem.generated.h"


class ULevel; 
struct FActorSpawnParameters;

// Handle for an actor spawning request
USTRUCT()
struct FMassActorSpawnRequestHandle : public FIndexedHandleBase
{
	GENERATED_BODY()

	FMassActorSpawnRequestHandle() = default;

	/** @note passing INDEX_NONE as index will make this handle Invalid */
	FMassActorSpawnRequestHandle(const int32 InIndex, const uint32 InSerialNumber) : FIndexedHandleBase(InIndex, InSerialNumber)
	{
	}
};

// Managing class of spawning requests handles
typedef FIndexedHandleManager<FMassActorSpawnRequestHandle, true/*bOptimizeHandleReuse*/> FMassEntityHandleManager_ActorSpawnRequest;


DECLARE_DELEGATE_TwoParams(FMassActorPreSpawnDelegate, const FMassActorSpawnRequestHandle& SpawnRequestHandle, FConstStructView SpawnRequest);
enum class EMassActorSpawnRequestAction : uint8
{
	Keep, // Will leave spawning request in the queue and it will be users job to call RemoveActorSpawnRequest
	Remove, // Will remove the spawning request from the queue once the callback ends
};
DECLARE_DELEGATE_RetVal_TwoParams(EMassActorSpawnRequestAction, FMassActorPostSpawnDelegate, const FMassActorSpawnRequestHandle& SpawnRequestHandle, FConstStructView SpawnRequest);

UENUM()
enum class ESpawnRequestStatus : uint8
{
	None, // Not in the queue to be spawn
	Pending, // Still in the queue to be spawn
	Processing, // in the process of spawning the actor
	Succeeded, // Successfully spawned the actor
	Failed, // Error while spawning the actor
	RetryPending, // Failed spawn request that are being retried (lower priority)
};

/**
 * Base class for all spawn request
 */
USTRUCT()
struct FMassActorSpawnRequest
{
	GENERATED_BODY()
public:
	/** The actual mass agent handle corresponding to the actor to spawn */
	FMassEntityHandle MassAgent;

	/** The template BP actor to spawn */
	UPROPERTY(Transient)
	TSubclassOf<AActor> Template;

	/** The location of where to spawn that actor */
	FTransform	Transform;

	/** Priority of this spawn request in comparison with the others, the lower the value is, the higher the priority is */
	float Priority = MAX_FLT;

	/** Delegate that will be called just before the spawning an actor, giving the chance to the processor to prepare it */
	FMassActorPreSpawnDelegate ActorPreSpawnDelegate;

	/** 
	 * Delegate that will be called once the spawning is done. Note that it will be called regardless of whether 
	 * the spawning was successful. 
	 */
	FMassActorPostSpawnDelegate ActorPostSpawnDelegate;

	/** The current status of the spawn request */
	ESpawnRequestStatus SpawnStatus = ESpawnRequestStatus::None;

	/** The pointer to the actor once it is spawned */
	UPROPERTY(Transient)
	TObjectPtr<AActor> SpawnedActor = nullptr;

	/** Internal request serial number (used to cycle through next spawning request) */
	uint32 SerialNumber = 0;

	/** Requested world time seconds */
	double RequestedTime = 0.;

	/** If set, will be used to name the spawned character */
	FGuid Guid;

	void Reset()
	{
		MassAgent = FMassEntityHandle();
		Template = nullptr;
		Priority = MAX_FLT;
		ActorPreSpawnDelegate.Unbind();
		ActorPostSpawnDelegate.Unbind();
		SpawnStatus = ESpawnRequestStatus::None;
		SpawnedActor = nullptr;
		SerialNumber = 0;
		RequestedTime = 0.0f;
	}

	bool IsFinished() const { return SpawnStatus == ESpawnRequestStatus::Failed || SpawnStatus == ESpawnRequestStatus::Succeeded; }
};

/**
 * A subsystem managing spawning of actors for all mass subsystems
 */
UCLASS(MinimalAPI, transient)
class UMassActorSpawnerSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

protected:
	// USubsystem BEGIN
	MASSACTORS_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSACTORS_API virtual void Deinitialize() override;
	// USubsystem END

public:
	static MASSACTORS_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Request an actor to spawn
	 * Note: If you do not provide a spawn delegate, the requester is responsible to remove the request by hand.
	 * It will be auto removed after the execution of the spawn delegate.
	 * @param InSpawnRequest the spawn request parameters, You can provide any type of UStruct as long at it derives from FMassActorSpawnRequest. This let you add more spawning information.
	 */
	template< typename T, typename = typename TEnableIf<TIsDerivedFrom<typename TRemoveReference<T>::Type, FMassActorSpawnRequest>::IsDerived, void>::Type >
	FMassActorSpawnRequestHandle RequestActorSpawn(const T& InSpawnRequest)
	{
		return RequestActorSpawnInternal(FConstStructView::Make(InSpawnRequest));
	}

	/**
	 * Process a valid spawn request indicated by given handle. Can be used to force instant-spawn of an actor provided 
	 * a valid handle is obtained by calling RequestActorSpawn first. 
	 * @return indicates the status of processed spawn request, with ESpawnRequestStatus::None indicating that "something 
	 *	went wrong" and spawning request has not been processed. 
	 */
	[[nodiscard]] MASSACTORS_API ESpawnRequestStatus ProcessSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle);

	/** 
	 * Similar to the other ProcessSpawnRequest flavor, but with SpawnRequestView and SpawnRequest already provided. 
	 */
	[[nodiscard]] MASSACTORS_API ESpawnRequestStatus ProcessSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle, FStructView SpawnRequestView, FMassActorSpawnRequest& SpawnRequest);

	/** Retries a failed spawn request
	 * @param SpawnRequestHandle the spawn request handle to retry
	 */
	MASSACTORS_API void RetryActorSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle);

	/**
	 * Removes a spawn request
	 * The only time a spawn request cannot be removed is when its state is processing
	 * Also spawn requests are auto removed if you provided a spawn delegate after it was being executed.
	 * @param SpawnRequestHandle [IN/OUT] the spawn request handle to remove
	 * @return true if successfully removed the request
	 */
	MASSACTORS_API bool RemoveActorSpawnRequest(FMassActorSpawnRequestHandle& SpawnRequestHandle);

	/**
	 * Returns the stored spawn request from the handle, useful to update the transform
	 * @param SpawnRequestHandle the spawn request handle to get the request from
	 * @return The spawn request cast in the provided template argument
	 */
	template<typename T>
	const T& GetSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle) const
	{
		check(SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle));
		check(SpawnRequests.IsValidIndex(SpawnRequestHandle.GetIndex()));
		return SpawnRequests[SpawnRequestHandle.GetIndex()].Get<T>();
	}

	/**
	 * Returns the stored spawn request from the handle, useful to update the transform
	 * @param SpawnRequestHandle the spawn request handle to get the request from
	 * @return The spawn request cast in the provided template argument
	 */
	template<typename T> 
	T& GetMutableSpawnRequest(const FMassActorSpawnRequestHandle SpawnRequestHandle)
	{
		check(SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle));
		check(SpawnRequests.IsValidIndex(SpawnRequestHandle.GetIndex()));
		return SpawnRequests[SpawnRequestHandle.GetIndex()].GetMutable<T>();
	}

	bool IsSpawnRequestHandleValid(const FMassActorSpawnRequestHandle SpawnRequestHandle) const
	{
		return SpawnRequestHandleManager.IsValidHandle(SpawnRequestHandle);
	}

	/**
	 * Checks whether the given handle has already been released.
	 * Note that the function will return `false` for invalid handles - the function is intended
	 * to check whether a given "valid" handle that the caller has access to has already been released.
	 */
	bool IsHandleReleased(const FMassActorSpawnRequestHandle SpawnRequestHandle) const
	{
		return IsSpawnRequestHandleValid(SpawnRequestHandle) == false
			&& SpawnRequestHandleManager.GetHandles().IsValidIndex(SpawnRequestHandle.GetIndex())
			&& SpawnRequestHandleManager.GetHandles()[SpawnRequestHandle.GetIndex()].IsValid() == false;
	}

	/**
	 * Destroy an actor 
	 * @param Actor to destroy
	 * @param bImmediate to do the destruction immediately, otherwise will be queued up for later
	 */
	MASSACTORS_API virtual void DestroyActor(AActor* Actor, bool bImmediate = false);
	
	MASSACTORS_API void EnableActorPooling();
	MASSACTORS_API void DisableActorPooling();
	MASSACTORS_API bool IsActorPoolingEnabled();

	MASSACTORS_API void ReleaseAllResources();

protected:
	/** 
	 * Provides consistent way of conditional destroying Actor within World. The actual destruction depends on Actor's state
	 * and whether it belongs to World
	 */
	static MASSACTORS_API void ConditionalDestroyActor(UWorld& World, AActor& ActorToDestroy);

	/** Called at the start of the PrePhysics mass processing phase and calls ProcessPendingSpawningRequest */ 
	MASSACTORS_API void OnPrePhysicsPhaseStarted(const float DeltaSeconds);
	
	/** Called at the end of the PrePhysics mass processing phase and calls ProcessPendingDestruction */ 
	MASSACTORS_API void OnPrePhysicsPhaseFinished(const float DeltaSeconds); 
	
	/** 
	 *  Retrieve what would be the next best spawning request to spawn, can be overridden to have different logic
	 *  Default implementation is the first valid request in the list, no interesting logic yet
	 *  @param InOutHandleIndex used to start the search in subsequent locations. Also the index ensures the same handle 
	 *    won't get returned twice in a row. InOutHandleIndex being INDEX_NONE indicates this is the first run, so all 
	 *    handles are to be considered. If it's a  valid index then we iterate all but one to not even consider the 
	 *    handle indicated by InOutHandleIndex.
	 *  @return the next best handle to spawn. 
	 */
	MASSACTORS_API virtual FMassActorSpawnRequestHandle GetNextRequestToSpawn(int32& InOutHandleIndex) const;

	struct FSpawnArgs
	{
		TNotNull<UWorld*>   World;
		FMassEntityManager& EntityManager;

		FMassActorSpawnRequestHandle RequestHandle;
		FStructView                  RequestView;
		FMassActorSpawnRequest&      Request;
	};

	[[nodiscard]] MASSACTORS_API ESpawnRequestStatus ProcessSpawnRequestImpl(const FSpawnArgs& Args);
	MASSACTORS_API virtual ESpawnRequestStatus SpawnOrRetrieveFromPool(const FSpawnArgs& Args);

	UE_DEPRECATED(5.8, "Use the overload that accepts FSpawnArgs. OutSpawnedActor is equivalent to Args.Request.SpawnedActor")
	virtual ESpawnRequestStatus SpawnOrRetrieveFromPool(FConstStructView SpawnRequestView, TObjectPtr<AActor>& OutSpawnedActor) final
	{
		return ESpawnRequestStatus::Failed;
	}

	/** Actual code that will spawn the actor, overridable by subclass if need to be. */
	MASSACTORS_API virtual ESpawnRequestStatus SpawnActor(const FSpawnArgs& Args, FActorSpawnParameters& InOutSpawnParameters) const;

	UE_DEPRECATED(5.8, "Use the overload that accepts FSpawnArgs. OutSpawnedActor is equivalent to Args.Request.SpawnedActor")
	virtual ESpawnRequestStatus SpawnActor(FConstStructView SpawnRequestView, TObjectPtr<AActor>& OutSpawnedActor, FActorSpawnParameters& InOutSpawnParameters) const final
	{
		return ESpawnRequestStatus::Failed;
	}

	MASSACTORS_API TObjectPtr<AActor> FindActorByName(const FName ActorName, ULevel* OverrideLevel) const;

	/** Go through the spawning request and spawn them until we reach the budget 
	 * @param MaxTimeSlicePerTick is the budget in seconds allowed to do spawning */
	MASSACTORS_API void ProcessPendingSpawningRequest(const double MaxTimeSlicePerTick);

	/** Go through the queued actors to destroy and destroy them until we reach the budget
	 * @param MaxTimeSlicePerTick is the budget in seconds allowed to do destruction */
	MASSACTORS_API void ProcessPendingDestruction(const double MaxTimeSlicePerTick);

	/** Try releasing this actor to pool if possible 
	 * @param Actor to release to the pool
	 * @return true if the actor was actually released to the pool */
	MASSACTORS_API virtual bool ReleaseActorToPool(AActor* Actor);

	/** Internal generic request actor spawn to make sure the request derives from FMassActorSpawnRequest 
	 *  @param SpawnRequest the spawn request parameters, We are allowing any type of spawn request, let's store it internally as a FInstancedStruct. This parameter is the FStructView over provide user struct */
	MASSACTORS_API FMassActorSpawnRequestHandle RequestActorSpawnInternal(const FConstStructView SpawnRequest);

protected:

	UPROPERTY()
	TArray<FInstancedStruct> SpawnRequests;

	UPROPERTY()
	TArray<TObjectPtr<AActor>> ActorsToDestroy;

	UPROPERTY()
	TArray<TObjectPtr<AActor>> DeactivatedActorsToDestroy;

	bool bActorPoolingEnabled = true;

	TMap<FObjectKey, TArray<TObjectPtr<AActor>>> PooledActors;

	FMassEntityHandleManager_ActorSpawnRequest SpawnRequestHandleManager;

	std::atomic<uint32> RequestSerialNumberCounter;

	mutable int32 NumActorSpawned = 0;
	mutable int32 NumActorPooled = 0;

	int32 StartingHandleIndex = INDEX_NONE;
};
