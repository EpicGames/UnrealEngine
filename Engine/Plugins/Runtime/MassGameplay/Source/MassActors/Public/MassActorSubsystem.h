// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "Engine/ActorInstanceHandle.h"
#include "MassCommonFragments.h"
#include "Misc/MTAccessDetector.h"
#include "UObject/ObjectKey.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassActorSubsystem.generated.h"

#define UE_API MASSACTORS_API

struct FMassEntityHandle;
class AActor;
struct FMassEntityManager;
class UMassActorSubsystem;

USTRUCT()
struct FMassGuidFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()

	bool operator==(const FMassGuidFragment& Other) const
	{
		return Guid == Other.Guid;
	}

	UPROPERTY()
	FGuid Guid;
};

/**
 * Fragment to store the instanced actor handle of a mass entity if it needs one.
 */
USTRUCT()
struct FMassActorInstanceFragment : public FMassFragment
{
	GENERATED_BODY();

	FMassActorInstanceFragment() = default;
	explicit FMassActorInstanceFragment(const FActorInstanceHandle& InHandle)
		: Handle(InHandle)
	{
	}

	UPROPERTY()
	FActorInstanceHandle Handle;
};

template<>
struct TMassFragmentTraits<FMassActorInstanceFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

namespace UE::Mass::Signals
{
	/** Signal use when the actor instance handle is set or cleared in the associated fragment. */
	const FName ActorInstanceHandleChanged = FName(TEXT("ActorInstanceHandleChanged"));
}

/**
 * Fragment to save the actor pointer of a mass entity if it exists
 */
USTRUCT()
struct FMassActorFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()

	enum class EActorAccess
	{
		OnlyWhenAlive, // Only return an actor pointer if the actor is alive. This is the default.
		IncludePendingKill, // Return an actor pointer even if the actor is marked for destruction.
		IncludeUnreachable // Return an actor pointer even if the actor is unreachable. This implies it's being destroyed.
	};

	/**
	 * Set the actor associated to a mass agent, will also keep the map back in MassActorSubsystem up to date.
	 * @param MassAgent to associated with the actor
	 * @param InActor the actor associated with the mass agent
	 * @param bInIsOwnedByMass tell whether the actors was spawned by mass(MassVisualization) or externally(ReplicatedActors)
	 */
	UE_API void SetAndUpdateHandleMap(const FMassEntityHandle MassAgent, AActor* InActor, const bool bInIsOwnedByMass);

	/** 
	 * Resets the actor pointed by this fragment, will also keep the map back in UMassActorSubsystem up to date 
	 * @param CachedActorSubsystem if provided will be used directly, otherwise an instance of UMassActorSubsystem will 
	 *	be deduced from Actor's world (at additional runtime cost)
	 */
	UE_API void ResetAndUpdateHandleMap(UMassActorSubsystem* CachedActorSubsystem = nullptr);

	/**
	 * Set the actor associated to a mass agent, will NOT keep map back in MassActorSubsystem up to date.
	 * The user needs to do the map update by hand.
	 * Useful in multithreaded environment, to queue the update of the map inside a deferred command
	 * @param MassAgent to associated with the actor
	 * @param InActor the actor associated with the mass agent
	 * @param bInIsOwnedByMass tell whether the actors was spawned by mass(MassVisualization) or externally(ReplicatedActors)
	 */
	UE_API void SetNoHandleMapUpdate(const FMassEntityHandle MassAgent, AActor* InActor, const bool bInIsOwnedByMass);

	/** Resets the actor pointed by this fragment, will NOT keep map back in MassActorSubsystem up to date.
	 * The user needs to do the map update by hand.
	 * Useful in multithreaded environment, to queue the update of the map inside a deferred command
	 */
	UE_API void ResetNoHandleMapUpdate();


	/** @return none const pointer to the actor	*/
	inline AActor* GetMutable() { return Actor.Get(); }
	UE_API AActor* GetMutable(EActorAccess Access);

	/** @return none const pointer to the actor	only if owned by mass */
	inline AActor* GetOwnedByMassMutable() { return bIsOwnedByMass ? Actor.Get() : nullptr; }

	/** @return none const pointer to the actor	only if owned by mass */
	inline const AActor* Get() const { return Actor.Get(); }
	UE_API const AActor* Get(EActorAccess Access) const;

	/** @return if the actor is owned by mass */
	inline bool IsOwnedByMass() const { return bIsOwnedByMass; }

	/** @return if the actor is a valid pointer */
	inline bool IsValid() const { return Actor.IsValid(); }

private:
	// made visible for debugging purposes. It will show up in Mass's gameplay debugger category when viewing fragment details
	UPROPERTY(VisibleAnywhere, Category="Mass", Transient)
	TWeakObjectPtr<AActor> Actor;

	/** Ownership of the actor */
	bool bIsOwnedByMass = false;
};

struct FMassActorManager : public TSharedFromThis<FMassActorManager>
{
public:
	UE_API explicit FMassActorManager(const TSharedPtr<FMassEntityManager>& EntityManager, UObject* InOwner = nullptr);

	/** Get mass handle from an actor */
	UE_API FMassEntityHandle GetEntityHandleFromActor(const TObjectKey<const AActor> Actor);

	/** Set the mass handle associated to an actor */
	UE_API void SetHandleForActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle);

	/** Remove entry associated to an actor */
	UE_API void RemoveHandleForActor(const TObjectKey<const AActor> Actor);

	/** Get an actor pointer from a mass handle */
	UE_API AActor* GetActorFromHandle(const FMassEntityHandle Handle, 
		FMassActorFragment::EActorAccess Access = FMassActorFragment::EActorAccess::OnlyWhenAlive) const;

	/** 
	 *  Removes the connection between Actor and the given entity. Does all the required book keeping 
	 *  (as opposed to straight up RemoveHandleForActor call). If the Handle doesn't match Actor no action is taken.
	 */
	UE_API void DisconnectActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle);

	/** @return The number of actors with their associated entity registered in the manager */
	int32 DebugGetRegisteredActorCount() const
	{
		UE_MT_SCOPED_READ_ACCESS(ActorHandleMapDetector);
		return ActorHandleMap.Num();
	}

protected:

	TMap<TObjectKey<const AActor>, FMassEntityHandle> ActorHandleMap;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(ActorHandleMapDetector);
	
	TSharedPtr<FMassEntityManager> EntityManager;

	/** Points at an UObject hosting this instance of the FMassActorManager. It's fine for this to be null. */
	TWeakObjectPtr<UObject> Owner;
};

/**
 * A subsystem managing communication between Actors and Mass
 */
UCLASS(MinimalAPI)
class UMassActorSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:
	/** Get mass handle from an actor */
	inline FMassEntityHandle GetEntityHandleFromActor(const TObjectKey<const AActor> Actor);

	/** Set the mass handle associated to an actor */
	inline void SetHandleForActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle);

	/** Remove entry associated to an actor */
	inline void RemoveHandleForActor(const TObjectKey<const AActor> Actor);

	/** Get an actor pointer from a mass handle */
	inline AActor* GetActorFromHandle(const FMassEntityHandle Handle,
		FMassActorFragment::EActorAccess Access = FMassActorFragment::EActorAccess::OnlyWhenAlive) const;

	/** 
	 *  Removes the connection between Actor and the given entity. Does all the required book keeping 
	 *  (as opposed to straight up RemoveHandleForActor call). If the Handle doesn't match Actor no action is taken.
	 */
	inline void DisconnectActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle);

	/** @return The number of actors with their associated entity registered in the subsystem */
	inline int32 DebugGetRegisteredActorCount() const;

protected:
	// USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// USubsystem END
	
	TSharedPtr<FMassActorManager> ActorManager;
};

template<>
struct TMassExternalSubsystemTraits<UMassActorSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};

//----------------------------------------------------------------------//
//  INLINES
//----------------------------------------------------------------------//
FMassEntityHandle UMassActorSubsystem::GetEntityHandleFromActor(const TObjectKey<const AActor> Actor)
{
	return ActorManager->GetEntityHandleFromActor(Actor);
}

void UMassActorSubsystem::SetHandleForActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle)
{
	ActorManager->SetHandleForActor(Actor, Handle);
}

void UMassActorSubsystem::RemoveHandleForActor(const TObjectKey<const AActor> Actor)
{
	ActorManager->RemoveHandleForActor(Actor);
}

AActor* UMassActorSubsystem::GetActorFromHandle(const FMassEntityHandle Handle,
	FMassActorFragment::EActorAccess Access) const
{
	return ActorManager->GetActorFromHandle(Handle, Access);
}

void UMassActorSubsystem::DisconnectActor(const TObjectKey<const AActor> Actor, const FMassEntityHandle Handle) 
{ 
	return ActorManager->DisconnectActor(Actor, Handle);
}

int32 UMassActorSubsystem::DebugGetRegisteredActorCount() const
{
	return ActorManager->DebugGetRegisteredActorCount();
}

#undef UE_API
