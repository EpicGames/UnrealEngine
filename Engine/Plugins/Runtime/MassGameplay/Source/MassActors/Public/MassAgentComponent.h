// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "MassCommonTypes.h"
#include "MassEntityTemplate.h"
#include "MassEntityConfigAsset.h"
#include "MassAgentComponent.generated.h"

#define UE_API MASSACTORS_API


UENUM()
enum class EAgentComponentState : uint8
{
	None, // Default starting state

	// States of actors needing mass entity creation
	EntityPendingCreation, // Actor waiting for entity creation
	EntityCreated, // Actor with a created entity

	// States are for Actor driven by Mass (puppet) needing fragments initialization
	PuppetPendingInitialization, // Puppet actor queued for fragments initialization
	PuppetInitialized, // Puppet actor with all initialized fragments
	PuppetPaused, // Puppet actor with all fragments removed 
	PuppetPendingReplication, // Replicated puppet actor waiting for NetID
	PuppetReplicatedOrphan, // Replicated puppet actor not associated to a MassAgent
};


/** 
 *  There are two primary use cases for this component:
 *  1. If placed on an AActor blueprint it lets the user specify additional fragments that will be created for 
 *     entities spawned based on this given blueprint. 
 *  2. If present on an actor in the world it makes it communicate with the MassSimulation which will create an 
 *     entity representing given actor. Use case 1) will also be applicable in this case. The component is unregistered by 
 *     default and requires manual enabling via a 'Enable' call.
 * 
 *  @todo use case 2) is currently sitting in a shelved CL of mine. Will be worked on next.
 */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = Mass, meta = (BlueprintSpawnableComponent), hidecategories = (Sockets, Collision))
class UMassAgentComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UE_API UMassAgentComponent();

protected:
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

#if WITH_EDITOR
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(const bool bDuplicateForPIE) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

public:
	// Support poolable mass agents going into stasis
	UE_API void RegisterWithAgentSubsystem();
	UE_API void UnregisterWithAgentSubsystem();
	UE_API bool IsReadyForPooling() const;

	/** @todo to enforce encapsulation we could make this protected and have a UMassAgentSubsystem friend. I'm not sure I like it though. */

	/** Methods handling the state for actors that needs mass entity creation*/
	FMassEntityHandle GetEntityHandle() const { return AgentHandle; }
	UE_API void SetEntityHandle(const FMassEntityHandle NewHandle);
	UE_API void ClearEntityHandle();
	UE_API void EntityCreationPending();
	UE_API void EntityCreationAborted();
	bool IsEntityPendingCreation() const { return (State == EAgentComponentState::EntityPendingCreation); }

	/** Methods handling the state of puppet actors that needs fragments initialization */
	UE_API void SetPuppetHandle(const FMassEntityHandle NewHandle);
	UE_API void PuppetInitializationPending();
	UE_API void PuppetInitializationDone();
	UE_API void PuppetInitializationAborted();
	/** Called when a puppet actor is destroyed before its NetID was replicated (e.g., Iris relevancy). */
	UE_API void PuppetReplicationAborted();
	UE_API void PuppetUnregistrationDone();
	bool IsPuppetPendingInitialization() const { return (State == EAgentComponentState::PuppetPendingInitialization); }
	bool IsPuppetReplicationPending() const { return (State == EAgentComponentState::PuppetPendingReplication); }
	bool IsPuppet() const { return State == EAgentComponentState::PuppetInitialized || State == EAgentComponentState::PuppetPendingInitialization || State == EAgentComponentState::PuppetPaused; }
	/**
	 * Re-adds/Removes all puppet fragments added on the mass agent
	 * This is only supported in Puppet flow
	 * @param bPause true to pause or false to unpause
	 */
	UE_API void PausePuppet(const bool bPause);
	/* @return boolean whether this component was paused via PausePuppet method */
	bool IsPuppetPaused() const { return State == EAgentComponentState::PuppetPaused; }

	/** Methods handling the state of a server replicated puppet */
	UE_API void PuppetReplicationPending();
	UE_API void SetReplicatedPuppetHandle(FMassEntityHandle NewHandle);
	UE_API void ClearReplicatedPuppetHandle();
	UE_API void MakePuppetAReplicatedOrphan();

	FMassEntityTemplateID GetTemplateID() const { return TemplateID; }

	const FMassEntityConfig& GetEntityConfig() const { return EntityConfig; }
	UE_API void SetEntityConfig(const FMassEntityConfig& InEntityConfig);

	const FMassArchetypeCompositionDescriptor& GetPuppetSpecificAddition() const { return PuppetSpecificAddition; }
	FMassArchetypeCompositionDescriptor& GetMutablePuppetSpecificAddition() { return PuppetSpecificAddition; }

	/** Registers the component with the owner effectively turning it on. Calling it multiple times won't break anything  */
	UFUNCTION(BlueprintCallable, Category = Mass)
	UE_API void Enable();

	/** Registers the component with the owner effectively turning it off */
	UFUNCTION(BlueprintCallable, Category = Mass)
	UE_API void Disable();

	UFUNCTION(BlueprintCallable, Category = Mass)
	UE_API void KillEntity(const bool bDestroyActor);

	//~ Begin UObject Interface
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UObject Interface

	FMassNetworkID GetNetID() const
	{
		return NetID;
	}

	UFUNCTION()
	UE_API virtual void OnRep_NetID();

protected:

	UE_API void SwitchToState(EAgentComponentState NewState);
	UE_API virtual void SetEntityHandleInternal(const FMassEntityHandle NewHandle);
	UE_API void ClearEntityHandleInternal();
	UE_API void DebugCheckStateConsistency();

	/**
	 *  Contains all the fragments added to the entity during puppet's initialization. Required for clean up when
	 *  despawning puppet while the entity remains alive.
	 */
	FMassArchetypeCompositionDescriptor PuppetSpecificAddition;

	UPROPERTY(EditAnywhere, Category = "Mass")
	FMassEntityConfig EntityConfig;
	
	FMassEntityHandle AgentHandle;
	FMassEntityTemplateID TemplateID;

	EAgentComponentState State;

	UPROPERTY(replicatedUsing = OnRep_NetID)
	FMassNetworkID NetID;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Mass")
	uint32 bAutoRegisterInEditorMode : 1;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
