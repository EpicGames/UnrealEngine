// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassEntityTypes.h"
#include "MassProcessingTypes.h"
#include "MassEntityTemplate.h"
#include "MassSubsystemBase.h"
#include "MassAgentSubsystem.generated.h"

#define UE_API MASSACTORS_API

class AActor;
struct FMassEntityManager;
class UMassSpawnerSubsystem;
class UMassAgentComponent;
class UMassSimulationSubsystem;
class UMassAgentSubsystem;
class UMassAgentComponent;
class UMassReplicationSubsystem;

namespace UE::MassActor
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FMassAgentComponentDelegate, const UMassAgentComponent& /*AgentComponent*/);

} // UE::MassActor

USTRUCT()
struct FMassAgentInitializationQueue
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UMassAgentComponent>> AgentComponents;
};

/**
 * A subsystem managing communication between Actors and Mass
 */
UCLASS(MinimalAPI)
class UMassAgentSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

protected:
	// USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// USubsystem END

public:
	/** Registers given AgentComp with the Mass Simulation, including creation of a FMassEntityHandle using 
	 *  UMassAgentComponent.LWComponentList to determine the Archetype to use. */
	UE_API FMassEntityTemplateID RegisterAgentComponent(UMassAgentComponent& AgentComp);

	/** Called for AgentComp that has already been registered with the Agent Manager to notify it that AgentComp's 
	 *  Mass-relevant properties had changed, most notably its fragment composition. The Agent Manager will update 
	 *  the information on Mass side potentially reallocating the associated entity to a different archetype */
	UE_API void UpdateAgentComponent(const UMassAgentComponent& AgentComp);

	/**
	 * Removes given AgentComp instance from the system. If there's an entity created with the AgentComp
	 *  instance then it will be destroyed.
	 * @param AgentComp the component to unregister from the system
	 */
	 UE_API void UnregisterAgentComponent(UMassAgentComponent& AgentComp);

	/**
	 * Same as UnregisterAgentComponent, but on top of that it tells the system it will never register again
	 * @param AgentComp the component to shutdown from the system
	 */
	UE_API void ShutdownAgentComponent(UMassAgentComponent& AgentComp);

	/** lets the system know given agent is a puppet (an unreal-side representation of a mass entity) */
	UE_API void MakePuppet(UMassAgentComponent& AgentComp);

	/**
	 * Notifies that this MassAgentComponent is now replicated with a valid NetID
	 * @param AgentComp that is now replicated
	 */
	UE_API void NotifyMassAgentComponentReplicated(UMassAgentComponent& AgentComp);

	/**
	 * Notifies that this MassAgentComponent is now associated to a mass entity
	 * @param AgentComp that is now associated to a mass entity
	 */
	UE_API void NotifyMassAgentComponentEntityAssociated(const UMassAgentComponent& AgentComp) const;

	/**
	 * Notifies that this MassAgentComponent is now detaching from its mass entity
	 * @param AgentComp that is detaching from its mass entity
	 */
	UE_API void NotifyMassAgentComponentEntityDetaching(const UMassAgentComponent& AgentComp) const;

	/**
	 * @return The delegate of when MassAgentComponent gets associated to a mass entity
	 */
	UE::MassActor::FMassAgentComponentDelegate& GetOnMassAgentComponentEntityAssociated()
	{
		return OnMassAgentComponentEntityAssociated;
	}

	/**
	 * @return The delegate of when MassAgentComponent is detaching from its mass entity
	 */
	UE::MassActor::FMassAgentComponentDelegate& GetOnMassAgentComponentEntityDetaching()
	{
		return OnMassAgentComponentEntityDetaching;
	}

protected:
	/** 
	 * Processes PendingAgentEntities to initialize fragments of recently created agent entities and PendingPuppets 
	 * to create and initialize puppet-specific fragments 
	 */
	UE_API void HandlePendingInitialization();

	/** Bound to UMassSimulationSubsystem.OnProcessingPhaseStartedDelegate and called before every processing phase start */
	UE_API void OnProcessingPhaseStarted(const float DeltaSeconds, const EMassProcessingPhase Phase);

	/** Callback registered to the replication manager when a mass agent is added to the replication (client only) */
	UE_API void OnMassAgentAddedToReplication(FMassNetworkID NetID, FMassEntityHandle Entity);

	/** Callback registered to the replication manager when a mass agent is removed from the replication (client only) */
	UE_API void OnMassAgentRemovedFromReplication(FMassNetworkID NetID, FMassEntityHandle Entity);

protected:
	TSharedPtr<FMassEntityManager> EntityManager;

	UPROPERTY()
	TObjectPtr<UMassSpawnerSubsystem> SpawnerSystem;

	UPROPERTY()
	TObjectPtr<UMassSimulationSubsystem> SimulationSystem;

	UPROPERTY()
	TMap<FMassEntityTemplateID, FMassAgentInitializationQueue> PendingAgentEntities;

	UPROPERTY()
	TMap<FMassEntityTemplateID, FMassAgentInitializationQueue> PendingPuppets;

	UPROPERTY()
	TObjectPtr<UMassReplicationSubsystem> ReplicationSubsystem;

	UPROPERTY()
	TMap<FMassNetworkID, TObjectPtr<UMassAgentComponent>> ReplicatedAgentComponents;

	UE::MassActor::FMassAgentComponentDelegate OnMassAgentComponentEntityAssociated;
	UE::MassActor::FMassAgentComponentDelegate OnMassAgentComponentEntityDetaching;
};

#undef UE_API
