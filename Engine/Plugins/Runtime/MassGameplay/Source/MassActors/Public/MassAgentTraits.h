// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTraitBase.h"
#include "MassTranslator.h"
#include "MassAgentTraits.generated.h"

class USceneComponent;

UCLASS(MinimalAPI, Abstract)
class UMassAgentSyncTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

public:
	EMassTranslationDirection GetSyncDirection() const { return SyncDirection; }
	void SetSyncDirection(const EMassTranslationDirection NewDirection) { SyncDirection = NewDirection; }

protected:
	UPROPERTY(EditAnywhere, Category = Mass)
	EMassTranslationDirection SyncDirection = EMassTranslationDirection::BothWays;
};

/** The trait initializes the entity with actor capsule component's radius. In addition, if bSyncTransform is true 
 *  the trait keeps actor capsule component's and entity's transforms in sync. */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Capsule Collision Sync"))
class UMassAgentCapsuleCollisionSyncTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

protected:
	MASSACTORS_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(EditAnywhere, Category = Mass)
	bool bSyncTransform = true;
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Movement Sync"))
class UMassAgentMovementSyncTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

protected:
	MASSACTORS_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Orientation Sync"))
class UMassAgentOrientationSyncTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

	protected:
	MASSACTORS_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Feet Location Sync"))
class UMassAgentFeetLocationSyncTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

protected:
	MASSACTORS_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};
