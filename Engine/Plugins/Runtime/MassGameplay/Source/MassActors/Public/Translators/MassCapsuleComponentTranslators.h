// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonFragments.h"
#include "MassTranslator.h"
#include "MassEntityQuery.h"
#include "MassCapsuleComponentTranslators.generated.h"

#define UE_API MASSACTORS_API


class UCapsuleComponent;
struct FAgentRadiusFragment;

USTRUCT()
struct FCapsuleComponentWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()
	TWeakObjectPtr<UCapsuleComponent> Component;
};

/**
 * @todo TBD
 * I'm a bit on a fence regarding having separate tags per copy direction. My concern is that we can end up with a very 
 * dispersed entity population spread across multiple archetypes storing a small number of entities each. An alternative
 * would be to have a property on the Wrapper fragment, but that doesn't sit well with me either, since that data would be 
 * essentially static, meaning it will (in most cases) never change for a given entity, and we could waste a lot of time 
 * iterating over fragments just to check that specific value.
 */
USTRUCT()
struct FMassCapsuleTransformCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassCapsuleTransformToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()
public:
	UE_API UMassCapsuleTransformToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;	

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassCapsuleTransformCopyToActorTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTransformToActorCapsuleTranslator : public UMassTranslator
{
	GENERATED_BODY()
public:
	UE_API UMassTransformToActorCapsuleTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
