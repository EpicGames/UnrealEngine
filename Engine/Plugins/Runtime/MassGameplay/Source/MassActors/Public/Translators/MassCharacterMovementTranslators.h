// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassCommonFragments.h"
#include "MassTranslator.h"
#include "MassObserverProcessor.h"
#include "MassCharacterMovementTranslators.generated.h"

#define UE_API MASSACTORS_API

class UCharacterMovementComponent;

USTRUCT()
struct FCharacterMovementComponentWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()
	TWeakObjectPtr<UCharacterMovementComponent> Component;
};

USTRUCT()
struct FMassCharacterMovementCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassCharacterMovementToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassCharacterMovementToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassCharacterMovementCopyToActorTag : public FMassTag
{
	GENERATED_BODY()
};


UCLASS(MinimalAPI)
class UMassCharacterMovementToActorTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassCharacterMovementToActorTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};


USTRUCT()
struct FMassCharacterOrientationCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassCharacterOrientationToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassCharacterOrientationToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassCharacterOrientationCopyToActorTag : public FMassTag
{
	GENERATED_BODY()
};


UCLASS(MinimalAPI)
class UMassCharacterOrientationToActorTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassCharacterOrientationToActorTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
