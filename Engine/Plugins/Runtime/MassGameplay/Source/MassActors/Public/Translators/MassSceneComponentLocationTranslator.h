// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassCommonFragments.h"
#include "MassTranslator.h"
#include "MassSceneComponentLocationTranslator.generated.h"

#define UE_API MASSACTORS_API


USTRUCT()
struct FMassSceneComponentWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()
	TWeakObjectPtr<USceneComponent> Component;
};

USTRUCT()
struct FMassSceneComponentLocationCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassSceneComponentLocationToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassSceneComponentLocationToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};


USTRUCT()
struct FMassSceneComponentLocationCopyToActorTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassSceneComponentLocationToActorTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassSceneComponentLocationToActorTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
