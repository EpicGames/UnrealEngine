// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassTranslator.h"
#include "MassSceneComponentVelocityTranslator.generated.h"

#define UE_API MASSACTORS_API

USTRUCT()
struct FMassSceneComponentVelocityCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassSceneComponentVelocityToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassSceneComponentVelocityToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
