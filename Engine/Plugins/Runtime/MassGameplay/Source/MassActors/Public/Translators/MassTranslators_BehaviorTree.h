// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassTranslator.h"
#include "MassCommonFragments.h"
#include "MassTranslators_BehaviorTree.generated.h"

#define UE_API MASSACTORS_API


class UBehaviorTreeComponent;

USTRUCT()
struct FDataFragment_BehaviorTreeComponentWrapper : public FObjectWrapperFragment
{
	GENERATED_BODY()
	TWeakObjectPtr<UBehaviorTreeComponent> Component;
};

UCLASS(MinimalAPI)
class UMassTranslator_BehaviorTree : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassTranslator_BehaviorTree();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override {}

	FMassEntityQuery EntityQuery;
};

#undef UE_API
