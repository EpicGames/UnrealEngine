// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassCharacterTrajectory : ModuleRules
	{
		public MassCharacterTrajectory(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassMovement",
					"PoseSearch",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MassSpawner",
				}
			);
		}
	}
}
