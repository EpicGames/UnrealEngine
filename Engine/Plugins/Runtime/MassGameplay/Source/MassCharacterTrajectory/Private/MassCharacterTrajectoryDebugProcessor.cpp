// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCharacterTrajectoryDebugProcessor.h"

#include "DrawDebugHelpers.h"
#include "MassDebugLogging.h"
#include "MassExecutionContext.h"
#include "MassCharacterTrajectoryTypes.h"
#include "MassCharacterTrajectoryFragments.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCharacterTrajectoryDebugProcessor)

DEFINE_LOG_CATEGORY_STATIC(LogTrajectorySamples, Log, All);

namespace UE::MassPose::Trajectory::CVars
{
#if ENABLE_POSE_TRAJECTORY_DEBUG_DRAW
static bool GDebugDrawSamples = false;
static bool GDebugDrawVelocity = false;
static FAutoConsoleVariableRef CVarDebugDrawTrajectoryVelocity(TEXT("Mass.CharacterTrajectory.Debug.Velocity"), GDebugDrawVelocity, TEXT("Draw trajectory velocities"));
static FAutoConsoleVariableRef CVarDebugDrawTrajectorySamples(TEXT("Mass.CharacterTrajectory.Debug.Samples"), GDebugDrawSamples, TEXT("Enable debug drawing for MassPoseTrajectory generation"));
#endif
}


UCharacterTrajectoryDebugProcessor::UCharacterTrajectoryDebugProcessor()
	: EntityQuery(*this)
{
	// Todo: how to compile out entire class? Not allowed to compile out a UCLASS
#if ENABLE_POSE_TRAJECTORY_DEBUG_DRAW
	ExecutionFlags = (int32)EProcessorExecutionFlags::All & ~((int32)EProcessorExecutionFlags::Server);
#else
	ExecutionFlags = (int32)EProcessorExecutionFlags::None;
#endif

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CharacterTrajectoryDebug;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::CharacterTrajectoryCollision);
	bRequiresGameThreadExecution = true;
}

void UCharacterTrajectoryDebugProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FCharacterTrajectoryFragment>(EMassFragmentAccess::ReadOnly);
#if ENABLE_POSE_TRAJECTORY_DEBUG_DRAW
	EntityQuery.AddRequirement<FMassDebugLogFragment>(EMassFragmentAccess::ReadOnly);
#endif
}

void UCharacterTrajectoryDebugProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
#if ENABLE_POSE_TRAJECTORY_DEBUG_DRAW
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassSteeringToTrajectoryProcessorDebug);

	if (UE::MassPose::Trajectory::CVars::GDebugDrawSamples == false
		&& UE::MassPose::Trajectory::CVars::GDebugDrawVelocity == false)
	{
		return;
	}

	const UWorld* World = Context.GetWorld();

	QUICK_SCOPE_CYCLE_COUNTER(UMassSteeringToTrajectoryProcessor);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const TConstArrayView<FCharacterTrajectoryFragment> TrajectoryList = Context.GetFragmentView<FCharacterTrajectoryFragment>();
			const TConstArrayView<FMassDebugLogFragment> DebugLogOwnerList = Context.GetFragmentView<FMassDebugLogFragment>();
			const float ZOffset = -80.0f;

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				const FCharacterTrajectoryFragment& TrajectoryFrag = TrajectoryList[EntityIndex];
				const FMassDebugLogFragment& DebugLogOwner = DebugLogOwnerList[EntityIndex];
				if (UE::MassPose::Trajectory::CVars::GDebugDrawSamples)
				{
					UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(TrajectoryFrag.Trajectory, DebugLogOwner.LogOwner.Get(),
						LogTrajectorySamples, ELogVerbosity::Display, 0.0f, 0.0f, UE::MassPose::Trajectory::CVars::GDebugDrawVelocity);
				}

			}
		});
#endif // ENABLE_POSE_TRAJECTORY_DEBUG_DRAW
}
