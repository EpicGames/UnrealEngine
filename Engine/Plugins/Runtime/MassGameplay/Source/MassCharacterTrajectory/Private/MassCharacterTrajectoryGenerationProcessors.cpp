// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCharacterTrajectoryGenerationProcessors.h"

#include "Math/SpringMath.h"
#include "MassCommonFragments.h"
#include "MassCommonTypes.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassMovementTypes.h"
#include "MassCharacterTrajectoryTypes.h"
#include "MassSpringMovementFragments.h"
#include "MassCharacterTrajectoryFragments.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCharacterTrajectoryGenerationProcessors)

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USpringMovementToCharacterTrajectoryProcessor::USpringMovementToCharacterTrajectoryProcessor()
	: CalculateTrajectoryEntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CharacterTrajectoryGeneration;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SpringMovement);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Movement);
}

void USpringMovementToCharacterTrajectoryProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	CalculateTrajectoryEntityQuery.AddRequirement<FCharacterTrajectoryFragment>(EMassFragmentAccess::ReadWrite);
	CalculateTrajectoryEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	CalculateTrajectoryEntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadOnly);
	CalculateTrajectoryEntityQuery.AddRequirement<FSpringMovementRuntime>(EMassFragmentAccess::ReadOnly);
	CalculateTrajectoryEntityQuery.AddConstSharedRequirement<FCharacterTrajectoryParameters>(EMassFragmentPresence::All);
	CalculateTrajectoryEntityQuery.AddConstSharedRequirement<FSpringMovementSettings>(EMassFragmentPresence::All);
}

void USpringMovementToCharacterTrajectoryProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USpringMovementToCharacterTrajectoryProcessor);

	const float DeltaTime = Context.GetDeltaTimeSeconds();

	QUICK_SCOPE_CYCLE_COUNTER(USpringMovementToCharacterTrajectoryProcessor);

	CalculateTrajectoryEntityQuery.ParallelForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const FCharacterTrajectoryParameters& TrajectoryParams = Context.GetConstSharedFragment<FCharacterTrajectoryParameters>();
			const FSpringMovementSettings& SpringSettings = Context.GetConstSharedFragment<FSpringMovementSettings>();

			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FCharacterTrajectoryFragment> TrajectoryList = Context.GetMutableFragmentView<FCharacterTrajectoryFragment>();
			const TConstArrayView<FMassDesiredMovementFragment> SteeringInputList = Context.GetFragmentView<FMassDesiredMovementFragment>();
			const TConstArrayView<FSpringMovementRuntime> SteeringRuntimeList = Context.GetFragmentView<FSpringMovementRuntime>();

			using PredictorAllocator = TInlineAllocator<32>;
			TArray<FVector, PredictorAllocator> PredictedPositions;
			TArray<FVector, PredictorAllocator> PredictedVelocities;
			TArray<FVector, PredictorAllocator> PredictedAccels;
			TArray<FQuat, PredictorAllocator> PredictedFacing;
			TArray<FVector, PredictorAllocator> PredictedAngularVelocities;

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(USpringMovementToCharacterTrajectoryProcessor_SingleEntity);

				const FMassDesiredMovementFragment& SteeringInput = SteeringInputList[EntityIndex];
				const FSpringMovementRuntime& SteeringRuntime = SteeringRuntimeList[EntityIndex];
				FCharacterTrajectoryFragment& Trajectory = TrajectoryList[EntityIndex];

				FTransform CurrentTransform = TransformList[EntityIndex].GetTransform();

				FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
				TrajectoryDataSampling.NumHistorySamples = TrajectoryParams.NumHistorySamples;
				TrajectoryDataSampling.SecondsPerHistorySample = -1.0f;
				TrajectoryDataSampling.NumPredictionSamples = TrajectoryParams.NumPredictionSamples;
				TrajectoryDataSampling.SecondsPerPredictionSample = TrajectoryParams.PredictionSamplingInterval;

				// Note the mass transform is synchronized to the primary actor world transform (or entity transform)
				// but trajectory will be recorded in the space of the mesh component

				const FTransform MeshRelativeTransform = TrajectoryParams.Offset * Trajectory.MeshRelativeTransform;
				// Handle mesh to world offset
				CurrentTransform = MeshRelativeTransform * CurrentTransform;

				const FVector CurrentPosition = CurrentTransform.GetLocation();
				FQuat CurrentRotation = CurrentTransform.GetRotation();

				UPoseSearchTrajectoryLibrary::InitTrajectorySamples(Trajectory.Trajectory, CurrentPosition, CurrentRotation, TrajectoryDataSampling,
					DeltaTime);

				// Shift history
				UPoseSearchTrajectoryLibrary::UpdateHistory_WorldSpace(Trajectory.Trajectory, TrajectoryDataSampling, DeltaTime);

				// Write the current position from spring state
				CurrentTransform = FTransform(SteeringRuntime.CurrentFacing, SteeringRuntime.CurrentPosition);
				CurrentTransform = MeshRelativeTransform * CurrentTransform;

				{
					FTransformTrajectorySample & Sample = Trajectory.Trajectory.Samples[TrajectoryParams.NumHistorySamples];
					Sample.TimeInSeconds = DeltaTime;
					Sample.Position = CurrentTransform.GetTranslation();
					Sample.Facing = CurrentTransform.GetRotation();
				}

				FQuat DesiredFacing = SteeringInput.DesiredFacing;

				// Transform current and desired facing into mesh space
				DesiredFacing = DesiredFacing * MeshRelativeTransform.GetRotation();

				// Predict spring
				PredictedPositions.SetNumUninitialized(TrajectoryParams.NumPredictionSamples, EAllowShrinking::No);
				PredictedVelocities.SetNumUninitialized(TrajectoryParams.NumPredictionSamples, EAllowShrinking::No);
				PredictedAccels.SetNumUninitialized(TrajectoryParams.NumPredictionSamples, EAllowShrinking::No);
				PredictedFacing.SetNumUninitialized(TrajectoryParams.NumPredictionSamples, EAllowShrinking::No);
				PredictedAngularVelocities.SetNumUninitialized(TrajectoryParams.NumPredictionSamples, EAllowShrinking::No);

				// Output is in world space, but we want the mesh transform rather than the entity transform, so apply MeshRelativeTransform
				SpringMath::SpringCharacterPredict(
					TArrayView<FVector>(PredictedPositions),
					TArrayView<FVector>(PredictedVelocities),
					TArrayView<FVector>(PredictedAccels),
					CurrentTransform.GetTranslation(),
					SteeringRuntime.CurrentVelocity,
					SteeringRuntime.CurrentAccel,
					SteeringInput.DesiredVelocity,
					SpringSettings.VelocitySmoothingTime,
					TrajectoryParams.PredictionSamplingInterval,
					SpringSettings.VelocityDeadzoneThreshold);

				SpringMath::CriticalSpringDamperQuatPredict(
					TArrayView<FQuat>(PredictedFacing),
					TArrayView<FVector>(PredictedAngularVelocities),
					TrajectoryParams.NumPredictionSamples,
					CurrentTransform.GetRotation(),
					SteeringRuntime.CurrentAngularVelocity,
					DesiredFacing,
					SpringSettings.FacingSmoothingTime,
					TrajectoryParams.PredictionSamplingInterval);

				for (int i = 0; i < TrajectoryParams.NumPredictionSamples; i++)
				{
					int32 SampleIndex = TrajectoryParams.NumHistorySamples + i + 1;
					float PredictTime = DeltaTime + ((i + 1) * TrajectoryParams.PredictionSamplingInterval);

					Trajectory.Trajectory.Samples[SampleIndex].TimeInSeconds = PredictTime;
					Trajectory.Trajectory.Samples[SampleIndex].Position = PredictedPositions[i];
					Trajectory.Trajectory.Samples[SampleIndex].Facing = PredictedFacing[i];
				}

				// Write out the steering parameters
				// Use the current frame as the target
				Trajectory.SteeringTarget = Trajectory.Trajectory.GetSampleAtTime(DeltaTime).Facing;
			}
		});
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// UMovementToCharacterTrajectoryProcessor
// Generates trajectory from standard Mass movement (constant velocity prediction, no smoothing)

UMovementToCharacterTrajectoryProcessor::UMovementToCharacterTrajectoryProcessor()
	: CalculateTrajectoryEntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CharacterTrajectoryGeneration;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::ApplyForces);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Movement);
}

void UMovementToCharacterTrajectoryProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	CalculateTrajectoryEntityQuery.AddRequirement<FCharacterTrajectoryFragment>(EMassFragmentAccess::ReadWrite);
	CalculateTrajectoryEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	CalculateTrajectoryEntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadOnly);
	CalculateTrajectoryEntityQuery.AddConstSharedRequirement<FCharacterTrajectoryParameters>(EMassFragmentPresence::All);

	// Exclude entities using spring movement - those are handled by USpringMovementToCharacterTrajectoryProcessor
	CalculateTrajectoryEntityQuery.AddRequirement<FSpringMovementRuntime>(EMassFragmentAccess::None, EMassFragmentPresence::None);
}

void UMovementToCharacterTrajectoryProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMovementToCharacterTrajectoryProcessor);

	const float DeltaTime = Context.GetDeltaTimeSeconds();

	CalculateTrajectoryEntityQuery.ParallelForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const FCharacterTrajectoryParameters& TrajectoryParams = Context.GetConstSharedFragment<FCharacterTrajectoryParameters>();

			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FCharacterTrajectoryFragment> TrajectoryList = Context.GetMutableFragmentView<FCharacterTrajectoryFragment>();
			const TConstArrayView<FMassDesiredMovementFragment> DesiredMovementList = Context.GetFragmentView<FMassDesiredMovementFragment>();

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				const FMassDesiredMovementFragment& DesiredMovement = DesiredMovementList[EntityIndex];
				FCharacterTrajectoryFragment& Trajectory = TrajectoryList[EntityIndex];

				FTransform CurrentTransform = TransformList[EntityIndex].GetTransform();

				FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
				TrajectoryDataSampling.NumHistorySamples = TrajectoryParams.NumHistorySamples;
				TrajectoryDataSampling.SecondsPerHistorySample = -1.0f;
				TrajectoryDataSampling.NumPredictionSamples = TrajectoryParams.NumPredictionSamples;
				TrajectoryDataSampling.SecondsPerPredictionSample = TrajectoryParams.PredictionSamplingInterval;

				const FTransform MeshRelativeTransform = TrajectoryParams.Offset * Trajectory.MeshRelativeTransform;
				CurrentTransform = MeshRelativeTransform * CurrentTransform;

				const FVector CurrentPosition = CurrentTransform.GetLocation();
				const FQuat CurrentRotation = CurrentTransform.GetRotation();

				UPoseSearchTrajectoryLibrary::InitTrajectorySamples(Trajectory.Trajectory, CurrentPosition, CurrentRotation, TrajectoryDataSampling, DeltaTime);

				// Shift history
				UPoseSearchTrajectoryLibrary::UpdateHistory_WorldSpace(Trajectory.Trajectory, TrajectoryDataSampling, DeltaTime);

				// Write the current sample using desired velocity directly
				// (Velocity.Value may be stale/zero if a custom movement processor like CharacterTrajectoryMovement
				// disables the default movement processors)
				const FQuat DesiredFacing = DesiredMovement.DesiredFacing * MeshRelativeTransform.GetRotation();
				{
					FTransformTrajectorySample& Sample = Trajectory.Trajectory.Samples[TrajectoryParams.NumHistorySamples];
					Sample.TimeInSeconds = DeltaTime;
					Sample.Position = CurrentPosition + DesiredMovement.DesiredVelocity * DeltaTime;
					Sample.Facing = DesiredFacing;
				}

				// Predict future samples using constant desired velocity extrapolation (no smoothing)
				const FVector BasePosition = CurrentPosition + DesiredMovement.DesiredVelocity * DeltaTime;

				for (int32 i = 0; i < TrajectoryParams.NumPredictionSamples; i++)
				{
					const int32 SampleIndex = TrajectoryParams.NumHistorySamples + i + 1;
					const float PredictTime = DeltaTime + ((i + 1) * TrajectoryParams.PredictionSamplingInterval);
					const float FutureTime = (i + 1) * TrajectoryParams.PredictionSamplingInterval;

					Trajectory.Trajectory.Samples[SampleIndex].TimeInSeconds = PredictTime;
					Trajectory.Trajectory.Samples[SampleIndex].Position = BasePosition + DesiredMovement.DesiredVelocity * FutureTime;
					Trajectory.Trajectory.Samples[SampleIndex].Facing = DesiredFacing;
				}

				Trajectory.SteeringTarget = Trajectory.Trajectory.GetSampleAtTime(DeltaTime).Facing;
			}
		});
}
