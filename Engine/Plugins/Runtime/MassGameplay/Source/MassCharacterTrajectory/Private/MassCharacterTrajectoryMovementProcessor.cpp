// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCharacterTrajectoryMovementProcessor.h"
#include "MassMovementFragments.h"
#include "MassExecutionContext.h"
#include "MassCommonTypes.h"
#include "MassCharacterTrajectoryTypes.h"
#include "Animation/TrajectoryTypes.h"
#include "MassSpringMovementFragments.h"
#include "MassCharacterTrajectoryFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCharacterTrajectoryMovementProcessor)

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UCharacterTrajectoryToMovementProcessor::UCharacterTrajectoryToMovementProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::CharacterTrajectoryGeneration);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::CharacterTrajectoryCollision);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::UpdateWorldFromMass);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CharacterTrajectoryMovement;
}

void UCharacterTrajectoryToMovementProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddConstSharedRequirement<FCharacterTrajectoryParameters>(EMassFragmentPresence::All);

	EntityQuery.AddTagRequirement<FCharacterTrajectoryMovementTag>(EMassFragmentPresence::All);

	// Inputs
	EntityQuery.AddRequirement<FCharacterTrajectoryFragment>(EMassFragmentAccess::ReadOnly);

	// Outputs
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
}

void UCharacterTrajectoryToMovementProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	const float DeltaTime = Context.GetDeltaTimeSeconds();

	EntityQuery.ParallelForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();

			const TConstArrayView<FCharacterTrajectoryFragment> TrajectoryList = Context.GetFragmentView<FCharacterTrajectoryFragment>();
			const FCharacterTrajectoryParameters& TrajectoryParams = Context.GetConstSharedFragment<FCharacterTrajectoryParameters>();

			const TArrayView<FTransformFragment> TransformList = Context.GetMutableFragmentView<FTransformFragment>();
			const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				FTransformFragment& TransformFragment = TransformList[EntityIndex];
				const FCharacterTrajectoryFragment& TrajectoryFragment = TrajectoryList[EntityIndex];
				FMassVelocityFragment& VelocityFragment = VelocityList[EntityIndex];

				if(TrajectoryFragment.Trajectory.Samples.Num() <= 1)
				{
					// Not enough samples to move
					continue;
				}

				FVector PrevPos = TransformFragment.GetTransform().GetLocation();

				// Overwrite position based on trajectory
				FTransformTrajectorySample Sample = TrajectoryFragment.Trajectory.GetSampleAtTime(DeltaTime);

				FTransform SampleTransform(Sample.Facing, Sample.Position);

				// Transform from mesh to actor space
				const FTransform MeshRelativeTransform = TrajectoryParams.Offset * TrajectoryFragment.MeshRelativeTransform;
				SampleTransform = MeshRelativeTransform.Inverse() * SampleTransform;

				// Apply results
				TransformFragment.GetMutableTransform().SetRotation(SampleTransform.GetRotation());
				TransformFragment.GetMutableTransform().SetLocation(SampleTransform.GetTranslation());

				// For mover sync to work, we need to sync our desired position according to Euler integration, NOT the actual current spring velocity
				FVector EffectiveVel = DeltaTime > 0.0f ? (SampleTransform.GetTranslation() - PrevPos) / DeltaTime : FVector::ZeroVector;
				VelocityFragment.Value = EffectiveVel;
			}
		});
}
