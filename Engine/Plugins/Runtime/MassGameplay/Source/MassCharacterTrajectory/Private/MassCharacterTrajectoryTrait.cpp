// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCharacterTrajectoryTrait.h"

#include "MassDebugLogging.h"
#include "Components/SkeletalMeshComponent.h"
#include "MassEntityTemplateRegistry.h"
#include "MassEntityView.h"
#include "MassMovementFragments.h"
#include "MassCharacterTrajectoryFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCharacterTrajectoryTrait)

struct FMassDebugLogFragment;

void UCharacterTrajectoryTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FCharacterTrajectoryFragment>();
#if WITH_MASSENTITY_DEBUG
	BuildContext.AddFragment<FMassDebugLogFragment>();
#endif

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);
	const FConstSharedStruct PoseTrajectoryParamsSharedFragment = EntityManager.GetOrCreateConstSharedFragment(PoseTrajectoryParameters);
	BuildContext.AddConstSharedFragment(PoseTrajectoryParamsSharedFragment);

	BuildContext.GetMutableObjectFragmentInitializers().Add([=](UObject& Owner, const FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
		{
			// actor is the authority - initialize MeshRelativeTransform from SkeletalMeshComponent
			if (CurrentDirection != EMassTranslationDirection::MassToActor)
			{
				const AActor* AsActor = Cast<AActor>(&Owner);
				USkeletalMeshComponent* MeshComponent = AsActor
					? AsActor->FindComponentByClass<USkeletalMeshComponent>()
					: Cast<USkeletalMeshComponent>(&Owner);

				if (MeshComponent != nullptr)
				{
					FCharacterTrajectoryFragment& TrajectoryFragment = EntityView.GetFragmentData<FCharacterTrajectoryFragment>();
					TrajectoryFragment.MeshRelativeTransform = MeshComponent->GetRelativeTransform();
				}
			}

#if WITH_MASSENTITY_DEBUG
			const AActor* AsActor = Cast<AActor>(&Owner);
			FMassDebugLogFragment& DebugLogFragment = EntityView.GetFragmentData<FMassDebugLogFragment>();
			DebugLogFragment.LogOwner = AsActor;
#endif // WITH_MASSENTITY_DEBUG
		});
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
///
void UCharacterTrajectoryMovementTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	Super::BuildTemplate(BuildContext, World);

	BuildContext.AddTag<FCharacterTrajectoryMovementTag>();
	BuildContext.AddTag<FMassCustomMovementTag>();

	BuildContext.RequireFragment<FCharacterTrajectoryFragment>();
}

bool UCharacterTrajectoryMovementTrait::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	if (!BuildContext.HasFragment<FCharacterTrajectoryFragment>())
	{
		UE_LOG(LogMass, Error, TEXT("%s: CharacterTrajectoryMovement requires a Trajectory Generation trait (FCharacterTrajectoryFragment missing)"), *GetName());
		return false;
	}

	return Super::ValidateTemplate(BuildContext, World, OutTraitRequirements);
}
