// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCharacterTrajectoryTypes.h"

namespace UE::Mass::ProcessorGroupNames
{
	const FName CharacterTrajectoryGeneration = TEXT("CharacterTrajectoryGeneration");
	const FName CharacterTrajectoryCollision = TEXT("CharacterTrajectoryCollision");
	const FName CharacterTrajectoryMovement = TEXT("CharacterTrajectoryMovement");
	const FName CharacterTrajectoryDebug = TEXT("CharacterTrajectoryDebug");
}
