// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EngineDefines.h"
#include "MassEntityQuery.h"
#include "MassProcessor.h"

#include "MassCharacterTrajectoryDebugProcessor.generated.h"

#define ENABLE_POSE_TRAJECTORY_DEBUG_DRAW (UE_ENABLE_DEBUG_DRAWING && ENABLE_ANIM_DEBUG)

/** Debug visualization processor for trajectory samples, velocities, and AABB bounds.
 *  Controlled by console variables Mass.CharacterTrajectory.Debug.* */
UCLASS(MinimalAPI)
class UCharacterTrajectoryDebugProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSCHARACTERTRAJECTORY_API UCharacterTrajectoryDebugProcessor();

protected:
	MASSCHARACTERTRAJECTORY_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCHARACTERTRAJECTORY_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};
