// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Animation/TrajectoryTypes.h"
#include "MassCharacterTrajectoryFragments.generated.h"

/**
 * Shared configuration for trajectory generation. Controls how many samples
 * are recorded and predicted, and at what interval.
 *
 * The total trajectory has NumHistorySamples + 1 + NumPredictionSamples samples:
 *   - NumHistorySamples past positions (recorded each frame)
 *   - 1 "current" sample at t=DeltaTime (the predicted end-of-frame position)
 *   - NumPredictionSamples future positions spaced by PredictionSamplingInterval
 *
 * Added to entities by UCharacterTrajectoryTrait. Used by both
 * USpringMovementToCharacterTrajectoryProcessor and UMovementToCharacterTrajectoryProcessor.
 */
USTRUCT()
struct FCharacterTrajectoryParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	// Number of past positions to record. Each frame the oldest sample is
	// discarded and the current position is pushed into history.
	UPROPERTY(EditAnywhere, Category=Generation)
	int32 NumHistorySamples = 30;

	// Time in seconds between each prediction sample.
	UPROPERTY(EditAnywhere, Category=Generation)
	float PredictionSamplingInterval = 0.1f;

	// Number of future prediction samples to generate beyond the current frame.
	UPROPERTY(EditAnywhere, Category=Generation)
	int32 NumPredictionSamples = 15;

	// Additional transform offset applied when computing the trajectory.
	// Useful in actor-less contexts where the mesh-to-world offset cannot
	// be deduced automatically from a SkeletalMeshComponent.
	UPROPERTY(EditAnywhere, Category=Generation)
	FTransform Offset = FTransform::Identity;
};


/**
 * Per-entity trajectory data. Follows PoseSearch sample layout convention:
 *
 *   Trajectory.Samples layout (total = NumHistorySamples + 1 + NumPredictionSamples):
 *     [0 .. NumHistory-2]  : older history samples (negative time values)
 *     [NumHistory-1]       : t=0, last frame's position (beginning of current frame)
 *     [NumHistory]         : t=DeltaTime, this frame's predicted position
 *     [NumHistory+1 .. end]: future prediction samples
 *
 * Trajectory generation always runs BEFORE movement processors, so it reads
 * the start-of-frame transform from TransformFragment consistently.
 */
USTRUCT()
struct FCharacterTrajectoryFragment : public FMassFragment
{
	GENERATED_BODY()

	FTransform MeshRelativeTransform = FTransform::Identity;
	FTransform MeshRootWorldTransform = FTransform::Identity;
	FQuat SteeringTarget = FQuat::Identity;
	FTransformTrajectory Trajectory;
};

template<>
struct TMassFragmentTraits<FCharacterTrajectoryFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/** Enables trajectory-driven movement via UCharacterTrajectoryToMovementProcessor.
 *  Added by UCharacterTrajectoryMovementTrait, which also sets FMassCustomMovementTag
 *  to disable default movement processors. */
USTRUCT()
struct FCharacterTrajectoryMovementTag : public FMassTag
{
	GENERATED_BODY();
};
