// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MassProcessor.h"

#include "MassCharacterTrajectoryGenerationProcessors.generated.h"

/**
 * This processor converts spring movement data to a generated trajectory.
 * Entities must have FSpringMovementRuntime for this processor to match.
 */
UCLASS(MinimalAPI)
class USpringMovementToCharacterTrajectoryProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSCHARACTERTRAJECTORY_API USpringMovementToCharacterTrajectoryProcessor();

protected:
	MASSCHARACTERTRAJECTORY_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCHARACTERTRAJECTORY_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery CalculateTrajectoryEntityQuery;
};

/**
 * This processor generates a trajectory from standard Mass movement (no spring smoothing).
 * Uses the current velocity and desired movement to predict future positions with constant
 * velocity extrapolation. Matches entities that do NOT have FSpringMovementRuntime.
 */
UCLASS(MinimalAPI)
class UMovementToCharacterTrajectoryProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSCHARACTERTRAJECTORY_API UMovementToCharacterTrajectoryProcessor();

protected:
	MASSCHARACTERTRAJECTORY_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCHARACTERTRAJECTORY_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery CalculateTrajectoryEntityQuery;
};
