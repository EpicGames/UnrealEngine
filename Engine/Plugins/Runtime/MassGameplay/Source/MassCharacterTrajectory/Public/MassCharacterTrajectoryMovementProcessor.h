// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassCharacterTrajectoryMovementProcessor.generated.h"

struct FMassExecutionContext;

/** Moves the character along the generated trajectory.
 *  Requires FCharacterTrajectoryMovementTag (added by UCharacterTrajectoryMovementTrait). */
UCLASS(MinimalAPI)
class UCharacterTrajectoryToMovementProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSCHARACTERTRAJECTORY_API UCharacterTrajectoryToMovementProcessor();

protected:
	MASSCHARACTERTRAJECTORY_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCHARACTERTRAJECTORY_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};
