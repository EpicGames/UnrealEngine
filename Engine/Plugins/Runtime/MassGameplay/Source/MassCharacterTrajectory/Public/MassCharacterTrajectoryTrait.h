// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MassCharacterTrajectoryFragments.h"
#include "Movement/MassMovementTrait.h"

#include "MassCharacterTrajectoryTrait.generated.h"


/**
 * Enables trajectory generation for this entity. Adds FCharacterTrajectoryFragment and configures
 * trajectory sampling parameters (history size, prediction samples, sampling interval).
 *
 * The trajectory is generated each frame by one of two processors depending on configuration:
 *   - USpringMovementToCharacterTrajectoryProcessor: if USpringMovementTrait is also present,
 *     uses the spring runtime state to predict a smoothed trajectory.
 *   - UMovementToCharacterTrajectoryProcessor: if no spring is present, uses current velocity
 *     and desired movement with constant-velocity extrapolation.
 *
 * This trait can be combined with:
 *   - USpringMovementTrait: spring-damped movement with trajectory prediction for animation.
 *   - UCharacterTrajectoryMovementTrait: overrides default movement to drive the entity along the
 *     generated trajectory instead.
 *   - Both: spring update feeds trajectory generation, trajectory drives movement.
 *
 * Also initializes MeshRelativeTransform from the entity's SkeletalMeshComponent if present.
 */
UCLASS(MinimalAPI, EditInlineNew, CollapseCategories, meta = (DisplayName = "Character Trajectory Generation"))
class UCharacterTrajectoryTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	MASSCHARACTERTRAJECTORY_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category = "Steering", EditAnywhere, meta = (EditInline))
	FCharacterTrajectoryParameters PoseTrajectoryParameters;
};

/**
 * Overrides default movement to drive the entity along the generated trajectory.
 * Extends UMassMovementTrait to provide FMassMovementParameters for AI systems.
 *
 * Adds FCharacterTrajectoryMovementTag (matched by UCharacterTrajectoryToMovementProcessor) and
 * FMassCustomMovementTag (disables UMassApplyMovementProcessor and USpringMovementApplyProcessor
 * so that trajectory movement is the sole movement authority).
 *
 * Requires UCharacterTrajectoryTrait — validated at template build time.
 *
 * Execution flows with this trait:
 *   - With USpringMovementTrait: DesiredMovement -> UpdateSpring -> GenerateTrajectory -> MoveAlongTrajectory
 *   - Without spring:           DesiredMovement -> GenerateTrajectory -> MoveAlongTrajectory
 */
UCLASS(MinimalAPI, EditInlineNew, CollapseCategories, meta = (DisplayName = "Character Trajectory Movement"))
class UCharacterTrajectoryMovementTrait : public UMassMovementTrait
{
	GENERATED_BODY()

protected:
	MASSCHARACTERTRAJECTORY_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	MASSCHARACTERTRAJECTORY_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const override;
};
