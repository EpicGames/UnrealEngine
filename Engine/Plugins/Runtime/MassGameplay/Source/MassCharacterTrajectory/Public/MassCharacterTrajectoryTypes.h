// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"

namespace UE::Mass::ProcessorGroupNames
{
	extern MASSCHARACTERTRAJECTORY_API const FName CharacterTrajectoryGeneration;
	extern MASSCHARACTERTRAJECTORY_API const FName CharacterTrajectoryCollision;
	extern MASSCHARACTERTRAJECTORY_API const FName CharacterTrajectoryMovement;
	extern MASSCHARACTERTRAJECTORY_API const FName CharacterTrajectoryDebug;
}
