// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IMassCommonModule.h"
#include "MassCommonTypes.h"


class FMassCommonModule : public IMassCommonModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassCommonModule, MassCommon)



void FMassCommonModule::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void FMassCommonModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}



