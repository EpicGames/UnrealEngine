// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCommonTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCommonTypes)

namespace UE::Mass::ProcessorGroupNames
{
	const FName UpdateWorldFromMass = TEXT("UpdateWorldFromMass");
	const FName SyncWorldToMass = TEXT("SyncWorldToMass");
	const FName Behavior = TEXT("Behavior");
	const FName Tasks = TEXT("Tasks");
	const FName Avoidance = TEXT("Avoidance");
	const FName ApplyForces = TEXT("ApplyForces");
	const FName Movement = TEXT("Movement");
}
