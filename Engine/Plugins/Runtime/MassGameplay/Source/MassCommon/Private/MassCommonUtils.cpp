// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCommonUtils.h"
#include "Misc/App.h"
#include "HAL/IConsoleManager.h"

namespace UE::Mass::Private
{
    int32 RandomSeedOverride = 7;
    FAutoConsoleVariableRef CVarRandomSeedOverride(
		TEXT("mass.RandomSeedOverride"), 
		RandomSeedOverride, 
		TEXT("If FApp::bUseFixedSeed is true (e.g., -FixedSeed or -Deterministic) this value is used as random seed for all Mass uses."));
}

namespace UE::Mass::Utils
{

	TArray<FMassEntityHandle> EntityQueueToArray(TQueue<FMassEntityHandle, EQueueMode::Mpsc>& EntitiesQueue, const int32 EntitiesCount)
	{
		check(EntitiesCount > 0);
		TArray<FMassEntityHandle> EntitiesArray;
		EntitiesArray.AddUninitialized(EntitiesCount);

		FMassEntityHandle TempEntity;
		uint32 CurrentIndex = 0;
		while (EntitiesQueue.Dequeue(TempEntity))
		{
			EntitiesArray[CurrentIndex++] = TempEntity;
		}
		ensure(CurrentIndex == EntitiesCount);

		return MoveTemp(EntitiesArray);
	}

#if !UE_BUILD_SHIPPING

	bool IsDeterministic()
	{
		return FApp::bUseFixedSeed;
	}

	int32 OverrideRandomSeedForTesting(int32 InSeed)
	{
		if (IsDeterministic())
		{
			return UE::Mass::Private::RandomSeedOverride;
		}
		return InSeed;
	}

	int32 GenerateRandomSeed()
	{
		if (IsDeterministic())
		{
			return UE::Mass::Private::RandomSeedOverride;
		}
		return FMath::Rand();
	}
#endif

} // namespace UE::Mass::Utils
