// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Mass/EntityFragments.h"

#include "MassCommonFragments.generated.h"

USTRUCT()
struct FAgentRadiusFragment : public FMassFragment
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category = "")
	float Radius = 40.f;
};

USTRUCT()
struct FAgentHeightFragment : public FMassFragment
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category = "")
	float Height = 180.f;
};

/** This is a common type for all the wrappers pointing at UObjects used to copy data from them or set data based on
 *	Mass simulation..
 */
USTRUCT()
struct FObjectWrapperFragment : public FMassFragment
{
	GENERATED_BODY()
};
