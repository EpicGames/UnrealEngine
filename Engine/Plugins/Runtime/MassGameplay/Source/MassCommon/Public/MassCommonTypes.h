// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "StructUtils/InstancedStruct.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassProcessingTypes.h"
#include "SequentialID.h"
#include "MassCommonTypes.generated.h"

#define WITH_MASSGAMEPLAY_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && WITH_MASSENTITY_DEBUG && 1)

class UStaticMesh;
class UMaterialInterface;

namespace UE::Mass::ProcessorGroupNames
{
	extern MASSCOMMON_API const FName UpdateWorldFromMass;
	extern MASSCOMMON_API const FName SyncWorldToMass;
	extern MASSCOMMON_API const FName Behavior;
	extern MASSCOMMON_API const FName Tasks;
	extern MASSCOMMON_API const FName Avoidance;
	extern MASSCOMMON_API const FName ApplyForces;
	extern MASSCOMMON_API const FName Movement;
}

USTRUCT()
struct FMassNetworkID : public FSequentialIDBase
{
	GENERATED_BODY()

	FMassNetworkID() = default;
	explicit FMassNetworkID(uint32 InID) : FSequentialIDBase(InID) {}
};

/** Float encoded in int16, 1cm accuracy. */
USTRUCT()
struct FMassInt16Real
{
	GENERATED_BODY()

	FMassInt16Real() = default;
	
	explicit FMassInt16Real(const float InValue)
	{
		Set(InValue);
	}
	
	void Set(const float InValue)
	{
		Value = (int16)FMath::Clamp(FMath::RoundToInt32(InValue), -(int32)MAX_int16, (int32)MAX_int16);
	}

	float Get() const
	{
		return (float)Value;
	}

	bool operator<(const FMassInt16Real RHS) const { return Value < RHS.Value; }
	bool operator<=(const FMassInt16Real RHS) const { return Value <= RHS.Value; }
	bool operator>(const FMassInt16Real RHS) const { return Value > RHS.Value; }
	bool operator>=(const FMassInt16Real RHS) const { return Value >= RHS.Value; }
	bool operator==(const FMassInt16Real RHS) const { return Value == RHS.Value; }
	bool operator!=(const FMassInt16Real RHS) const { return Value != RHS.Value; }
	
protected:
	UPROPERTY(Transient)
	int16 Value = 0;
};

/** Float encoded in int16, 10cm accuracy. */
USTRUCT()
struct FMassInt16Real10
{
	GENERATED_BODY()

	static constexpr float Scale = 0.1f;
	
	FMassInt16Real10() = default;
	
	explicit FMassInt16Real10(const float InValue)
	{
		Set(InValue);
	}
	
	void Set(const float InValue)
	{
		Value = (int16)FMath::Clamp(FMath::RoundToInt32(InValue * Scale), -(int32)MAX_int16, (int32)MAX_int16);
	}

	float Get() const
	{
		return (float)Value / Scale;
	}

	bool operator<(const FMassInt16Real10 RHS) const { return Value < RHS.Value; }
	bool operator<=(const FMassInt16Real10 RHS) const { return Value <= RHS.Value; }
	bool operator>(const FMassInt16Real10 RHS) const { return Value > RHS.Value; }
	bool operator>=(const FMassInt16Real10 RHS) const { return Value >= RHS.Value; }
	bool operator==(const FMassInt16Real10 RHS) const { return Value == RHS.Value; }
	bool operator!=(const FMassInt16Real10 RHS) const { return Value != RHS.Value; }

protected:
	UPROPERTY(Transient)
	int16 Value = 0;
};

/** Vector which components are in range [-1..1], encoded in signed bytes. */
USTRUCT()
struct FMassSnorm8Vector
{
	GENERATED_BODY()

	static constexpr float Scale = (float)MAX_int8;   

	FMassSnorm8Vector() = default;
	
	explicit FMassSnorm8Vector(const FVector& InVector)
	{
		Set(InVector);
	}
	
	void Set(const FVector& InVector)
	{
		X = (int8)FMath::Clamp(FMath::RoundToInt32(InVector.X * Scale), -(int32)MAX_int8, (int32)MAX_int8);
		Y = (int8)FMath::Clamp(FMath::RoundToInt32(InVector.Y * Scale), -(int32)MAX_int8, (int32)MAX_int8);
		Z = (int8)FMath::Clamp(FMath::RoundToInt32(InVector.Z * Scale), -(int32)MAX_int8, (int32)MAX_int8);
	}

	FVector Get() const
	{
		return FVector(X / Scale, Y / Scale, Z / Scale);
	}

protected:
	UPROPERTY(Transient)
	int8 X = 0;

	UPROPERTY(Transient)
	int8 Y = 0;

	UPROPERTY(Transient)
	int8 Z = 0;
};

/** Vector2D which components are in range [-1..1], encoded in signed bytes. */
USTRUCT()
struct FMassSnorm8Vector2D
{
	GENERATED_BODY()

	static constexpr float Scale = (float)MAX_int8;

	FMassSnorm8Vector2D() = default;

	explicit FMassSnorm8Vector2D(const FVector2D& InVector)
	{
		Set(InVector);
	}
	
	explicit FMassSnorm8Vector2D(const FVector& InVector)
	{
		Set(FVector2D(InVector));
	}

	void Set(const FVector2D& InVector)
	{
		X = (int8)FMath::Clamp(FMath::RoundToInt32(InVector.X * Scale), -(int32)MAX_int8, (int32)MAX_int8);
		Y = (int8)FMath::Clamp(FMath::RoundToInt32(InVector.Y * Scale), -(int32)MAX_int8, (int32)MAX_int8);
	}

	FVector2D Get() const
	{
		return FVector2D(X / Scale, Y / Scale);
	}

	FVector GetVector(const float InZ = 0.0f) const
	{
		return FVector(X / Scale, Y / Scale, InZ);
	}
	
protected:
	UPROPERTY(Transient)
	int8 X = 0;

	UPROPERTY(Transient)
	int8 Y = 0;
};

/** Real in range [0..1], encoded in signed bytes. */
USTRUCT()
struct FMassUnorm8Real
{
	GENERATED_BODY()

	static constexpr float Scale = MAX_uint8;

	FMassUnorm8Real() = default;

	explicit FMassUnorm8Real(const float InValue)
	{
		Set(InValue);
	}

	void Set(const float InValue)
	{
		Value = (int8)FMath::Clamp(FMath::RoundToInt32(InValue * Scale), 0, (int32)MAX_uint8);
	}

	float Get() const
	{
		return (Value / Scale);
	}

protected:
	UPROPERTY(Transient)
	uint8 Value = 0;
};

/** Vector encoded in int16, 1cm accuracy. */
USTRUCT()
struct FMassInt16Vector
{
	GENERATED_BODY()

	FMassInt16Vector() = default;
	FMassInt16Vector(const FVector& InVector)
	{
		Set(InVector);
	}
	
	void Set(const FVector& InVector)
	{
		X = (int16)FMath::Clamp(FMath::RoundToInt32(InVector.X), -(int32)MAX_int16, (int32)MAX_int16);
		Y = (int16)FMath::Clamp(FMath::RoundToInt32(InVector.Y), -(int32)MAX_int16, (int32)MAX_int16);
		Z = (int16)FMath::Clamp(FMath::RoundToInt32(InVector.Z), -(int32)MAX_int16, (int32)MAX_int16);
	}

	FVector Get() const
	{
		return FVector(X, Y, Z);
	}

protected:
	UPROPERTY(Transient)
	int16 X = 0;

	UPROPERTY(Transient)
	int16 Y = 0;

	UPROPERTY(Transient)
	int16 Z = 0;
};

/** Vector2D encoded in int16, 1cm accuracy. */
USTRUCT()
struct FMassInt16Vector2D
{
	GENERATED_BODY()

	FMassInt16Vector2D() = default;
	FMassInt16Vector2D(const FVector2D& InVector)
	{
		Set(InVector);
	}
	FMassInt16Vector2D(const FVector& InVector)
	{
		Set(FVector2D(InVector));
	}
	
	void Set(const FVector2D& InVector)
	{
		X = (int16)FMath::Clamp(FMath::RoundToInt32(InVector.X), -(int32)MAX_int16, (int32)MAX_int16);
		Y = (int16)FMath::Clamp(FMath::RoundToInt32(InVector.Y), -(int32)MAX_int16, (int32)MAX_int16);
	}

	FVector2D Get() const
	{
		return FVector2D(X, Y);
	}

	FVector GetVector(const float InZ = 0.0f) const
	{
		return FVector(X, Y, InZ);
	}

protected:
	UPROPERTY(Transient)
	int16 X = 0;

	UPROPERTY(Transient)
	int16 Y = 0;
};
