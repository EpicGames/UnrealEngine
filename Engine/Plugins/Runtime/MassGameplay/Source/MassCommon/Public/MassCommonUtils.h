// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Mass/EntityHandle.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

namespace UE::Mass::Utils
{
	/** 
	 * Creates a TArray of entities based on the given EntitiesQueue. Note that it's the caller's responsibility to 
	 * ensure EntitiesCount > 0, otherwise the function will fail a check (with explosive results).
	 */
	MASSCOMMON_API TArray<FMassEntityHandle> EntityQueueToArray(TQueue<FMassEntityHandle, EQueueMode::Mpsc>& EntitiesQueue, const int32 EntitiesCount);

#if !UE_BUILD_SHIPPING

	/**
	 * If this is true, then the mass systems should strive to be as deterministic as possible, this will also enable the fixed random seed
	 * Currently maps to FApp::bUseFixedSeed
	 */
	MASSCOMMON_API bool IsDeterministic();

	/**
	 * If IsDeterministic() returns true, then this function will return the value of ai.massrepresentation.OverrideRandomSeed in place of InSeed
	 */
	MASSCOMMON_API int32 OverrideRandomSeedForTesting(int32 InSeed);

	/**
	 * If IsDeterministic() returns true, then this function will return the value of ai.massrepresentation.OverrideRandomSeed in place of FMath::Rand()
	 */
	MASSCOMMON_API int32 GenerateRandomSeed();

#else

	inline constexpr bool IsDeterministic() { return false; }

	inline int32 OverrideRandomSeedForTesting(int32 InSeed) { return InSeed; }

	inline int32 GenerateRandomSeed() { return FMath::Rand(); }
#endif

} // namespace UE::Mass::Utils