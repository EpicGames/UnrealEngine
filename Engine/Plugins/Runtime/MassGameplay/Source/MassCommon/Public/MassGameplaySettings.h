// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassGameplaySettings.generated.h"


UCLASS(MinimalAPI, config = Mass, defaultconfig, DisplayName = "Mass Gameplay")
class UMassGameplaySettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, config, Category = Debug, meta = (ConsoleVariable = "mass.debug.VLogSpawnLocations"))
	bool bLogSpawnLocations = true;
};
