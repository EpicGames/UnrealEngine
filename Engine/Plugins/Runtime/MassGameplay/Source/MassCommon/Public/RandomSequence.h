// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Helper functions to get a random number in a sequence based on Fibonacci hashing.
 * The random numbers are uniformly distributed.
 *
 * In contrast to FRandomStream, where each call to a random function will return new random number,
 * these functions will always return the same value for the same input.
 * That is UE::RandomSequence::FRand(42) is always 0.575462.
 *
 * This is useful for randomizing behaviors or appearances, where an entity index can be used to get a random value,
 * without having to store the actual value:
 *
 *   const float PreferredSpeed = UE::RandomSequence::FRandRange(EntityId.Index, MinSpeed, MaxSpeed);
 * 
 * If multiple values are needed, different parts of the sequence can be used by offsetting the index:
 *
 *   const int32 HatIndex = UE::RandomSequence::RandHelper(EntityId.Index + 31, 0, Hats.Num());
 *   const int32 GroomIndex = UE::RandomSequence::RandHelper(EntityId.Index + 1021, Grooms.Num());
 */

namespace UE::RandomSequence
{
	/**
	 * @return 32 bit fibonacci hash at specified index.
	 */
	inline uint32 FibonacciHash(const int32 SeqIndex)
	{
		constexpr uint32 K = 2654435769u; // 2^32 / phi (golden ratio)
		// Offset the sequence by 1, so that index 0 is not always 0.
		return (uint32)(SeqIndex + 1) * K;
	}
	
	/**
	 * Helper function to return random float.
	 * @return A random number in [0..1]
	 */
	inline float FRand(const int32 SeqIndex)
	{
		float Result;
		*(uint32*)&Result = 0x3F800000U | (FibonacciHash(SeqIndex) >> 9);
		return Result - 1.0f; 
	}

	/**
	 * Helper function to return random int on specified range.
	 * @return A random number in [0..A)
	 */
	inline int32 RandHelper(const int32 SeqIndex, const int32 A)
	{
		return (int32)(((int64)FibonacciHash(SeqIndex) * (int64)A) >> 32);
	}

	/** 
	 * Helper function to return random int in specified range.
	 * @return A random number >= Min and <= Max
	 */
	inline int32 RandRange(const int32 SeqIndex, const int32 InMin, const int32 InMax)
	{
		const int32 Range = (InMax - InMin) + 1;
		return InMin + RandHelper(SeqIndex, Range);
	}
	
	/** 
	 * Helper function to return random float in specified range.
	 * @return A random number >= Min and <= Max
	 */
	inline float FRandRange(const int32 SeqIndex, const float InMin, const float InMax)
	{
		return InMin + (InMax - InMin) * FRand(SeqIndex);
	}

	/** 
	 * Helper function to return random float in specified range.
	 * @return A random number >= Min and <= Max
	 */
	inline float RandRange(const int32 SeqIndex, const float InMin, const float InMax)
	{
		return FRandRange(SeqIndex, InMin, InMax);
	}

	UE_DEPRECATED(5.4, "Please use the correctly spelled version FibonacciHash()")
	inline uint32 FibocciHash(const int32 SeqIndex)
	{
		return FibonacciHash(SeqIndex);
	}
}; // UE::RandomSequence