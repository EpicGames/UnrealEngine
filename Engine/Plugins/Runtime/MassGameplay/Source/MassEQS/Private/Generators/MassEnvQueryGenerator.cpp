// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/MassEnvQueryGenerator.h"
#include "MassEQSSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryGenerator)

UMassEnvQueryGenerator::UMassEnvQueryGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRunAsync = true;
}

void UMassEnvQueryGenerator::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	MassEQSRequestHandler.SendOrRecieveRequest(QueryInstance, *this);
}
