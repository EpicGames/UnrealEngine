// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/MassEnvQueryGenerator_MassEntityHandles.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "Items/EnvQueryItemType_MassEntityHandle.h"
#include "MassEQSSubsystem.h"
#include "MassEQSUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryGenerator_MassEntityHandles)


UMassEnvQueryGenerator_MassEntityHandles::UMassEnvQueryGenerator_MassEntityHandles(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ItemType = UEnvQueryItemType_MassEntityHandle::StaticClass();

	SearchRadius.DefaultValue = 500.0f;
	SearchCenter = UEnvQueryContext_Querier::StaticClass();
}

bool UMassEnvQueryGenerator_MassEntityHandles::TryAcquireResults(FEnvQueryInstance& QueryInstance) const
{
	check(MassEQSRequestHandler.MassEQSSubsystem);

	TUniquePtr<FMassEQSRequestData> RawRequestData = MassEQSRequestHandler.MassEQSSubsystem->TryAcquireResults(MassEQSRequestHandler.RequestHandle);
	if (FMassEnvQueryResultData_MassEntityHandles* RequestData = FMassEQSUtils::TryAndEnsureCast<FMassEnvQueryResultData_MassEntityHandles>(RawRequestData))
	{
		QueryInstance.AddItemData<UEnvQueryItemType_MassEntityHandle>(RequestData->GeneratedEntityInfo);
		return true;
	}

	return false;
}

TUniquePtr<FMassEQSRequestData> UMassEnvQueryGenerator_MassEntityHandles::GetRequestData(FEnvQueryInstance& QueryInstance) const
{
	TArray<FVector> PreparedContextPositions = {};
	QueryInstance.PrepareContext(SearchCenter, PreparedContextPositions);

	return MakeUnique<FMassEQSRequestData_MassEntityHandles>(MoveTemp(PreparedContextPositions), SearchRadius.GetValue());
}
