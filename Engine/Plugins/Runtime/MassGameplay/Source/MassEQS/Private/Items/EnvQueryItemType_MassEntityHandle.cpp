// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/EnvQueryItemType_MassEntityHandle.h"
#include "EnvironmentQuery/EnvQueryTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_MassEntityHandle)


UEnvQueryItemType_MassEntityHandle::UEnvQueryItemType_MassEntityHandle()
{
	ValueSize = sizeof(FMassEnvQueryEntityInfo);
}

const FMassEnvQueryEntityInfo& UEnvQueryItemType_MassEntityHandle::GetValue(const uint8* RawData)
{
	return GetValueFromMemory<FMassEnvQueryEntityInfo>(RawData);
}

void UEnvQueryItemType_MassEntityHandle::SetValue(uint8* RawData, const FMassEnvQueryEntityInfo& Value)
{
	return SetValueInMemory<FMassEnvQueryEntityInfo>(RawData, Value);
}

FVector UEnvQueryItemType_MassEntityHandle::GetItemLocation(const uint8* RawData) const
{
	const FMassEnvQueryEntityInfo& EntityInfo = GetValue(RawData);	
	return EntityInfo.CachedTransform.GetLocation();
}

FRotator UEnvQueryItemType_MassEntityHandle::GetItemRotation(const uint8* RawData) const
{
	const FMassEnvQueryEntityInfo& EntityInfo = GetValue(RawData);
	return EntityInfo.CachedTransform.GetRotation().Rotator();
}
