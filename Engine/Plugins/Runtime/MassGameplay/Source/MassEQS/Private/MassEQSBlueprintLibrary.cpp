// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEQSBlueprintLibrary.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "MassCommonFragments.h"
#include "MassSignalSubsystem.h"
#include "MassEntitySubsystem.h"

#include "Items/EnvQueryItemType_MassEntityHandle.h"
#include "EnvironmentQuery/EnvQueryInstanceBlueprintWrapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEQSBlueprintLibrary)

UMassEQSBlueprintLibrary::UMassEQSBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: UBlueprintFunctionLibrary(ObjectInitializer)
{
}

//----------------------------------------------------------------------//
// Commands
//----------------------------------------------------------------------//

void UMassEQSBlueprintLibrary::SendSignalToEntity(const AActor* Owner, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo, const FName Signal)
{
	if (!ensureMsgf(Owner, TEXT("Must supply owner to function. Reference to self should suffice.")))
	{
		return;
	}

	const UWorld* World = Owner->GetWorld();
	check(World);

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	check(EntitySubsystem);

	UMassSignalSubsystem* SignalSubsystem = World->GetSubsystem<UMassSignalSubsystem>();
	check(SignalSubsystem);

	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();

	FMassEntityHandle EntityHandle = EntityInfo.GetEntityHandle();
	if (!EntityManager.IsEntityValid(EntityHandle))
	{
		return;
	}

	SignalSubsystem->SignalEntity(Signal, EntityHandle);
}

//----------------------------------------------------------------------//
// Utils
//----------------------------------------------------------------------//

FVector UMassEQSBlueprintLibrary::GetCurrentEntityPosition(const AActor* Owner, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo)
{
	if (!ensureMsgf(Owner, TEXT("Must supply owner to function. Reference to self should suffice.")))
	{
		return FVector::ZeroVector;
	}

	const UWorld* World = Owner->GetWorld();
	check(World);

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	check(EntitySubsystem);

	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();

	const FTransformFragment* TransformFragment = EntityManager.GetFragmentDataPtr<FTransformFragment>(EntityInfo.GetEntityHandle());
	if (TransformFragment)
	{
		return TransformFragment->GetTransform().GetLocation();
	}

	return FVector::ZeroVector;
}

bool UMassEQSBlueprintLibrary::ContainsEntity(const TArray<FMassEnvQueryEntityInfoBlueprintWrapper>& EntityList, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo)
{
	for (const FMassEnvQueryEntityInfoBlueprintWrapper& Entity : EntityList)
	{
		if (Entity == EntityInfo)
		{
			return true;
		}
	}

	return false;
}

TArray<FMassEnvQueryEntityInfoBlueprintWrapper> UMassEQSBlueprintLibrary::GetEnviromentQueryResultAsEntityInfo(const UEnvQueryInstanceBlueprintWrapper* QueryInstance)
{
	check(QueryInstance);

	TArray<FMassEnvQueryEntityInfoBlueprintWrapper> ResultInfo = {};

	if (const FEnvQueryResult* QueryResult = QueryInstance->GetQueryResult())
	{
		const TSubclassOf<UEnvQueryItemType> ItemType = QueryResult->ItemType;
		const EEnvQueryRunMode::Type RunMode = QueryInstance->GetRunMode();

		if ((QueryResult->GetRawStatus() == EEnvQueryStatus::Success) 
			&& ItemType && ItemType->IsChildOf(UEnvQueryItemType_MassEntityHandle::StaticClass()))
		{
			if (RunMode != EEnvQueryRunMode::AllMatching)
			{
				ResultInfo.Add(GetItemAsEntityInfoBPWrapper(QueryResult, 0));
			}
			else
			{
				GetAllAsEntityInfoBPWrappers(QueryResult, ResultInfo);
			}
		}
	}

	return ResultInfo;
}

FMassEnvQueryEntityInfoBlueprintWrapper UMassEQSBlueprintLibrary::GetItemAsEntityInfoBPWrapper(const FEnvQueryResult* QueryResult, int32 Index)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryResult->ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();

	check(DefTypeOb != nullptr);
	FMassEnvQueryEntityInfo EntityInfo = DefTypeOb->GetValue(QueryResult->RawData.GetData() + QueryResult->Items[Index].DataOffset);

	return FMassEnvQueryEntityInfoBlueprintWrapper(EntityInfo);
}

void UMassEQSBlueprintLibrary::GetAllAsEntityInfoBPWrappers(const FEnvQueryResult* QueryResult, TArray<FMassEnvQueryEntityInfoBlueprintWrapper>& OutInfo)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryResult->ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	OutInfo.Reserve(OutInfo.Num() + QueryResult->Items.Num());

	for (const FEnvQueryItem& Item : QueryResult->Items)
	{
		FMassEnvQueryEntityInfo EntityInfo = DefTypeOb->GetValue(QueryResult->RawData.GetData() + Item.DataOffset);
		OutInfo.Add(FMassEnvQueryEntityInfoBlueprintWrapper(EntityInfo));
	}
}
