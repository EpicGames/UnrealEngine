// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEQSSubsystem.h"

#include "MassCommandBuffer.h"
#include "MassEntitySubsystem.h"
#include "MassEQS.h"

#include "Items/EnvQueryItemType_MassEntityHandle.h"
#include "Generators/MassEnvQueryGenerator.h"
#include "Tests/MassEnvQueryTest.h"
#include "MassEQSUtils.h"

//----------------------------------------------------------------------//
// RequestQueue
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEQSSubsystem)

FMassEQSRequestHandle UMassEQSSubsystem::PushRequest(const FEnvQueryInstance& QueryInstance, const int32 RequestQueueIndex, TUniquePtr<FMassEQSRequestData>&& Request)
{
	// It's a "read" since the detector only cares about the RequestQueues array, not its elements
	UE_MT_SCOPED_READ_ACCESS(RequestAccessDetector);
	check(RequestQueueIndex >= 0);

	FMassEQSRequestHandle RequestHandle = HandleManager.GetNextHandle();
	Request.Get()->RequestHandle = RequestHandle;

	// Fill up the Requests EntityHandle List with all Entities previously generated. 
	// Won't do anything for Generators.
	FMassEQSUtils::GetAllAsEntityHandles(QueryInstance, Request->EntityHandles);

	RequestQueues[RequestQueueIndex].Enqueue(MoveTemp(Request));

	return RequestHandle;
}


TUniquePtr<FMassEQSRequestData> UMassEQSSubsystem::PopRequest(const int32 RequestQueueIndex)
{
	// It's a "read" since the detector only cares about the RequestQueues array, not its elements
	UE_MT_SCOPED_READ_ACCESS(RequestAccessDetector);
	check(RequestQueueIndex >= 0);

	TUniquePtr<FMassEQSRequestData> OutRequest = nullptr;
	RequestQueues[RequestQueueIndex].Dequeue(OutRequest);

	return OutRequest;
}

//----------------------------------------------------------------------//
// Results
//----------------------------------------------------------------------//

void UMassEQSSubsystem::SubmitResults(FMassEQSRequestHandle RequestHandle, TUniquePtr<FMassEQSRequestData>&& Result)
{
	UE_MT_SCOPED_WRITE_ACCESS(ResultAccessDetector);

	check(RequestHandle.IsValid());

	int32 CancelledRequestIndex = CancelledRequests.Find(RequestHandle);
	if (CancelledRequestIndex >= 0)
	{
		CancelledRequests.RemoveAt(CancelledRequestIndex);
		FinalizeRequest(RequestHandle);
		return;
	}
	
	Result->RequestHandle = RequestHandle;
	Results.Add(RequestHandle, MoveTemp(Result));
}

TUniquePtr<FMassEQSRequestData> UMassEQSSubsystem::TryAcquireResults(FMassEQSRequestHandle RequestHandle)
{
	UE_MT_SCOPED_WRITE_ACCESS(ResultAccessDetector);

	// Should never try to Acquire invalid RequestHandle
	check(RequestHandle.IsValid());

	TUniquePtr<FMassEQSRequestData> Result = nullptr;
	if (Results.RemoveAndCopyValue(RequestHandle, Result))
	{
		FinalizeRequest(RequestHandle);
	}
	
	return Result;
}

//----------------------------------------------------------------------//
// Misc
//----------------------------------------------------------------------//

void UMassEQSSubsystem::FinalizeRequest(FMassEQSRequestHandle RequestHandle)
{
	HandleManager.RemoveHandle(RequestHandle);
}

void UMassEQSSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	bIsInitialized = true;

	OverrideSubsystemTraits<UMassEQSSubsystem>(Collection);
}

void UMassEQSSubsystem::Deinitialize()
{
	UE_MT_SCOPED_WRITE_ACCESS(RequestAccessDetector);
	UE_MT_SCOPED_WRITE_ACCESS(ResultAccessDetector);

	Super::Deinitialize();

	Results.Empty();
	RequestQueues.Empty();
	RequestQueueLookup.Reset();

	bIsInitialized = false;
}

int32 UMassEQSSubsystem::GetRequestQueueIndex(TSubclassOf<UEnvQueryNode> RequestClass)
{
	if (int32* IndexPtr = RequestQueueLookup.Find(RequestClass))
	{
		return *IndexPtr;
	}

	UE_MT_SCOPED_WRITE_ACCESS(RequestAccessDetector);
	RequestQueues.Emplace();
	int32 NewIndex = RequestQueues.Num() - 1;
	
	return RequestQueueLookup.Add(RequestClass, NewIndex);
}

void UMassEQSSubsystem::CancelRequest(FMassEQSRequestHandle RequestHandle)
{
	// Not necessary after De-initialization
	if (!bIsInitialized)
	{
		return;
	}

	UE_MT_SCOPED_WRITE_ACCESS(ResultAccessDetector);

	int32 NumRemoved = Results.Remove(RequestHandle);

	// Log this query if its results are not yet present, we need to ignore results once they are submitted.
	if (NumRemoved == 0u)
	{
		CancelledRequests.Add(RequestHandle);
	}
}
