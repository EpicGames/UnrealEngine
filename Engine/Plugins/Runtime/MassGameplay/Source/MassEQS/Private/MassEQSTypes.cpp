// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEQSTypes.h"
#include "Engine/World.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "MassEQSSubsystem.h"
#include "MassEQS.h"


//-----------------------------------------------------------------------------
// FMassEnvQueryEntityInfo
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEQSTypes)
FMassEnvQueryEntityInfo::FMassEnvQueryEntityInfo(int32 Index, int32 SerialNumber, const FTransform& Transform)
	: EntityHandle(Index, SerialNumber)
	, CachedTransform(Transform)
{
}

//-----------------------------------------------------------------------------
// FMassEQSRequestHandle
//-----------------------------------------------------------------------------
FMassEQSRequestHandle FMassEQSRequestHandle::Invalid = FMassEQSRequestHandle(INDEX_NONE, 0);

FString FMassEQSRequestHandle::ToString() const
{
	return FString::Printf(TEXT("[%d,%u]"), Index, SerialNumber);
}

//-----------------------------------------------------------------------------
// IMassEQSRequestInterface::FMassEQSRequestHandler
//-----------------------------------------------------------------------------
void IMassEQSRequestInterface::FMassEQSRequestHandler::SendOrRecieveRequest(FEnvQueryInstance& QueryInstance, const IMassEQSRequestInterface& MassEQSRequestInterface)
{
#if WITH_EDITOR
	if (QueryInstance.World->IsEditorWorld() && !QueryInstance.World->IsPlayInEditor())
	{
		UE_LOGF(LogEQS, Warning, "Asynchronous request type [%ls] is only available when simulating the game.", *MassEQSRequestInterface.GetRequestClass()->GetName());
		return;
	}
#endif
	if (!MassEQSSubsystem)
	{
		check(QueryInstance.World)

		MassEQSSubsystem = QueryInstance.World->GetSubsystem<UMassEQSSubsystem>();
		CachedRequestQueueIndex = MassEQSSubsystem->GetRequestQueueIndex(MassEQSRequestInterface.GetRequestClass());
		check(MassEQSSubsystem);
	}
	
	if (RequestHandle > 0u)
	{
		float CurrentTime = QueryInstance.World->GetTimeSeconds();
		if (CurrentTime - RequestStartTime > MaxRequestTime)
		{
			CancelRequest();
			Reset();
		}
		else if (MassEQSRequestInterface.TryAcquireResults(QueryInstance))
		{
			Reset();
		}
	}
	else
	{
		RequestHandle = MassEQSSubsystem->PushRequest(QueryInstance, CachedRequestQueueIndex, MassEQSRequestInterface.GetRequestData(QueryInstance));
		RequestStartTime = QueryInstance.World->GetTimeSeconds();
	}
}

void IMassEQSRequestInterface::FMassEQSRequestHandler::CancelRequest() const
{
	if (!IsPendingResults())
	{
		// No active request
		return;
	}

	check(MassEQSSubsystem)
	MassEQSSubsystem->CancelRequest(RequestHandle);
}

void IMassEQSRequestInterface::FMassEQSRequestHandler::Reset()
{
	RequestHandle = FMassEQSRequestHandle::Invalid;
	RequestStartTime = -1.f;
}

IMassEQSRequestInterface::FMassEQSRequestHandler::~FMassEQSRequestHandler()
{
	CancelRequest();
}
