// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEQSUtils.h"

#include "MassEQSTypes.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "Items/EnvQueryItemType_MassEntityHandle.h"

#include "MassEntityTypes.h"

FMassEnvQueryEntityInfo FMassEQSUtils::GetItemAsEntityInfo(const FEnvQueryInstance& QueryInstance, int32 Index)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryInstance.ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	return DefTypeOb->GetValue(QueryInstance.RawData.GetData() + QueryInstance.Items[Index].DataOffset);
}

FMassEnvQueryEntityInfo FMassEQSUtils::GetItemAsEntityInfo(const FEnvQueryResult& QueryResult, int32 Index)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryResult.ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	return DefTypeOb->GetValue(QueryResult.RawData.GetData() + QueryResult.Items[Index].DataOffset);
}

void FMassEQSUtils::GetAllAsEntityInfo(const FEnvQueryInstance& QueryInstance, TArray<FMassEnvQueryEntityInfo>& OutEntityInfo)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryInstance.ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	OutEntityInfo.Reserve(OutEntityInfo.Num() + QueryInstance.Items.Num());
	for (const FEnvQueryItem& Item : QueryInstance.Items)
	{
		OutEntityInfo.Add(DefTypeOb->GetValue(QueryInstance.RawData.GetData() + Item.DataOffset));
	}
}

void FMassEQSUtils::GetAllAsEntityInfo(const FEnvQueryResult& QueryResult, TArray<FMassEnvQueryEntityInfo>& OutEntityInfo)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryResult.ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	OutEntityInfo.Reserve(OutEntityInfo.Num() + QueryResult.Items.Num());
	for (const FEnvQueryItem& Item : QueryResult.Items)
	{
		OutEntityInfo.Add(DefTypeOb->GetValue(QueryResult.RawData.GetData() + Item.DataOffset));
	}
}

void FMassEQSUtils::GetEntityHandles(const TArray<FMassEnvQueryEntityInfo>& EntityInfo, TArray<FMassEntityHandle>& OutHandles)
{
	OutHandles.Reserve(OutHandles.Num() + EntityInfo.Num());
	for (const FMassEnvQueryEntityInfo& Info : EntityInfo)
	{
		OutHandles.Add(Info.EntityHandle);
	}
}

void FMassEQSUtils::GetAllAsEntityHandles(const FEnvQueryInstance& QueryInstance, TArray<FMassEntityHandle>& OutHandles)
{
	UEnvQueryItemType_MassEntityHandle* DefTypeOb = QueryInstance.ItemType->GetDefaultObject<UEnvQueryItemType_MassEntityHandle>();
	check(DefTypeOb != nullptr);

	OutHandles.Reserve(OutHandles.Num() + QueryInstance.Items.Num());
	for (const FEnvQueryItem& Item : QueryInstance.Items)
	{
		OutHandles.Add(DefTypeOb->GetValue(QueryInstance.RawData.GetData() + Item.DataOffset).EntityHandle);
	}
}