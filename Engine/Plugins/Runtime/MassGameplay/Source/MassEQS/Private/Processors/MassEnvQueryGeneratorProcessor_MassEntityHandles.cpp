// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/MassEnvQueryGeneratorProcessor_MassEntityHandles.h"
#include "MassExecutionContext.h"
#include "MassCommonFragments.h"
#include "MassObserverRegistry.h"
#include "MassEQSUtils.h"
#include "MassEQSSubsystem.h"
#include "Generators/MassEnvQueryGenerator_MassEntityHandles.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryGeneratorProcessor_MassEntityHandles)


UMassEnvQueryGeneratorProcessor_MassEntityHandles::UMassEnvQueryGeneratorProcessor_MassEntityHandles()
	: EntityQuery(*this)
{
	CorrespondingRequestClass = UMassEnvQueryGenerator_MassEntityHandles::StaticClass();
}

void UMassEnvQueryGeneratorProcessor_MassEntityHandles::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	ProcessorRequirements.AddSubsystemRequirement<UMassEQSSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassEnvQueryGeneratorProcessor_MassEntityHandles::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	const UWorld* World = GetWorld();
	check(World);

	UMassEQSSubsystem* MassEQSSubsystem = Context.GetMutableSubsystem<UMassEQSSubsystem>();
	check(MassEQSSubsystem);

	// Check for any requests of this type from MassEQSSubsystem, complete one if found.
	TUniquePtr<FMassEQSRequestData> GeneratorDataUniquePtr = MassEQSSubsystem->PopRequest(CachedRequestQueryIndex);
	FMassEQSRequestData_MassEntityHandles* GeneratorData = FMassEQSUtils::TryAndEnsureCast<FMassEQSRequestData_MassEntityHandles>(GeneratorDataUniquePtr);
	if (!GeneratorData)
	{
		return;
	}

	const float SearchRadius = GeneratorData->SearchRadius;
	const TArray<FVector>& ContextPositions = GeneratorData->ContextPositions;

	TArray<FMassEnvQueryEntityInfo> Items = {};
	if (SearchRadius <= 0)
	{
		EntityQuery.ForEachEntityChunk(Context, [&GeneratorData, &Items, &SearchRadius, &ContextPositions](FMassExecutionContext& ChunkContext)
		{
			const TConstArrayView<FTransformFragment> TransformFragmentList = ChunkContext.GetFragmentView<FTransformFragment>();
			for (FMassExecutionContext::FEntityIterator EntityIt = ChunkContext.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassEntityHandle EntityHandle = ChunkContext.GetEntity(EntityIt);
				Items.Emplace(EntityHandle.Index, EntityHandle.SerialNumber, TransformFragmentList[EntityIt].GetTransform());
			}
		});
	}
	else
	{
		FVector::FReal SearchRadiusSqr = FMath::Square(SearchRadius);
		EntityQuery.ForEachEntityChunk(Context, [&GeneratorData, &Items, &SearchRadiusSqr, &ContextPositions](FMassExecutionContext& ChunkContext)
		{
			const TConstArrayView<FTransformFragment> TransformFragmentList = ChunkContext.GetFragmentView<FTransformFragment>();
			for (FMassExecutionContext::FEntityIterator EntityIt = ChunkContext.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FTransformFragment& TransformFragment = TransformFragmentList[EntityIt];
				const FVector& EntityPosition = TransformFragment.GetTransform().GetTranslation();
				for (const FVector& ContextPosition : ContextPositions)
				{
					const FVector::FReal ContextDistanceToEntitySqr = FVector::DistSquared(EntityPosition, ContextPosition);

					if (ContextDistanceToEntitySqr <= SearchRadiusSqr)
					{
						FMassEntityHandle EntityHandle = ChunkContext.GetEntity(EntityIt);
						Items.Emplace(EntityHandle.Index, EntityHandle.SerialNumber, TransformFragmentList[EntityIt].GetTransform());
						break;
					}
				}
			}
		});
	}
	MassEQSSubsystem->SubmitResults(GeneratorData->RequestHandle, MakeUnique<FMassEnvQueryResultData_MassEntityHandles>(MoveTemp(Items)));
}
