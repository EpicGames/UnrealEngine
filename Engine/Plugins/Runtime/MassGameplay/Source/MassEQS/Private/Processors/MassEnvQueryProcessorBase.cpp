// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/MassEnvQueryProcessorBase.h"
#include "Engine/World.h"
#include "MassEQSSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryProcessorBase)

void UMassEnvQueryProcessorBase::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	if (CorrespondingRequestClass)
	{
		UWorld* World = Owner.GetWorld();
		check(World);
		UMassEQSSubsystem* MassEQSSubsystem = World->GetSubsystem<UMassEQSSubsystem>();
		check(MassEQSSubsystem)
	
		CachedRequestQueryIndex = MassEQSSubsystem->GetRequestQueueIndex(CorrespondingRequestClass);
	}
}
