// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/MassEnvQueryTestProcessor_MassEntityTags.h"
#include "Tests/MassEnvQueryTest_MassEntityTags.h"
#include "MassEQSSubsystem.h"
#include "MassEQSUtils.h"
#include "MassEQS.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryTestProcessor_MassEntityTags)


namespace Mass::EQS::Utils
{

	bool TestChunkForAnyTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context);
	bool TestChunkForAllTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context);
	bool TestChunkForNoTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context);

} // namespace Mass::EQS::Utils


UMassEnvQueryTestProcessor_MassEntityTags::UMassEnvQueryTestProcessor_MassEntityTags()
	: EntityQuery(*this)
{
	CorrespondingRequestClass = UMassEnvQueryTest_MassEntityTags::StaticClass();
}

void UMassEnvQueryTestProcessor_MassEntityTags::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassEQSSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassEnvQueryTestProcessor_MassEntityTags::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	const UWorld* World = GetWorld();
	check(World);

	UMassEQSSubsystem* MassEQSSubsystem = ExecutionContext.GetMutableSubsystem<UMassEQSSubsystem>();
	check(MassEQSSubsystem);

	// Check for any requests of this type from MassEQSSubsystem, complete one if found.
	TUniquePtr<FMassEQSRequestData> TestDataUniquePtr = MassEQSSubsystem->PopRequest(CachedRequestQueryIndex);
	FMassEQSRequestData_MassEntityTags* TestData = FMassEQSUtils::TryAndEnsureCast<FMassEQSRequestData_MassEntityTags>(TestDataUniquePtr);
	if (!TestData)
	{
		return;
	}
	if (TestDataUniquePtr->EntityHandles.IsEmpty())
	{
		UE_LOGF(LogMassEQS, Error, "Request: [%ls] Acquired by UMassEnvQueryTestProcessor_MassEntityTags, but had no Entities to query.", *TestDataUniquePtr->RequestHandle.ToString());
		return;
	}

	const EMassEntityTagsTestMode TagTestMode = TestData->TagTestMode;
	const TArray<FInstancedStruct> Tags = TestData->Tags;

	TFunction<bool(const TArray<FInstancedStruct>&, const FMassExecutionContext&)> ScoringFunction;
	TMap<FMassEntityHandle, bool> ScoreMap = {};

	switch (TagTestMode)
	{
	case EMassEntityTagsTestMode::Any:
	{
		ScoringFunction = Mass::EQS::Utils::TestChunkForAnyTags;
		break;
	}
	case EMassEntityTagsTestMode::All:
	{
		ScoringFunction = Mass::EQS::Utils::TestChunkForAllTags;
		break;
	}
	case EMassEntityTagsTestMode::None:
	{
		ScoringFunction = Mass::EQS::Utils::TestChunkForNoTags;
		break;
	}
	default:
	{
		ScoringFunction = [](const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context) { return false; };
		break;
	}
	}

	ensureMsgf(ExecutionContext.GetEntityCollection().IsEmpty(), TEXT("We don't expect any collections to be set at this point. The data is going to be overridden."));

	TArray<FMassArchetypeEntityCollection> EntityCollectionsToTest;
	UE::Mass::Utils::CreateEntityCollections(EntityManager, TestDataUniquePtr->EntityHandles, FMassArchetypeEntityCollection::NoDuplicates, EntityCollectionsToTest);

	EntityQuery.ForEachEntityChunkInCollections(EntityCollectionsToTest, ExecutionContext
		, [&ScoreMap, &ScoringFunction, &Tags](FMassExecutionContext& Context)
		{
			bool ChunkSuccess = ScoringFunction(Tags, Context);

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
				ScoreMap.Add(EntityHandle, ChunkSuccess);
			}
		});

	MassEQSSubsystem->SubmitResults(TestData->RequestHandle, MakeUnique<FMassEnvQueryResultData_MassEntityTags>(MoveTemp(ScoreMap)));
}


bool Mass::EQS::Utils::TestChunkForAnyTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context)
{
	for (const FInstancedStruct& Tag : Tags)
	{
		const UScriptStruct* TagScriptStruct = Tag.GetScriptStruct();
#if !UE_BUILD_SHIPPING
		if (!ensureMsgf(UE::Mass::IsA<FMassTag>(TagScriptStruct), TEXT("Non Tag Element Detected in Environment Query Mass-Entity-Tags-Test. Likely left empty.")))
		{
			continue;
		}
#endif
		if (Context.DoesArchetypeHaveTag(TagScriptStruct))
		{
			return true;
		}
	}

	return false;
}

bool Mass::EQS::Utils::TestChunkForAllTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context)
{
	for (const FInstancedStruct& Tag : Tags)
	{
		const UScriptStruct* TagScriptStruct = Tag.GetScriptStruct();
#if !UE_BUILD_SHIPPING
		if (!ensureMsgf(UE::Mass::IsA<FMassTag>(TagScriptStruct), TEXT("Non Tag Element Detected in Environment Query Mass-Entity-Tags-Test. Likely left empty.")))
		{
			continue;
		}
#endif
		if (!Context.DoesArchetypeHaveTag(TagScriptStruct))
		{
			return false;
		}
	}

	return true;
}

bool Mass::EQS::Utils::TestChunkForNoTags(const TArray<FInstancedStruct>& Tags, const FMassExecutionContext& Context)
{
	for (const FInstancedStruct& Tag : Tags)
	{
		const UScriptStruct* TagScriptStruct = Tag.GetScriptStruct();
#if !UE_BUILD_SHIPPING
		if (!ensureMsgf(UE::Mass::IsA<FMassTag>(TagScriptStruct), TEXT("Non Tag Element Detected in Environment Query Mass-Entity-Tags-Test. Likely left empty.")))
		{
			continue;
		}
#endif
		if (Context.DoesArchetypeHaveTag(Tag.GetScriptStruct()))
		{
			return false;
		}
	}

	return true;
}








