// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MassEnvQueryTest.h"
#include "MassEQSSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryTest)

UMassEnvQueryTest::UMassEnvQueryTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMassEnvQueryTest::RunTest(FEnvQueryInstance& QueryInstance) const
{
	MassEQSRequestHandler.SendOrRecieveRequest(QueryInstance, *this);
}
