// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MassEnvQueryTest_MassEntityTags.h"
#include "Items/EnvQueryItemType_MassEntityHandle.h"
#include "MassEQSUtils.h"
#include "MassEQSSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEnvQueryTest_MassEntityTags)


UMassEnvQueryTest_MassEntityTags::UMassEnvQueryTest_MassEntityTags(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Low;
	TestPurpose = EEnvTestPurpose::Type::Filter;
	FilterType = EEnvTestFilterType::Type::Match;

	ValidItemType = UEnvQueryItemType_MassEntityHandle::StaticClass();

	SetWorkOnFloatValues(false);

}

TUniquePtr<FMassEQSRequestData> UMassEnvQueryTest_MassEntityTags::GetRequestData(FEnvQueryInstance& QueryInstance) const
{
	return MakeUnique<FMassEQSRequestData_MassEntityTags>(TagTestMode, Tags);
}

bool UMassEnvQueryTest_MassEntityTags::TryAcquireResults(FEnvQueryInstance& QueryInstance) const
{
	check(MassEQSRequestHandler.MassEQSSubsystem)
	TUniquePtr<FMassEQSRequestData> RawRequestData = MassEQSRequestHandler.MassEQSSubsystem->TryAcquireResults(MassEQSRequestHandler.RequestHandle);
	FMassEnvQueryResultData_MassEntityTags* RequestData = FMassEQSUtils::TryAndEnsureCast<FMassEnvQueryResultData_MassEntityTags>(RawRequestData);
	if (!RequestData)
	{
		return false;
	}

	FEnvQueryInstance::ItemIterator It(this, QueryInstance);
	for (It.IgnoreTimeLimit(); It; ++It)
	{
		const FMassEnvQueryEntityInfo& EntityInfo = FMassEQSUtils::GetItemAsEntityInfo(QueryInstance, It.GetIndex());

		const bool bSuccess = RequestData->ResultMap.Contains(EntityInfo.EntityHandle) && RequestData->ResultMap[EntityInfo.EntityHandle];

		It.SetScore(TestPurpose, FilterType, bSuccess, true);
	}

	return true;
}

FText UMassEnvQueryTest_MassEntityTags::GetDescriptionTitle() const
{
	return FText::FromString(FString::Printf(TEXT("Mass Entity Tags Test : Match %s Tags"),
		TagTestMode == EMassEntityTagsTestMode::Any ? TEXT("Any") :
		TagTestMode == EMassEntityTagsTestMode::All ? TEXT("All") : TEXT("None")));
}

FText UMassEnvQueryTest_MassEntityTags::GetDescriptionDetails() const
{
	return DescribeFloatTestParams();
}
