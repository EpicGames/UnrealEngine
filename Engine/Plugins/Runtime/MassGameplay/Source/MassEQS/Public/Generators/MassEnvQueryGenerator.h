// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "MassEQSTypes.h"
#include "MassEnvQueryGenerator.generated.h"

/** Generator that will send its work to MassEQSSubsystem in order to be processed in a Mass Processor */
UCLASS(EditInlineNew, Abstract, meta = (Category = "Generators"), MinimalAPI)
class UMassEnvQueryGenerator : public UEnvQueryGenerator, public IMassEQSRequestInterface
{
	GENERATED_UCLASS_BODY()
public:
	/** 
	 * This will send this Generator Request to MassEQSSubsystem the first time it is called,
	 * and Try to complete Generation with the Result from MassEQSSubsystem on subsequent calls.
	*/
	virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const final;

	virtual inline bool IsCurrentlyRunningAsync() const final { return MassEQSRequestHandler.IsPendingResults(); }

	// To be implemented by child class:
	// Begin IMassEQSRequestInterface
	virtual TUniquePtr<FMassEQSRequestData> GetRequestData(FEnvQueryInstance& QueryInstance) const override PURE_VIRTUAL(UMassEnvQueryGenerator::GetRequestData, return nullptr;);
	virtual UClass* GetRequestClass() const override PURE_VIRTUAL(UMassEnvQueryGenerator::GetRequestData, return nullptr;);
	virtual bool TryAcquireResults(FEnvQueryInstance& QueryInstance) const override PURE_VIRTUAL(UMassEnvQueryGenerator::TryAcquireResults, return false;);
	// ~IMassEQSRequestInterface

protected:
	mutable FMassEQSRequestHandler MassEQSRequestHandler;
};