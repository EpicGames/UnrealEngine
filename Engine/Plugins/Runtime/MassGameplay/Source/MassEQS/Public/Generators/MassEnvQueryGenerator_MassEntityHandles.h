// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataProviders/AIDataProvider.h"
#include "Generators/MassEnvQueryGenerator.h"
#include "MassEQSTypes.h"
#include "MassEnvQueryGenerator_MassEntityHandles.generated.h"

struct FMassEnvQueryEntityInfo;

/**
 * Generator to be sent to MassEQSSubsystem for processing on Mass.
 * This will Generate UEnvQueryItemType_MassEntityHandles within SearchRadius of any ContextPositions.
 * Set SearchRadius to a value <= 0 in order to get all EntityHandles who have an FTransformFragment.
 */
UCLASS(meta = (DisplayName = "Mass Entity Handles"), MinimalAPI)
class UMassEnvQueryGenerator_MassEntityHandles : public UMassEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

public:
	// Begin IMassEQSRequestInterface
	virtual TUniquePtr<FMassEQSRequestData> GetRequestData(FEnvQueryInstance& QueryInstance) const override;
	virtual UClass* GetRequestClass() const override { return StaticClass(); }
	
	virtual bool TryAcquireResults(FEnvQueryInstance& QueryInstance) const override;
	// ~IMassEQSRequestInterface

protected:
	/** Any Entity who is within SearchRadius of any SearchCenter will be acquired */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue SearchRadius;

	/** Context of query */
	UPROPERTY(EditAnywhere, Category=Generator)
	TSubclassOf<UEnvQueryContext> SearchCenter;
};

/** Data required to be sent to Mass for processing this Generator Request */
struct FMassEQSRequestData_MassEntityHandles : public FMassEQSRequestData
{
	FMassEQSRequestData_MassEntityHandles(const TArray<FVector>& InContextPositions, const float InSearchRadius)
		: ContextPositions(InContextPositions)
		, SearchRadius(InSearchRadius)
	{
	}

	TArray<FVector> ContextPositions;
	float SearchRadius;
};

struct FMassEnvQueryResultData_MassEntityHandles: public FMassEQSRequestData
{
	FMassEnvQueryResultData_MassEntityHandles(TArray<FMassEnvQueryEntityInfo>&& InGeneratedEntityInfo)
		: GeneratedEntityInfo(MoveTemp(InGeneratedEntityInfo))
	{
	}

	TArray<FMassEnvQueryEntityInfo> GeneratedEntityInfo;
};
