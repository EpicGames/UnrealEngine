// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEQSTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvQueryItemType_MassEntityHandle.generated.h"

/** 
 * EnvQueryType representing a MassEntity. Additionally stores a Cached Transform of the Entity at the
 * time of acquisition, in order to use in the implemented UEnvQueryItemType_VectorBase functions.
 */
UCLASS(meta = (DisplayName = "Mass Entity Handles Item Type"), MinimalAPI)
class UEnvQueryItemType_MassEntityHandle : public UEnvQueryItemType_VectorBase
{
	GENERATED_BODY()

public:
	typedef FMassEnvQueryEntityInfo FValueType;

	UEnvQueryItemType_MassEntityHandle();

	static const FMassEnvQueryEntityInfo& GetValue(const uint8* RawData);
	static void SetValue(uint8* RawData, const FMassEnvQueryEntityInfo& Value);

	virtual FVector GetItemLocation(const uint8* RawData) const override;
	virtual FRotator GetItemRotation(const uint8* RawData) const override;
};
