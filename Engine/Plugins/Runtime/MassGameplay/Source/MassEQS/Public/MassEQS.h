// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMassEQS, Log, All);

class FMassEQSModule : public IModuleInterface
{
};
