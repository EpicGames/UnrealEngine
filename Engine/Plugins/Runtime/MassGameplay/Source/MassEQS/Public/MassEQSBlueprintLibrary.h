// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MassEntityTypes.h"
#include "MassEQSTypes.h"
#include "MassEQSUtils.h"
#include "MassEQSBlueprintLibrary.generated.h"

class UEnvQueryInstanceBlueprintWrapper;
struct FEnvQueryResult;

/** Wrapper for Blueprints to be able to parse MassEntityInfo to use functions defined in UMassEQSBlueprintLibrary */
USTRUCT(Blueprintable, BlueprintType, meta = (DisplayName = "Mass Entity Info"))
struct FMassEnvQueryEntityInfoBlueprintWrapper
{
    GENERATED_BODY()

public:
	FMassEnvQueryEntityInfoBlueprintWrapper() = default;
	FMassEnvQueryEntityInfoBlueprintWrapper(FMassEnvQueryEntityInfo InEntityInfo)
		: EntityInfo(InEntityInfo)
	{
	}

	inline FVector GetCachedEntityPosition() const { return EntityInfo.CachedTransform.GetLocation(); }
	inline FMassEntityHandle GetEntityHandle() const { return EntityInfo.EntityHandle; }
	inline void SetEntityHandle(FMassEntityHandle Handle) { EntityInfo.EntityHandle = Handle; }

	const FMassEnvQueryEntityInfo& GetEntityInfo() const { return EntityInfo; };
	inline bool operator==(const FMassEnvQueryEntityInfoBlueprintWrapper& Other) const { return GetEntityInfo() == Other.GetEntityInfo(); }

private:
	FMassEnvQueryEntityInfo EntityInfo;

};

/** Function library for interfacing with EntityInfo inside blueprints. */
UCLASS()
class UMassEQSBlueprintLibrary : public UBlueprintFunctionLibrary
{
    GENERATED_UCLASS_BODY()

public:
	
	//----------------------------------------------------------------------//
	// Commands
	//----------------------------------------------------------------------//

	/** Sends the input Signal to the Entity defined by EntityInfo.EntityHandle using the UMassSignalSubsystem. */
	UFUNCTION(BlueprintCallable, Category = "MassEnvQuery|Commands")
	static void SendSignalToEntity(const AActor* Owner, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo, const FName Signal);

	//----------------------------------------------------------------------//
	// Utils
	//----------------------------------------------------------------------//

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MassEnvQuery|Utils")
	static inline FString EntityToString(const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo){ return EntityInfo.GetEntityHandle().DebugGetDescription(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MassEnvQuery|Utils")
	static inline FVector GetCachedEntityPosition(const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo) { return EntityInfo.GetCachedEntityPosition(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MassEnvQuery|Utils")
	static FVector GetCurrentEntityPosition(const AActor* Owner, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo);

	/** Custom comparison function, as the Blueprint Equals did not seem to work. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MassEnvQuery|Utils")
	static inline bool EntityComparison(const FMassEnvQueryEntityInfoBlueprintWrapper& A, const FMassEnvQueryEntityInfoBlueprintWrapper& B) { return A == B; }

	/** Custom array-contains function, as the Blueprint version did not seem to work. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MassEnvQuery|Utils")
	static bool ContainsEntity(const TArray<FMassEnvQueryEntityInfoBlueprintWrapper>& EntityList, const FMassEnvQueryEntityInfoBlueprintWrapper& EntityInfo);

	/** Outputs an array filled with resulting EntityInfos. Note that it makes sense only if ItemType is a EnvQueryItemType_MassEntityHandle-derived type. */
	UFUNCTION(BlueprintCallable, Category = "AI|EQS")
	static TArray<FMassEnvQueryEntityInfoBlueprintWrapper> GetEnviromentQueryResultAsEntityInfo(const UEnvQueryInstanceBlueprintWrapper* QueryInstance);

private:
	/** Get result and immediately convert to EntityInfoBlueprintWrapper to skip an extra copy step */
	static FMassEnvQueryEntityInfoBlueprintWrapper GetItemAsEntityInfoBPWrapper(const FEnvQueryResult* QueryResult, int32 Index);

	/** Get result array and convert each EntityInfo to EntityInfoBlueprintWrapper along the way to skip extra copy steps */
	static void GetAllAsEntityInfoBPWrappers(const FEnvQueryResult* QueryResult, TArray<FMassEnvQueryEntityInfoBlueprintWrapper>& OutEntityInfo);
};