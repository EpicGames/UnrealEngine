// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "EnvironmentQuery/EnvQueryNode.h"
#include "MassEQSTypes.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassEQSSubsystem.generated.h"

/**
 * Subsystem for sending UMassEQSGenerator and UMassEQSTest requests to Mass Processors, 
 * and for returning the results back to EQS. 
 */
UCLASS()
class UMassEQSSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
	/** Queue holding UniquePtrs to the data required for each Request */
	using FRequestQueue = TQueue<TUniquePtr<FMassEQSRequestData>>;

public:
	/** Push new Request into RequestQueue */
	FMassEQSRequestHandle PushRequest(const FEnvQueryInstance& QueryInstance, const int32 RequestQueueIndex, TUniquePtr<FMassEQSRequestData>&& Request);

	/** Dequeue next Request off the RequestQueue, removing it from the queue, and transferring ownership of UniquePtr */
	MASSEQS_API TUniquePtr<FMassEQSRequestData> PopRequest(const int32 RequestQueueIndex);

	/** Send results from a finished/processed request to be stored in this subsystem */
	MASSEQS_API void SubmitResults(FMassEQSRequestHandle RequestHandle, TUniquePtr<FMassEQSRequestData>&& Result);
	/** Try to Acquire the results from a request from the QueryInstance, if it is available. */
	MASSEQS_API TUniquePtr<FMassEQSRequestData> TryAcquireResults(FMassEQSRequestHandle RequestHandle);

	/** Returns the index into RequestQueues for the input class */
	int32 GetRequestQueueIndex(TSubclassOf<UEnvQueryNode> RequestClass);

	/**
	 * If Request in RequestQueue -> Remove from Queue
	 * If Result in Results -> Remove from Results
	 * Otherwise, Request is currently being processed -> Log this ID as Cancelled, and ignore result when submitted.
	 */
	void CancelRequest(FMassEQSRequestHandle RequestHandle);

	/** Finalization step for Handle Management */
	 void FinalizeRequest(FMassEQSRequestHandle RequestHandle);

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

protected:
	/**
	 * Mapping from Class of requester, to index into RequestQueues.
	 * Each Class that extends UMassEQSGenerator or UMassEQSTest 
	 * can be a requester, this will store a Queue for each of these types.
	 * @see PostInitialize()
	 */
	TMap<TSubclassOf<UEnvQueryNode>, int32> RequestQueueLookup = {};
	/**
	 * Array of Queues holding each Request. 
	 * TQueue does not support copy/assignment, which TMap requires.
	 * This is the reason the TQueues are stored in a separate Array.
	 */
	TArray<FRequestQueue> RequestQueues = {};
	/** 
	 * The detector only guards the extension of RequestQueues array. Modifying specific elements is not guarded since 
	 * those are a thread safe queues.
	 */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(RequestAccessDetector);

	/** Holds results for finished Query */
	TMap<FMassEQSRequestHandle, TUniquePtr<FMassEQSRequestData>> Results;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(ResultAccessDetector);

	FIndexedHandleManagerBase<FMassEQSRequestHandle, int32, uint32, true> HandleManager;

	/** 
	 * Holds all RequestHandles of requests who have been canceled, and not yet submitted.
	 * Once this request is submitted, we will ignore the results.
	 */
	TArray<FMassEQSRequestHandle> CancelledRequests;

	/** Used to ignore request cancellation after subsystem is De-Initialized */
	bool bIsInitialized = false;
};

template<>
struct TMassExternalSubsystemTraits<UMassEQSSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = true,
	};
};
