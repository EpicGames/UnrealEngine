// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexedHandle.h"
#include "Mass/EntityHandle.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/Interface.h"
#include "MassEQSTypes.generated.h"

struct FEnvQueryInstance;
struct FMassEQSRequestHandle;
class UMassEQSSubsystem;

/** Contents of EnvQueryItemType_MassEntityHandle
 *  @see EnvQueryItemType_MassEntityHandle
 */
USTRUCT()
struct FMassEnvQueryEntityInfo
{
	GENERATED_BODY()

	FMassEnvQueryEntityInfo() = default;
	FMassEnvQueryEntityInfo(int32 Index, int32 SerialNumber, const FTransform& Transform);

	FMassEntityHandle EntityHandle;
	/** Cached Transform from time of acquisition allows EnvQueryItemType_MassEntityHandle to implement UEnvQueryItemType_VectorBase */
	FTransform CachedTransform;

	inline bool operator==(const FMassEnvQueryEntityInfo& Other) const { return EntityHandle == Other.EntityHandle; }
};

/**
 * Provides a Unique Handle for incoming MassEQS Requests
 */
USTRUCT()
struct FMassEQSRequestHandle : public FIndexedHandleBase
{
	GENERATED_BODY()
	FMassEQSRequestHandle() = default;
	FMassEQSRequestHandle(int32 InIndex, uint32 InSerialNumber) : FIndexedHandleBase(InIndex, InSerialNumber)
	{
	}

	FString ToString() const;

	static FMassEQSRequestHandle Invalid;
};

/** To be extended to hold any data which needs to be transferred between EQS and Mass for a given Request. */
struct FMassEQSRequestData
{
	FMassEQSRequestData() = default;
	FMassEQSRequestData(FMassEQSRequestHandle Handle)
		: RequestHandle(Handle)
	{
	}

	/** Unique ID that MassEQSSubsystem's HandleManager has assigned to this Request */
	FMassEQSRequestHandle RequestHandle = FMassEQSRequestHandle::Invalid;

	/** EntityHandles to run this request on. Used for Tests. */
	TArray<FMassEntityHandle> EntityHandles;
};

/** Interface for sending and receiving EQS Requests to and from MassEQSSubsystem */
UINTERFACE()
class UMassEQSRequestInterface : public UInterface
{
	GENERATED_BODY()
};

/** Interface for sending and receiving EQS Requests to and from MassEQSSubsystem utilizing a MassEQSRequestHandler */
class IMassEQSRequestInterface
{
	GENERATED_BODY()
public:

	/**
	 * Stores relevant information for managing requests sent to MassEQSSubsystem.
	 * Provides generic functionality for Sending, Receiving, and Canceling these requests.
	 * Any class who implements IMassEQSRequestInterface should have one of these.
	 */
	struct FMassEQSRequestHandler
	{
		/**
		 * Called when an EnvQueryInstance is trying to run a Generator or Test who implements this.
		 * If this Request has not yet been sent (IE. RequestID == 0u) then we will send this request
		 * Otherwise, if this Request was already sent (IE. RequestID > 0u) then we will call TryAcquireResults on the input MassEnvRequestInterface.
		 * @param QueryInstance - The EnvQueryInstance sending/receiving this request
		 * @param MassEQSRequestInterface - The Interface used to call this,
		 * @return A unique ID that the QueryInstance can hold on to in order to later retrieve this request result.
		 */
		void SendOrRecieveRequest(FEnvQueryInstance& QueryInstance, const IMassEQSRequestInterface& MassEQSRequestInterface);

		/** Cancel request if currently ongoing. */
		void CancelRequest() const;

		/** Reset RequestID to 0 and RequestStartTime to -1.f */
		void Reset();

		inline bool IsPendingResults() const { return RequestHandle.IsValid(); }

		// Handle for request in MassEQSSubsystem.
		FMassEQSRequestHandle RequestHandle = FMassEQSRequestHandle::Invalid;
		// Cached MassEQSSubsystem pointer for interacting with this subsystem
		TObjectPtr<UMassEQSSubsystem> MassEQSSubsystem = nullptr;
		// The point in time that an ongoing request was sent, -1 if none have been sent.
		float RequestStartTime = -1.f;
		// The max amount of time allowed for this request. CancelRequest will be called if this is used after this time has elapsed.
		float MaxRequestTime = DefaultMaxRequestTime;

		int32 CachedRequestQueueIndex = -1;

		FMassEQSRequestHandler() = default;
		~FMassEQSRequestHandler();

	protected:
		static constexpr float DefaultMaxRequestTime = 5.f;
	};


	/**
	 * Called inside RequestProcessing, in order to create a copy of the relevant data for this Request.
	 * @param QueryInstance - The EnvQueryInstance that owns this Generator
	 * @return A UniquePtr to the Data-Copy created.
	 */
	virtual TUniquePtr<FMassEQSRequestData> GetRequestData(FEnvQueryInstance& QueryInstance) const = 0;

	/**
	 * Get StaticClass of the ImplementingClass. Implementer of this Interface must provide this so we can use
	 * the correct StaticClass inside RequestProcessing.
	 * Implementation should just be StaticClass() called in derived class.
	 */
	virtual UClass* GetRequestClass() const = 0;

	/**
	 * Called when EnvQueryInstance is checking to try to complete its request.
	 * @param QueryInstance - The EnvQueryInstance that owns this Generator
	 */
	virtual bool TryAcquireResults(FEnvQueryInstance& QueryInstance) const = 0;
};