// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MassEQSTypes.h"

#define UE_API MASSEQS_API

struct FEnvQueryResult;
struct FEnvQueryInstance;
struct FMassEnvQueryEntityInfo;
struct FMassEntityHandle;
struct FMassEntityHandle;
struct FMassEQSRequestData;

/** Holds Utility functions for Mass EQS Needs */
struct FMassEQSUtils
{

	/** Returns the Item stored in QueryInstance/QueryResult Items[Index] as EntityInfo */
	static UE_API FMassEnvQueryEntityInfo GetItemAsEntityInfo(const FEnvQueryInstance& QueryInstance, int32 Index);
	static UE_API FMassEnvQueryEntityInfo GetItemAsEntityInfo(const FEnvQueryResult& QueryResult, int32 Index);

	/** Returns all Items stored in QueryInstance/QueryResult as EntityInfo */
	static UE_API void GetAllAsEntityInfo(const FEnvQueryInstance& QueryInstance, TArray<FMassEnvQueryEntityInfo>& OutEntityInfo);
	static UE_API void GetAllAsEntityInfo(const FEnvQueryResult& QueryResult, TArray<FMassEnvQueryEntityInfo>& OutEntityInfo);

	/** Extracts all EntityHandles out of an array of EntityInfo */
	static UE_API void GetEntityHandles(const TArray<FMassEnvQueryEntityInfo>& EntityInfo, TArray<FMassEntityHandle>& OutHandles);
	static UE_API void GetAllAsEntityHandles(const FEnvQueryInstance& QueryInstance, TArray<FMassEntityHandle>& OutHandles);

	/**
	 * Used in MassEnvQueryProcessors, to cast generic FMassEQSRequestData to its corresponding child class.
	 * If InPtr is not null, then this Cast should never fail.
	 */
	template<typename RequestDataType>
	static inline RequestDataType* TryAndEnsureCast(TUniquePtr<FMassEQSRequestData>& InPtr)
	{
		if (!InPtr)
		{
			return nullptr;
		}

		RequestDataType* OutPtr = static_cast<RequestDataType*>(InPtr.Get());
		ensureMsgf(OutPtr, TEXT("RequestData was pushed to MassEQSSubsystem, but corresponding child RequestData was not found."));

		return OutPtr;
	}
};

#undef UE_API
