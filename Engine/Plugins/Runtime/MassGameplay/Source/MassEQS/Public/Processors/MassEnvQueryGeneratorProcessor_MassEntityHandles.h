// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Processors/MassEnvQueryProcessorBase.h"
#include "MassEntityQuery.h"
#include "MassEnvQueryGeneratorProcessor_MassEntityHandles.generated.h"

/** Processor for completing MassEQSSubsystem Requests sent from UMassEnvQueryGenerator_MassEntityHandles */
UCLASS(meta = (DisplayName = "Mass Entity Handles Generator Processor"))
class UMassEnvQueryGeneratorProcessor_MassEntityHandles : public UMassEnvQueryProcessorBase
{
	GENERATED_BODY()
public:
	UMassEnvQueryGeneratorProcessor_MassEntityHandles();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};