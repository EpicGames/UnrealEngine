// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEnvQueryProcessorBase.generated.h"

#define UE_API MASSEQS_API

class UEnvQueryNode;

/** Processor for completing MassEQSSubsystem Requests sent from UMassEnvQueryTest_MassEntityTags */
UCLASS(MinimalAPI, Abstract, meta = (DisplayName = "Mass EQS Processor Base"))
class UMassEnvQueryProcessorBase : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager);
	
	TSubclassOf<UEnvQueryNode> CorrespondingRequestClass;
	int32 CachedRequestQueryIndex = -1;
};

#undef UE_API
