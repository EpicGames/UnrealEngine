// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "Processors/MassEnvQueryProcessorBase.h"
#include "MassEnvQueryTestProcessor_MassEntityTags.generated.h"

/** Processor for completing MassEQSSubsystem Requests sent from UMassEnvQueryTest_MassEntityTags */
UCLASS(meta = (DisplayName = "Mass Entity Tags Test Processor"))
class UMassEnvQueryTestProcessor_MassEntityTags : public UMassEnvQueryProcessorBase
{
	GENERATED_BODY()
public:
	UMassEnvQueryTestProcessor_MassEntityTags();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }

	FMassEntityQuery EntityQuery;
};