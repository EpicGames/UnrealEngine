// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQuery/EnvQueryTest.h"
#include "MassEQSTypes.h"
#include "MassEnvQueryTest.generated.h"

/** Test that will send its work to MassEQSSubsystem in order to be processed in a Mass Processor */
UCLASS(EditInlineNew, Abstract, meta = (Category = "Tests"), MinimalAPI)
class UMassEnvQueryTest : public UEnvQueryTest, public IMassEQSRequestInterface
{
	GENERATED_UCLASS_BODY()
public:
	/**
	 * This will send this Test Request to MassEQSSubsystem the first time it is called,
	 * and Try to complete Testing with the Result from MassEQSSubsystem on subsequent calls.
	 */
	MASSEQS_API virtual void RunTest(FEnvQueryInstance& QueryInstance) const final;
	virtual FORCEINLINE bool IsCurrentlyRunningAsync() const final { return MassEQSRequestHandler.IsPendingResults(); }

	// Begin IMassEQSRequestInterface
	virtual TUniquePtr<FMassEQSRequestData> GetRequestData(FEnvQueryInstance& QueryInstance) const override PURE_VIRTUAL(UMassEnvQueryTest::GetRequestData, return nullptr;)
	virtual UClass* GetRequestClass() const override PURE_VIRTUAL(UMassEnvQueryTest::GetRequestClass, return nullptr;);

	virtual bool TryAcquireResults(FEnvQueryInstance& QueryInstance) const override PURE_VIRTUAL(UMassEnvQueryTest::GetRequestData, return false;)
	// ~IMassEQSRequestInterface

protected:
	mutable FMassEQSRequestHandler MassEQSRequestHandler;
};
