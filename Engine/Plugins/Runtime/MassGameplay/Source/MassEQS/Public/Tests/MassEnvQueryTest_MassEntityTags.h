// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/MassEnvQueryTest.h"
#include "StructUtils/InstancedStruct.h"
#include "MassEnvQueryTest_MassEntityTags.generated.h"


struct FMassEntityHandle;

/** Different modes that this Test can be run in */
UENUM(BlueprintType)
enum class EMassEntityTagsTestMode : uint8
{
	Any = 0	UMETA(DisplayName = "Any Tags", Tooltip = "Filter will require just one of the tags to be present on the Entity."),
	All		UMETA(DisplayName = "All Tags", Tooltip = "Filter will require All of the tags to be present on the Entity."),
	None	UMETA(DisplayName = "None of the Tags", Tooltip = "Filter will require that none of the tags are present on the Entity.")
};

/**
 * Test to be sent to MassEQSSubsystem for processing on Mass.
 * This will test the Entities in the QueryInstance based on the MassTags they have in comparison to the input Tags, and the TagTestMode selected.
 */
UCLASS(MinimalAPI)
class UMassEnvQueryTest_MassEntityTags : public UMassEnvQueryTest
{
	GENERATED_UCLASS_BODY()
public:
	// Begin IMassEQSRequestInterface
	virtual TUniquePtr<FMassEQSRequestData> GetRequestData(FEnvQueryInstance& QueryInstance) const override;
	virtual UClass* GetRequestClass() const override { return StaticClass(); }

	virtual bool TryAcquireResults(FEnvQueryInstance& QueryInstance) const override;
	// ~IMassEQSRequestInterface

protected:
	virtual FText GetDescriptionTitle() const override;
	virtual FText GetDescriptionDetails() const override;

	UPROPERTY(EditAnywhere, Category = "MassEntityTagsTest")
	EMassEntityTagsTestMode TagTestMode = EMassEntityTagsTestMode::All;

	UPROPERTY(EditAnywhere, Category = "MassEntityTagsTest", meta = (BaseStruct = "/Script/MassEntity.MassTag", ExcludeBaseStruct))
	TArray<FInstancedStruct> Tags;
};

/** Data required to be sent to Mass for processing this Test Request */
struct FMassEQSRequestData_MassEntityTags : public FMassEQSRequestData
{
	FMassEQSRequestData_MassEntityTags(EMassEntityTagsTestMode InTagTestMode, TArray<FInstancedStruct> InTags)
		: TagTestMode(InTagTestMode)
		, Tags(InTags)
	{
	}
	EMassEntityTagsTestMode TagTestMode;
	TArray<FInstancedStruct> Tags;
};

/** Data required to be sent to Mass for processing this Test Request */
struct FMassEnvQueryResultData_MassEntityTags : public FMassEQSRequestData
{
	FMassEnvQueryResultData_MassEntityTags(TMap<FMassEntityHandle, bool>&& InResultMap)
		: ResultMap(MoveTemp(InResultMap))
	{
	}

	TMap<FMassEntityHandle, bool> ResultMap;
};