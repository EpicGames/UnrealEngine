// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassGameplayDebug : ModuleRules
	{
		public MassGameplayDebug(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"MassCore",
					"MassEntity",
					"NavigationSystem",
					"StateTreeModule",
					"MassActors",
					"MassCommon",
					"MassMovement",
					"MassSmartObjects",
					"MassSpawner",
					"MassSimulation",
					"MassRepresentation",
					"MassLOD",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
				PublicDependencyModuleNames.Add("MassEntityEditor");
			}

			SetupGameplayDebuggerSupport(Target);
		}
	}
}
