// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssignDebugVisProcessor.h"

#if WITH_EDITORONLY_DATA
#include UE_INLINE_GENERATED_CPP_BY_NAME(AssignDebugVisProcessor)

#include "MassEntityTypes.h"
#include "MassGameplayDebugTypes.h"

#if WITH_MASSGAMEPLAY_DEBUG
#include "Mass/EntityFragments.h"
#include "MassDebugVisualizationComponent.h"
#include "MassDebuggerSubsystem.h"
#include "MassExecutionContext.h"
#endif // WITH_MASSGAMEPLAY_DEBUG

//----------------------------------------------------------------------//
// UAssignDebugVisProcessor
//----------------------------------------------------------------------//
UAssignDebugVisProcessor::UAssignDebugVisProcessor()
#if WITH_MASSGAMEPLAY_DEBUG
	: EntityQuery(*this)
#endif // WITH_MASSGAMEPLAY_DEBUG
{
	bAutoRegisterWithProcessingPhases = false;

	// Always set the observed type and operations even if WITH_MASSGAMEPLAY_DEBUG is 0
	// since the processor may still be registered even if it won't do anything.
	ObservedTypes.Add(FSimDebugVisFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;

#if WITH_MASSGAMEPLAY_DEBUG
	bRequiresGameThreadExecution = true; // due to UMassDebuggerSubsystem
#endif // WITH_MASSGAMEPLAY_DEBUG
}

#if WITH_MASSGAMEPLAY_DEBUG
void UAssignDebugVisProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FSimDebugVisFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddSubsystemRequirement<UMassDebuggerSubsystem>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassDebuggerSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UAssignDebugVisProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(AssignDebugVisProcessor_Execute);

	// Visualization relies on an Actor/Component combo added to the world
	// During world tear down those can be removed,
	// and we don't want to try to access or recreate them.
	const UWorld* World = Context.GetWorld();
	if (World == nullptr
		|| World->bIsTearingDown)
	{
		return;
	}

	// @todo this code bit is temporary, so is the Visualizer->DirtyVisuals at the end of the function. Will be wrapped in
	// "executable task" once that's implemented.
	UMassDebuggerSubsystem& Debugger = Context.GetMutableSubsystemChecked<UMassDebuggerSubsystem>();
	UMassDebugVisualizationComponent* Visualizer = Debugger.GetVisualizationComponent();
	check(Visualizer);
	// note that this function will create the "visual components" only if they're missing or out of sync.
	Visualizer->ConditionallyConstructVisualComponent();

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassDebuggerSubsystem& Debugger = Context.GetMutableSubsystemChecked<UMassDebuggerSubsystem>();
		UMassDebugVisualizationComponent* Visualizer = Debugger.GetVisualizationComponent();
		check(Visualizer);

		const TArrayView<FSimDebugVisFragment> DebugVisList = Context.GetMutableFragmentView<FSimDebugVisFragment>();
		const TConstArrayView<FTransformFragment> LocationsList = Context.GetFragmentView<FTransformFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// VisualComp.VisualType needs to be assigned by now. Should be performed as part of spawning, copied from the AgentTemplate
			if (ensure(DebugVisList[EntityIt].VisualType != INDEX_NONE))
			{
				DebugVisList[EntityIt].InstanceIndex = Visualizer->AddDebugVisInstance(DebugVisList[EntityIt].VisualType, LocationsList[EntityIt]);
			}
		}
	});

	if (ensure(Visualizer))
	{
		Visualizer->DirtyVisuals();
	}
}
#endif // WITH_MASSGAMEPLAY_DEBUG

#endif // WITH_EDITORONLY_DATA