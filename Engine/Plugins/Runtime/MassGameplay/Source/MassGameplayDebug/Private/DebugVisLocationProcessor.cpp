// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugVisLocationProcessor.h"
#include "MassDebugger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugVisLocationProcessor)

#if WITH_MASSGAMEPLAY_DEBUG
#include "MassDebuggerSubsystem.h"
#include "MassDebugVisualizationComponent.h"
#include "MassCommonFragments.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassDebugDrawHelpers.h"

namespace UE::Mass::Debug
{
	bool bDebugDrawAllEntities = false;

	namespace
	{
		FAutoConsoleVariableRef AnonymousCVars[] = {
			{TEXT("mass.debug.DrawAllEntities"), bDebugDrawAllEntities
				, TEXT("When enabled will debug-draw debug shapes marking all entities that have a TransformFragment")
				, ECVF_Cheat},
		};
	}
}
#endif // WITH_MASSGAMEPLAY_DEBUG

//----------------------------------------------------------------------//
// UDebugVisLocationProcessor
//----------------------------------------------------------------------//
UDebugVisLocationProcessor::UDebugVisLocationProcessor()
#if WITH_MASSGAMEPLAY_DEBUG
	: EntityQuery(*this)
	, AllLocationEntitiesQuery(*this)
#endif // WITH_MASSGAMEPLAY_DEBUG
{
#if WITH_MASSGAMEPLAY_DEBUG
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SyncWorldToMass);
	bRequiresGameThreadExecution = true; // due to UMassDebuggerSubsystem access
	QueryBasedPruning = EMassQueryBasedPruning::Never;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
#else
	bAutoRegisterWithProcessingPhases = false;
#endif // WITH_MASSGAMEPLAY_DEBUG
}

#if WITH_MASSGAMEPLAY_DEBUG
void UDebugVisLocationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FSimDebugVisFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddTagRequirement<FMassDebuggableTag>(EMassFragmentPresence::All);
	EntityQuery.AddSubsystemRequirement<UMassDebuggerSubsystem>(EMassFragmentAccess::ReadWrite);

	AllLocationEntitiesQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
}

void UDebugVisLocationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
#if WITH_EDITORONLY_DATA
	QUICK_SCOPE_CYCLE_COUNTER(DebugVisLocationProcessor_Run);

	if (UE::Mass::Debug::bDebugDrawAllEntities == false)
	{
		// MassDebugVisualization relies on an Actor/Component combo added to the world
		// During world tear down those can be removed,
		// and we don't want to try to access or recreate them.
		const UWorld* World = Context.GetWorld();
		if (World != nullptr
			&& !World->bIsTearingDown)
		{
			EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
				{
					UMassDebuggerSubsystem& Debugger = Context.GetMutableSubsystemChecked<UMassDebuggerSubsystem>();
					UMassDebugVisualizationComponent* Visualizer = Debugger.GetVisualizationComponent();
					check(Visualizer);
					TArrayView<UInstancedStaticMeshComponent* const> VisualDataISMCs = Visualizer->GetVisualDataISMCs();
					if (VisualDataISMCs.Num() > 0)
					{
						const int32 NumEntities = Context.GetNumEntities();
						const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
						const TConstArrayView<FSimDebugVisFragment> DebugVisList = Context.GetFragmentView<FSimDebugVisFragment>();

						for (int32 i = 0; i < NumEntities; ++i)
						{
							const FSimDebugVisFragment& VisualComp = DebugVisList[i];

							// @todo: remove this code once the asset is exported with correct alignment SM_Mannequin.uasset
							FTransform SMTransform = LocationList[i].GetTransform();
							FQuat FromEngineToSM(FVector::UpVector, -HALF_PI);
							SMTransform.SetRotation(FromEngineToSM * SMTransform.GetRotation());

							VisualDataISMCs[VisualComp.VisualType]->UpdateInstanceTransform(VisualComp.InstanceIndex, SMTransform, true);
						}
					}
					else
					{
						UE_LOGF(LogMassDebug, Log, "UDebugVisLocationProcessor: Trying to update InstanceStaticMeshes while none created. Check your debug visualization setup");
					}
				});

			UMassDebuggerSubsystem* Debugger = UWorld::GetSubsystem<UMassDebuggerSubsystem>(EntityManager.GetWorld());
			if (ensure(Debugger))
			{
				Debugger->GetVisualizationComponent()->DirtyVisuals();
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
	if (UE::Mass::Debug::bDebugDrawAllEntities)
	{
		UWorld* World = UE::Mass::Debug::GetDebugDrawWorld(Context);

		if (World && World->GetLineBatcher(UWorld::ELineBatcherType::World))
		{
			UE::Mass::Debug::FLineBatcher LineBatcher = UE::Mass::Debug::FLineBatcher::MakeLineBatcher(World);

			AllLocationEntitiesQuery.ForEachEntityChunk(Context, [&LineBatcher](const FMassExecutionContext& Context)
				{
					const FColor ArchetypeColor = Context.DebugGetArchetypeColor();

					const FVector DebugDrawBoxExtent(20.);
					constexpr float ArrowLength = 30.f;

					const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();

					for (const FTransformFragment& LocationFragment : LocationList)
					{
						LineBatcher.DrawSolidBox(LocationFragment.GetTransform().GetLocation(), DebugDrawBoxExtent, ArchetypeColor);
						if (LocationFragment.GetTransform().GetRotation().IsIdentity() == false)
						{
							LineBatcher.DrawArrow(LocationFragment.GetTransform(), ArrowLength, ArchetypeColor);
						}
					}
				});
		}
	}
}
#endif // WITH_MASSGAMEPLAY_DEBUG

//----------------------------------------------------------------------//
//  UMassProcessor_UpdateDebugVis
//----------------------------------------------------------------------//
UMassProcessor_UpdateDebugVis::UMassProcessor_UpdateDebugVis()
#if WITH_MASSGAMEPLAY_DEBUG
	: EntityQuery(*this)
#endif // WITH_MASSGAMEPLAY_DEBUG
{
#if WITH_MASSGAMEPLAY_DEBUG
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::UpdateWorldFromMass);
	bRequiresGameThreadExecution = true; // due to UMassDebuggerSubsystem access
#else
	bAutoRegisterWithProcessingPhases = false;
#endif // WITH_MASSGAMEPLAY_DEBUG
}

#if WITH_MASSGAMEPLAY_DEBUG
void UMassProcessor_UpdateDebugVis::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) 
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FDataFragment_DebugVis>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassDebuggableTag>(EMassFragmentPresence::All);

	ProcessorRequirements.AddSubsystemRequirement<UMassDebuggerSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassProcessor_UpdateDebugVis::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UMassProcessor_UpdateDebugVis_Run);

	UMassDebuggerSubsystem& Debugger = Context.GetMutableSubsystemChecked<UMassDebuggerSubsystem>();
	Debugger.ResetDebugShapes();

	EntityQuery.ForEachEntityChunk(Context, [&Debugger](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FDataFragment_DebugVis> DebugVisList = Context.GetMutableFragmentView<FDataFragment_DebugVis>();
			const TArrayView<FAgentRadiusFragment> RadiiList = Context.GetMutableFragmentView<FAgentRadiusFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				Debugger.AddShape(DebugVisList[EntityIt].Shape, LocationList[EntityIt].GetTransform().GetLocation(), RadiiList[EntityIt].Radius);
			}
		});
}
#endif // WITH_MASSGAMEPLAY_DEBUG
