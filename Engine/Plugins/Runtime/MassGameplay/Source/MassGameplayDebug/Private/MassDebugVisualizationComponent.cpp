// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebugVisualizationComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebugVisualizationComponent)


#if WITH_EDITORONLY_DATA
#include "MassDebugVisualizer.h"

void UMassDebugVisualizationComponent::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject) == false && GetOuter())
	{
		ensureMsgf(GetOuter()->GetClass()->IsChildOf(AMassDebugVisualizer::StaticClass()), TEXT("UMassDebugVisualizationComponent should only be added to AMassDebugVisualizer-like instances"));
	}
}

void UMassDebugVisualizationComponent::DirtyVisuals()
{
	for (UInstancedStaticMeshComponent* ISM : VisualDataISMCs)
	{
		if (ensure(ISM))
		{
			ISM->MarkRenderStateDirty();
		}
	}
}

int32 UMassDebugVisualizationComponent::AddDebugVisInstance(const uint16 VisualType, const FTransformFragment& Transform)
{
	return VisualDataISMCs[VisualType]->AddInstance(Transform.GetTransform());
}

void UMassDebugVisualizationComponent::ConditionallyConstructVisualComponent()
{
	if (VisualDataISMCs.Num() == 0 || VisualDataISMCs.Num() != VisualDataTable.Num())
	{
		ConstructVisualComponent();
	}
}

void UMassDebugVisualizationComponent::ConstructVisualComponent()
{
	AActor* ActorOwner = GetOwner();
	check(ActorOwner);
	
	// add ISMComponents only for types not added yet
	for (int NewTypeIndex = VisualDataISMCs.Num(); NewTypeIndex < VisualDataTable.Num(); ++NewTypeIndex)
	{
		const FAgentDebugVisualization& VisualData = VisualDataTable[NewTypeIndex];
		UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(ActorOwner);
		ISMComponent->SetStaticMesh(VisualData.Mesh);
		if (VisualData.MaterialOverride != nullptr)
		{
			ISMComponent->SetMaterial(0, VisualData.MaterialOverride);
		}

		ISMComponent->SetCullDistances(VisualData.VisualNearCullDistance, VisualData.VisualFarCullDistance);
		ISMComponent->SetupAttachment(ActorOwner->GetRootComponent());
		ISMComponent->SetCanEverAffectNavigation(false);
		ISMComponent->bDisableCollision = true;
		ISMComponent->SetCastShadow(false);
		ISMComponent->RegisterComponent();

		VisualDataISMCs.Add(ISMComponent);
	}
}

uint16 UMassDebugVisualizationComponent::AddDebugVisType(const FAgentDebugVisualization& Data)
{
	const int32 Index = VisualDataTable.Add(Data);
	check(VisualDataTable.Num() <= (int32)MAX_uint16);
	return (uint16)Index;
}

void UMassDebugVisualizationComponent::Clear()
{
	for (UInstancedStaticMeshComponent* ISM : VisualDataISMCs)
	{
		if (ensure(ISM))
		{
			ISM->ClearInstances();
			ISM->UnregisterComponent();
		}
	}
	VisualDataISMCs.Reset();
}
#endif // WITH_EDITORONLY_DATA
