// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebugVisualizationTrait.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebugVisualizationTrait)

#if WITH_MASSGAMEPLAY_DEBUG
#include "MassDebuggerSubsystem.h"
#include "MassDebugVisualizationComponent.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "Engine/World.h"
#endif // WITH_MASSGAMEPLAY_DEBUG

void UMassDebugVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
#if WITH_MASSGAMEPLAY_DEBUG
#if WITH_EDITORONLY_DATA
	const UStaticMesh* const DebugMesh = DebugShape.Mesh;
	if (DebugMesh || BuildContext.IsInspectingData())
	{
		FSimDebugVisFragment& DebugVisFragment = BuildContext.AddFragment_GetRef<FSimDebugVisFragment>();
		if (!BuildContext.IsInspectingData())
		{
			UMassDebuggerSubsystem* Debugger = World.GetSubsystem<UMassDebuggerSubsystem>();
			if (ensure(Debugger))
			{
				UMassDebugVisualizationComponent* DebugVisComponent = Debugger->GetVisualizationComponent();
				if (ensure(DebugVisComponent))
				{
					DebugVisFragment.VisualType = DebugVisComponent->AddDebugVisType(DebugShape);
				}
				// @todo this path requires a fragment destructor that will remove the mesh from the debugger.
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	// add fragments needed whenever we have debugging capabilities
	BuildContext.AddTag<FMassDebuggableTag>();

#if WITH_EDITORONLY_DATA
	BuildContext.AddFragment_GetRef<FDataFragment_DebugVis>().Shape = DebugShape.WireShape;
#else
	// DebugShape unavailable, will used default instead
	BuildContext.AddFragment<FDataFragment_DebugVis>();
#endif // WITH_EDITORONLY_DATA

	BuildContext.AddFragment<FAgentRadiusFragment>();
	BuildContext.AddFragment<FTransformFragment>();
#endif // WITH_MASSGAMEPLAY_DEBUG
}
