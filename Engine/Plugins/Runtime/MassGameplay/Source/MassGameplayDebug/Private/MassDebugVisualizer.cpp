// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebugVisualizer.h"

#if WITH_EDITORONLY_DATA
#include "MassDebugVisualizationComponent.h"
#endif // WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebugVisualizer)

AMassDebugVisualizer::AMassDebugVisualizer()
{
#if WITH_EDITORONLY_DATA
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	DebugVisComponent = CreateDefaultSubobject<UMassDebugVisualizationComponent>(TEXT("VisualizerComponent"));
	bEditable = false;
	bListedInSceneOutliner = false;
#endif
}

