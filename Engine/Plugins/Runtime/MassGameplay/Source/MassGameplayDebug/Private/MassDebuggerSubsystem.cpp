// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebuggerSubsystem.h"
#include "EngineUtils.h"
#include "MassDebugger.h"
#include "MassDebugVisualizationComponent.h"
#include "MassDebugVisualizer.h"
#include "MassEntityManager.h"
#include "MassSimulationSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebuggerSubsystem)

#if WITH_MASSGAMEPLAY_DEBUG
void UMassDebuggerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency(UMassSimulationSubsystem::StaticClass());
	Super::Initialize(Collection);

	OnEntitySelectedHandle = FMassDebugger::OnEntitySelectedDelegate.AddUObject(this, &UMassDebuggerSubsystem::OnEntitySelected);

	OverrideSubsystemTraits<UMassDebuggerSubsystem>(Collection);
}

void UMassDebuggerSubsystem::Deinitialize()
{
	FMassDebugger::OnEntitySelectedDelegate.Remove(OnEntitySelectedHandle);
	Super::Deinitialize();
}

void UMassDebuggerSubsystem::OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	if (EntityManager.GetWorld() == GetWorld())
	{
		SetSelectedEntity(EntityHandle);
	}
}

void UMassDebuggerSubsystem::ForEachShape(EMassEntityDebugShape Shape, const TFunctionRef<void(const FShapeDesc&)> Function) const
{
	UE_MT_SCOPED_READ_ACCESS(MTDetector);

	for (const FShapeDesc& Desc : Shapes[static_cast<uint8>(Shape)])
	{
		Function(Desc);
	}
}

void UMassDebuggerSubsystem::ResetDebugShapes()
{
	UE_MT_SCOPED_WRITE_ACCESS(MTDetector);

	// get ready to receive new debug info
	for (TArray<FShapeDesc>& Array : Shapes)
	{
		Array.Reset();
	}
}

void UMassDebuggerSubsystem::SetSelectedEntity(const FMassEntityHandle InSelectedEntity)
{
	UE_MT_SCOPED_WRITE_ACCESS(MTDetector);

	SelectedEntity = InSelectedEntity;
	SelectedEntityDetails.Empty();
}

void UMassDebuggerSubsystem::AppendSelectedEntityInfo(const FString& Info)
{
	UE_MT_SCOPED_WRITE_ACCESS(MTDetector);

	if (UpdateFrameNumber != GFrameNumber)
	{
		SelectedEntityDetails.Empty();
		UpdateFrameNumber = GFrameNumber;
	}

	SelectedEntityDetails += Info;
}

UMassDebugVisualizationComponent* UMassDebuggerSubsystem::GetVisualizationComponent()
{
#if WITH_EDITORONLY_DATA
	if (VisualizationComponent == nullptr)
	{
		if (!ensureMsgf(DebugVisualizer == nullptr, TEXT("If we do have a DebugVisualizer but don't have VisualizationComponent then something's wrong")))
		{
			VisualizationComponent = &DebugVisualizer->GetDebugVisComponent();
		}
		else
		{
			UWorld* World = GetWorld();
			if (World != nullptr
				&& !World->bIsTearingDown)
			{
				const AMassDebugVisualizer& VisualizerActor = GetOrSpawnDebugVisualizer(*World);
				VisualizationComponent = &VisualizerActor.GetDebugVisComponent();
			}
		}
	}
	ensureMsgf(VisualizationComponent, TEXT("In editor builds we always expect to have a visualizer component available"));
#endif // WITH_EDITORONLY_DATA
	return VisualizationComponent;
}

#if WITH_EDITORONLY_DATA
AMassDebugVisualizer& UMassDebuggerSubsystem::GetOrSpawnDebugVisualizer(UWorld& InWorld)
{
	if (DebugVisualizer)
	{
		return *DebugVisualizer;
	}

	// see if there is one already and we've missed it somehow
	for (const TActorIterator<AMassDebugVisualizer> It(&InWorld); It;)
	{
		DebugVisualizer = *It;
		return *DebugVisualizer;
	}

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	// The helper actor is created on demand and only once per world so we can allow it to spawn during construction script.
	SpawnInfo.bAllowDuringConstructionScript = true;
	DebugVisualizer = InWorld.SpawnActor<AMassDebugVisualizer>(SpawnInfo);
	check(DebugVisualizer);
	VisualizationComponent = &DebugVisualizer->GetDebugVisComponent();

	return *DebugVisualizer;
}
#endif // WITH_EDITORONLY_DATA
#endif //WITH_MASSGAMEPLAY_DEBUG