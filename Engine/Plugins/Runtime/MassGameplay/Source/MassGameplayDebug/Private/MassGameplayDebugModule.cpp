// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IMassGameplayDebugModule.h"
#include "MassCommonTypes.h"
#include "MassGameplayDebugTypes.h"

DEFINE_LOG_CATEGORY(LogMassDebug);

class FMassGameplayDebug : public IMassGameplayDebugModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassGameplayDebug, MassGameplayDebug)

void FMassGameplayDebug::StartupModule()
{
}

void FMassGameplayDebug::ShutdownModule()
{
}



