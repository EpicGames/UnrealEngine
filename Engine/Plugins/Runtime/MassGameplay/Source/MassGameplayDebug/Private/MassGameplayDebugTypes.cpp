// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassGameplayDebugTypes.h"
#include "MassCommonTypes.h"
#include "MassCommonFragments.h"
#include "MassEntityManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassGameplayDebugTypes)
#if WITH_MASSGAMEPLAY_DEBUG
#include "MassDebugger.h"


namespace UE::Mass::Debug
{

void GetDebugEntitiesAndLocations(const FMassEntityManager& EntityManager, TArray<FMassEntityHandle>& OutEntities, TArray<FVector>& OutLocations)
{
	int32 DebugEntityBegin, DebugEntityEnd;
	if (GetDebugEntitiesRange(DebugEntityBegin, DebugEntityEnd) == false)
	{
		return;
	}

	OutEntities.Reserve(DebugEntityEnd - DebugEntityBegin);
	OutLocations.Reserve(DebugEntityEnd - DebugEntityBegin);

	for (int32 i = DebugEntityBegin; i <= DebugEntityEnd; ++i)
	{
		const FMassEntityHandle EntityHandle = ConvertEntityIndexToHandle(EntityManager, i);
		if (EntityHandle.IsSet())
		{
			if (const FTransformFragment* TransformFragment = EntityManager.GetFragmentDataPtr<FTransformFragment>(EntityHandle))
			{
				OutEntities.Add(EntityHandle);
				OutLocations.Add(TransformFragment->GetTransform().GetLocation());
			}
		}
	}
}

FMassEntityHandle ConvertEntityIndexToHandle(const FMassEntityManager& EntityManager, const int32 EntityIndex)
{
	return EntityManager.CreateEntityIndexHandle(EntityIndex);
}

} // namespace UE::Mass::Debug
#endif // WITH_MASSGAMEPLAY_DEBUG
