// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if WITH_EDITORONLY_DATA
#include "MassCommonTypes.h"
#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "AssignDebugVisProcessor.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API


class UMassDebugVisualizationComponent;
struct FSimDebugVisFragment;

UCLASS(MinimalAPI)
class UAssignDebugVisProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UE_API UAssignDebugVisProcessor();

#if WITH_MASSGAMEPLAY_DEBUG
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

protected:
	FMassEntityQuery EntityQuery;
#endif // WITH_MASSGAMEPLAY_DEBUG
};

#undef UE_API
#endif // #if WITH_EDITORONLY_DATA
