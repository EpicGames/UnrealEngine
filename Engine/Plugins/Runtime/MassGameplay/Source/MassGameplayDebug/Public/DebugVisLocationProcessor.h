// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "DebugVisLocationProcessor.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API

class UMassDebugVisualizationComponent;
struct FSimDebugVisFragment;

UCLASS(MinimalAPI)
class UDebugVisLocationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UDebugVisLocationProcessor();

#if WITH_MASSGAMEPLAY_DEBUG
protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
	FMassEntityQuery AllLocationEntitiesQuery;
#endif // WITH_MASSGAMEPLAY_DEBUG
};

/**
 * Processor gathering debug data from entities to push debug shapes to the MassDebugger subsystem
 */
UCLASS(MinimalAPI)
class UMassProcessor_UpdateDebugVis : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassProcessor_UpdateDebugVis();

#if WITH_MASSGAMEPLAY_DEBUG
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

protected:
	FMassEntityQuery EntityQuery;
#endif // WITH_MASSGAMEPLAY_DEBUG
};

#undef UE_API
