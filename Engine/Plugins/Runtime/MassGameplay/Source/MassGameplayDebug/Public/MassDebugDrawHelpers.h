// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "Components/LineBatchComponent.h"
#include "Engine/World.h"


namespace UE::Mass::Debug
{
	struct FLineBatcher
	{
		static FLineBatcher MakeLineBatcher(const UWorld* InWorld, bool bPersistentLines = false, float LifeTime = -1.f)
		{
			return FLineBatcher(InWorld 
					? (( bPersistentLines || (LifeTime > 0.f)) 
						? InWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent)
						: InWorld->GetLineBatcher(UWorld::ELineBatcherType::World))
					: nullptr
				, LifeTime);
		}

		FLineBatcher(TNotNull<ULineBatchComponent*> InLineBatcherInstance, float InLifeTime = -1.f)
			: LineBatcherInstance(InLineBatcherInstance), LifeTime(InLifeTime)
		{	
		}

		inline void DrawSolidBox(const FVector& Center, const FVector& Extent, FColor const& Color) const
		{
			const FBox Box = FBox::BuildAABB(Center, Extent);
			LineBatcherInstance->DrawSolidBox(Box, FTransform::Identity, Color, /*DepthPriority=*/0, LifeTime);
		}

		inline void DrawWireBox(const FVector& Center, const FVector& Extent, FColor const& Color) const
		{
			LineBatcherInstance->DrawBox(Center, Extent, Color, LifeTime, /*DepthPriority=*/0, /*Thickness=*/0.f);
		}

		inline void DrawSphere(const FVector& Center, const float Radius, FLinearColor const& Color) const
		{
			LineBatcherInstance->DrawSphere(Center, Radius, /*Segments=*/8, Color, LifeTime, /*DepthPriority=*/0, /*Thickness=*/0.f);
		}

		inline void DrawArrow(const FTransform& Transform, const float Length, FColor const& Color) const
		{
			LineBatcherInstance->DrawDirectionalArrow(Transform.ToMatrixNoScale(), Color, Length, Length/5, /*DepthPriority=*/0);
		}

		TNotNull<ULineBatchComponent* const> LineBatcherInstance;
		float LifeTime = -1.f;
	};
}