// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassGameplayDebugTypes.h"
#include "Components/ActorComponent.h"
#include "Mass/EntityFragments.h"
#include "MassDebugVisualizationComponent.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API


class UInstancedStaticMeshComponent;

/** meant to be created procedurally and owned by an AMassDebugVisualizer instance. Will ensure if placed on a different type of actor */
UCLASS(MinimalAPI)
class UMassDebugVisualizationComponent : public UActorComponent
{
	GENERATED_BODY()
public:
#if WITH_EDITORONLY_DATA
	/**  will create Owner's "visual components" only it they're missing or out of sync with VisualDataTable */
	UE_API void ConditionallyConstructVisualComponent();
	UE_API void DirtyVisuals();
	UE_API int32 AddDebugVisInstance(const uint16 VisualType, const FTransformFragment& Transform = FTransformFragment(FTransform::Identity));
	/** returns index to the newly created VisualDataTable entry */
	UE_API uint16 AddDebugVisType(const FAgentDebugVisualization& Data);
	TArrayView<UInstancedStaticMeshComponent* const> GetVisualDataISMCs() { return MakeArrayView(VisualDataISMCs); }

	UE_API void Clear();
protected:
	UE_API virtual void PostInitProperties() override;
	UE_API void ConstructVisualComponent();

protected:

	UPROPERTY(Transient)
	TArray<FAgentDebugVisualization> VisualDataTable;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> VisualDataISMCs;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
