// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassGameplayDebugTypes.h"
#include "MassDebugVisualizationTrait.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API


UCLASS(MinimalAPI, meta = (DisplayName = "Debug Visualization"))
class UMassDebugVisualizationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Mass|Spawn")
	FAgentDebugVisualization DebugShape;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
