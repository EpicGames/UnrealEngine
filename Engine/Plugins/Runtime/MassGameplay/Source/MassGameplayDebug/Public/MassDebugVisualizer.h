// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "MassDebugVisualizer.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API


UCLASS(MinimalAPI, NotPlaceable, Transient)
class AMassDebugVisualizer : public AActor
{
	GENERATED_BODY()
public:
	UE_API AMassDebugVisualizer();

#if WITH_EDITORONLY_DATA
	/** If this function is callable we guarantee the debug vis component to exist*/
	class UMassDebugVisualizationComponent& GetDebugVisComponent() const { return *DebugVisComponent; }

protected:
	UPROPERTY()
	TObjectPtr<class UMassDebugVisualizationComponent> DebugVisComponent;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
