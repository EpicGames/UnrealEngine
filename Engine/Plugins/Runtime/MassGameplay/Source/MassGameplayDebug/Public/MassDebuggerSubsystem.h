// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassGameplayDebugTypes.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "MassDebuggerSubsystem.generated.h"

#define UE_API MASSGAMEPLAYDEBUG_API


class UMassDebugVisualizationComponent;
class AMassDebugVisualizer;

UCLASS(MinimalAPI)
class UMassDebuggerSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:

#if WITH_MASSGAMEPLAY_DEBUG

	struct FShapeDesc
	{
		FVector Location = {}; // no init on purpose, value will come from constructor
		float Size = {};
		FShapeDesc(const FVector& InLocation, const float InSize) : Location(InLocation), Size(InSize)
		{
		}
	};

	// Methods to optimize the collection of data to only when category is enabled
	bool IsCollectingData() const
	{
		return bCollectingData;
	}

	void SetCollectingData()
	{
		bCollectingData = true;
	}

	void DataCollected()
	{
		bCollectingData = false;
	}

	void AddShape(EMassEntityDebugShape Shape, const FVector& Location, const float Size)
	{
		UE_MT_SCOPED_WRITE_ACCESS(MTDetector);
		Shapes[static_cast<uint8>(Shape)].Add(FShapeDesc(Location, Size));
	}

	UE_API void ForEachShape(EMassEntityDebugShape Shape, const TFunctionRef<void(const FShapeDesc&)> Function) const;

	UE_DEPRECATED(5.8, "Use ForEachShape instead.")
	const TArray<FShapeDesc>* GetShapes() const
	{
		return Shapes;
	}

	UE_API void ResetDebugShapes();

	UE_API void SetSelectedEntity(const FMassEntityHandle InSelectedEntity);
	FMassEntityHandle GetSelectedEntity() const
	{
		UE_MT_SCOPED_READ_ACCESS(MTDetector);
		return SelectedEntity;
	}

	UE_API void AppendSelectedEntityInfo(const FString& Info);
	const FString& GetSelectedEntityInfo() const
	{
		UE_MT_SCOPED_READ_ACCESS(MTDetector);
		return SelectedEntityDetails;
	}

	/** Fetches the UMassDebugVisualizationComponent owned by lazily created DebugVisualizer */
	UE_API UMassDebugVisualizationComponent* GetVisualizationComponent();

#if WITH_EDITORONLY_DATA
	UE_API AMassDebugVisualizer& GetOrSpawnDebugVisualizer(UWorld& InWorld);
#endif

protected:
	//~ USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ USubsystem END

	UE_API void OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);

	TArray<FShapeDesc> Shapes[static_cast<uint8>(EMassEntityDebugShape::MAX)];

	FMassEntityHandle SelectedEntity;
	FString SelectedEntityDetails;

	uint64 UpdateFrameNumber = INDEX_NONE;

	FDelegateHandle OnEntitySelectedHandle;

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(MTDetector);

	bool bCollectingData = false;
#endif // WITH_MASSGAMEPLAY_DEBUG

	UPROPERTY(Transient)
	TObjectPtr<UMassDebugVisualizationComponent> VisualizationComponent;

	UPROPERTY(Transient)
	TObjectPtr<AMassDebugVisualizer> DebugVisualizer;
};

template<>
struct TMassExternalSubsystemTraits<UMassDebuggerSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

#undef UE_API
