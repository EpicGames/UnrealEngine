// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/ObjectMacros.h"
#include "Logging/LogMacros.h"
#include "Engine/DataTable.h"
#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "MassCommonTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "MassSpawnerTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassGameplayDebugTypes.generated.h"


DECLARE_LOG_CATEGORY_EXTERN(LogMassDebug, Warning, All);

#if WITH_EDITORONLY_DATA
class UBillboardComponent;
#endif // WITH_EDITORONLY_DATA
class UStaticMesh;
class UMaterialInterface;

#if WITH_MASSGAMEPLAY_DEBUG
namespace UE::Mass::Debug
{
	/**
	 * Fetches entity handles and their locations for entities indicated by index range as set by
	 * mass.debug.SetDebugEntityRange or mass.debug.DebugEntity console commands.
	 */
	MASSGAMEPLAYDEBUG_API extern void GetDebugEntitiesAndLocations(const FMassEntityManager& EntityManager, TArray<FMassEntityHandle>& OutEntities, TArray<FVector>& OutLocations);
	MASSGAMEPLAYDEBUG_API extern FMassEntityHandle ConvertEntityIndexToHandle(const FMassEntityManager& EntityManager, const int32 EntityIndex);
} // namespace UE::Mass::Debug
#endif // WITH_MASSGAMEPLAY_DEBUG

USTRUCT()
struct FSimDebugDataRow : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Debug)
	TObjectPtr<UStaticMesh> Mesh = nullptr;

	UPROPERTY(EditAnywhere, Category = Debug)
	TObjectPtr<UMaterialInterface> MaterialOverride = nullptr;

	UPROPERTY(EditAnywhere, Category = Debug)
	float Scale = 1.f;
};

USTRUCT()
struct FSimDebugVisFragment : public FMassFragment
{
	GENERATED_BODY()
	int32 InstanceIndex = INDEX_NONE;
	int16 VisualType = INDEX_NONE;
};

UENUM()
enum class EMassEntityDebugShape : uint8
{
	Box,
	Cone,
	Cylinder,
	Capsule,
	MAX
};

USTRUCT()
struct FDataFragment_DebugVis : public FMassFragment
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category = Debug)
	EMassEntityDebugShape Shape = EMassEntityDebugShape::Box;
};

USTRUCT()
struct FMassDebuggableTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FAgentDebugVisualization : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Mass|Debug")
	TObjectPtr<UStaticMesh> Mesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Mass|Debug")
	TObjectPtr<UMaterialInterface> MaterialOverride = nullptr;

	/** Near cull distance to override default value for that agent type */
	UPROPERTY(EditAnywhere, Category = "Mass|Debug")
	uint32 VisualNearCullDistance = 5000;

	/** Far cull distance to override default value for that agent type */
	UPROPERTY(EditAnywhere, Category = "Mass|Debug")
	uint32 VisualFarCullDistance = 7500;

	/** If Mesh is not set this WireShape will be used for debug drawing via GameplayDebugger */
	UPROPERTY(EditAnywhere, Category = "Mass|Debug")
	EMassEntityDebugShape WireShape = EMassEntityDebugShape::Box;
};
