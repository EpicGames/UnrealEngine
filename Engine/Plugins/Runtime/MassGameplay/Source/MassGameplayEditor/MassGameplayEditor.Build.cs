// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassGameplayEditor : ModuleRules
	{
		public MassGameplayEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"AssetTools",
				"UnrealEd",
				"Slate",
				"SlateCore",
				
				"PropertyEditor",
				"MassSpawner",
				"MassCore",
				"MassEntity",
				"MassEntityEditor",
				"MassActors",
				"DetailCustomizations",
				"ComponentVisualizers",
				"Projects",
				"EditorSubsystem",
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RenderCore",
				"GraphEditor",
				"KismetWidgets",
				"PropertyEditor",
				"AIGraph",
				"ToolMenus",
				"DataValidation",
			}
			);
		}

	}
}
