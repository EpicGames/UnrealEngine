// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassActorEditorSubsystem.h"
#include "Subsystems/SubsystemCollection.h"
#include "MassActorSubsystem.h"
#include "MassEntityManager.h"
#include "MassEntityEditorSubsystem.h"


//----------------------------------------------------------------------//
//  UMassActorEditorSubsystem
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassActorEditorSubsystem)
void UMassActorEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UMassEntityEditorSubsystem* MassEditorEditorSubsystem = UMassEntityEditorSubsystem::Get();
	check(MassEditorEditorSubsystem);
	TSharedRef<FMassEntityManager> MassEntityManager = MassEditorEditorSubsystem->GetMutableEntityManager();
	ActorManager = MakeShareable(new FMassActorManager(MassEntityManager));

	Super::Initialize(Collection);
}
