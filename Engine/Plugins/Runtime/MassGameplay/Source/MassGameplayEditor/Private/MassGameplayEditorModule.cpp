// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassGameplayEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"


#define LOCTEXT_NAMESPACE "MassGameplayEditor"

IMPLEMENT_MODULE(FMassGameplayEditorModule, MassGameplayEditor)

void FMassGameplayEditorModule::StartupModule()
{
	// Register the details customizers
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	RegisterSectionMappings();
	
	PropertyModule.NotifyCustomizationModuleChanged();

	// Ensure dependencies are loaded. We rely on subsystems within them.
	FModuleManager& ModuleManager = FModuleManager::Get();
	ModuleManager.LoadModuleChecked("MassEntityEditor");
}

void FMassGameplayEditorModule::ShutdownModule()
{
}

void FMassGameplayEditorModule::RegisterSectionMappings()
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("MassSpawner", "Mass", LOCTEXT("Mass", "Mass"));
		Section->AddCategory("Mass");
		Section->AddCategory("Debug");
	}

	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("MassAgentComponent", "Mass", LOCTEXT("Mass", "Mass"));
		Section->AddCategory("Mass");
	}
}
#undef LOCTEXT_NAMESPACE
