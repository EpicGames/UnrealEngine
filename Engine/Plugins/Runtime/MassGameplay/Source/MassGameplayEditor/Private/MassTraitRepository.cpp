// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "MassTraitRepository.h"
#include "MassEntityTraitBase.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectIterator.h"
#include "Editor.h"
#include "MassEntityTemplateRegistry.h"
#include "Misc/UObjectToken.h"
#include "Framework/Docking/TabManager.h"
#include "Logging/MessageLog.h"
#include "MassAssortedFragmentsTrait.h"
#include "DataValidationFixers.h"
#include "MassEntityConfigAsset.h"
#include "MassEntityEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassTraitRepository)

#define LOCTEXT_NAMESPACE "Mass"


namespace UE::Mass
{
	namespace Editor
	{
		TConstArrayView<FName> GetTraitsNameAddingElements(const FName ElementName)
		{
			if (UMassTraitRepository* TraitRepo = GEditor->GetEditorSubsystem<UMassTraitRepository>())
			{
				return TraitRepo->GetTraitsNameAddingElements(ElementName);
			}
			return TConstArrayView<FName>();
		}
	}

	namespace Private
	{
		/** a helper function that wraps up code for fixing the WeakConfig by adding a trait of class WeakTraitClass */
		FFixResult AddTraitToConfigFix(const TWeakObjectPtr<UMassEntityConfigAsset>& WeakConfig, const TWeakObjectPtr<UClass>& WeakTraitClass)
		{
			if (UMassEntityConfigAsset* ConfigAsset = WeakConfig.Get())
			{
				if (TSubclassOf<UMassEntityTraitBase> TraitClass = WeakTraitClass.Get())
				{
					if (ConfigAsset->AddTrait(TraitClass) != nullptr)
					{
						return FFixResult::Success();
					}
					return FFixResult::Failure(LOCTEXT("FailedToCreateTrait", "Failed to create an instance of the trait."));
				}
				return FFixResult::Failure(LOCTEXT("TraitClassNoLongerAvailable", "Trait class no longer available."));
			}
			return FFixResult::Failure(LOCTEXT("ConfigAssetNoLongerAvailable", "Config asset no longer available."));
		}
	}
};

//-----------------------------------------------------------------------------
// FMassTraitInspectionContext
//-----------------------------------------------------------------------------
FMassTraitInspectionContext::FMassTraitInspectionContext()
	: EntityTemplate()
	, BuildContext(EntityTemplate)
{
	BuildContext.EnableDataInvestigationMode();
}

FMassTraitInspectionContext::FInvestigationContext::FInvestigationContext(FMassEntityTemplateData& InTemplate)
	: Super(InTemplate)
{
}

void FMassTraitInspectionContext::FInvestigationContext::SetTrait(const UMassEntityTraitBase& Trait)
{
	SetTraitBeingProcessed(&Trait);
}

//-----------------------------------------------------------------------------
// UMassTraitRepository
//-----------------------------------------------------------------------------
TWeakObjectPtr<UWorld> UMassTraitRepository::GlobalInvestigationWorld;

void UMassTraitRepository::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

#if WITH_MASSENTITY_DEBUG
	OnNewTraitTypeHandle = UMassEntityTraitBase::GetOnNewTraitTypeEvent().AddUObject(this, &UMassTraitRepository::OnNewTraitType);
	FMassDebugger::OnDebugEvent.AddUObject(this, &UMassTraitRepository::OnDebugEvent);
#endif // WITH_MASSENTITY_DEBUG
}

// @todo we have an opportunity here to make it really flexible. Every message name could be associated with a 
// dedicated TFunction (via some map) that would handle the given message type. This way users could extend or override
// the way certain events are handled. 
#if WITH_MASSENTITY_DEBUG
void UMassTraitRepository::OnDebugEvent(const FName EventName, FConstStructView Payload, EMassDebugMessageSeverity SeverityOverride)
{
	static const FName MissingTraitMessageName = FMassMissingTraitMessage::StaticStruct()->GetFName();
	static const FName DuplicateElementsMessageName = FMassDuplicateElementsMessage::StaticStruct()->GetFName();

#define OVERRIDABLE_SEVERITY(DefaultSeverity) UE::Mass::Debug::MassSeverityToMessageSeverity(DefaultSeverity, SeverityOverride)

	if (EventName == MissingTraitMessageName)
	{
		if (const FMassMissingTraitMessage* MissingTraitMessage = Payload.GetPtr<const FMassMissingTraitMessage>())
		{
			FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);

			TArray<TSharedRef<FTokenizedMessage>> Messages;

			const FName MissingElement = GetFNameSafe(MissingTraitMessage->MissingType);
			const FName TraitClassName = MissingTraitMessage->RequestingTrait ? GetFNameSafe(MissingTraitMessage->RequestingTrait->GetClass()) : FName();

			TSharedRef<FTokenizedMessage> IntroMessage = Messages.Add_GetRef(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Error)))
				->AddToken(FAssetNameToken::Create(GetPathNameSafe(MissingTraitMessage->RequestingTrait)
					, FText::FormatOrdered(LOCTEXT("MissingElementSuggestionHeader"
						, "Trait {0}")
						, FText::FromName(TraitClassName))
				));

			// if the missing elements has been added but removed by some trait that's all we need to tell the user:
			if (MissingTraitMessage->RemovedByTrait)
			{
				IntroMessage->AddText(FText::FormatOrdered(LOCTEXT("MissingElementSuggestionRemoved"
					, "has unsatisfied dependency of {0}. The type has been explicitly removed by {1}.")
					, FText::FromName(MissingElement)
					, FText::FromName(MissingTraitMessage->RemovedByTrait->GetFName())));
			}
			else
			{
				UMassTraitRepository* TraitRepo = GEditor->GetEditorSubsystem<UMassTraitRepository>();
				TConstArrayView<FName> SuggestedTraitNames = TraitRepo
					? TraitRepo->GetTraitsNameAddingElements(MissingElement)
					: TConstArrayView<FName>();

				if (MissingTraitMessage->RequestingTrait)
				{
					IntroMessage->AddText(
						FText::FormatOrdered(LOCTEXT("ExecutionFlags", "(ExecutionFlags: {0}) "),
						UEnum::GetDisplayValueAsText(MissingTraitMessage->RequestingTrait->ValidTargetConfig))
					);
				}
				
				if (SuggestedTraitNames.Num())
				{
					// The FixController will coordinate IFixer instances and the relevant FFixToken to
					// ensure that only one of them can be applied. Once any of the fixes is applied
					// the rest will become inactive (the FFixToken tokens will become grayed out and non-clickable).
					// @todo at the moment FMutuallyExclusiveFixSet doesn't care whether fixing was successful. 
					//		Should be relatively easy to address but needs to be coordinated with the author
					TSharedRef<UE::DataValidation::FMutuallyExclusiveFixSet> FixController = MakeShareable(new UE::DataValidation::FMutuallyExclusiveFixSet());

					IntroMessage->AddText(FText::FormatOrdered(LOCTEXT("MissingElementSuggestionOptions"
						, "has unsatisfied dependency of {0}. The following actions can address it:")
						, FText::FromName(MissingElement)));

					for (const FName& SuggestedTraitName : SuggestedTraitNames)
					{
						bool bFixable = false;
						if (MissingTraitMessage->RequestingTrait)
						{
							TWeakObjectPtr<UClass> WeakTraitClass = GetTraitClass(SuggestedTraitName);
							UMassEntityConfigAsset* EntityConfigAsset = Cast<UMassEntityConfigAsset>(MissingTraitMessage->RequestingTrait->GetOuter());

							if (EntityConfigAsset && WeakTraitClass.IsValid())
							{
								bFixable = true;

								TWeakObjectPtr<UMassEntityConfigAsset> WeakConfig = EntityConfigAsset;
							
								// capturing FixController to make sure it exists as long as the fixes are alive. The lambda will
								// be destroyed once the Fixer tokens get destroyed, for example during MessageLog page clearing.
								TFunction<FFixResult()> ApplyFix = [WeakConfig, WeakTraitClass, _ = FixController]()
								{
									return UE::Mass::Private::AddTraitToConfigFix(WeakConfig, WeakTraitClass);
								};

								const TSharedRef<UE::DataValidation::IFixer> Fixer = UE::DataValidation::MakeFix(MoveTemp(ApplyFix));

								FixController->Add(FText::FormatOrdered(LOCTEXT("AddMissingTrait", "Add {0} trait to {1} entity config")
											, FText::FromName(SuggestedTraitName)
											, FText::FromName(EntityConfigAsset->GetFName()))
										, Fixer);
							}
						}
					
						if (bFixable == false)
						{
							// unfixable (since we're unable to determine the UMassEntityConfigAsset outer), so just report
							Messages.Add_GetRef(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Info)))
								->AddText(FText::FormatOrdered(LOCTEXT("MissingElementSuggestionUnfixable", "\t{0}")
									, FText::FromName(SuggestedTraitName)));
						}
					}

					// for every IFixer instance we created in the loop above the call bellow will create a FFixToken
					// related to that specific "fix". We attach all the tokens to the initial "here are your options" message
					FixController->CreateTokens([IntroMessage](TSharedRef<FFixToken> FixToken)
						{
							IntroMessage->AddToken(FixToken);
						}
					);
				}
				else
				{
					IntroMessage->AddText(FText::FormatOrdered(LOCTEXT("MissingElementSuggestionNoOptions"
						, "has unsatisfied dependency of {0}. There are no registered Traits that provide the type. Try using {1}.")
						, FText::FromName(MissingElement)
						, FText::FromName(UMassAssortedFragmentsTrait::StaticClass()->GetFName())));
				}
			}

			MessageLog.AddMessages(Messages);
		}
	}
	else if (EventName == UE::Mass::Debug::TraitFailedValidation)
	{
		if (const FMassGenericDebugEvent* GenericEvent = Payload.GetPtr<const FMassGenericDebugEvent>())
		{
			const UMassEntityTraitBase* Trait = Cast<UMassEntityTraitBase>(GenericEvent->Context);
			FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);
			MessageLog.AddMessage(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Error)))
				->AddToken(FUObjectToken::Create(Trait))
				->AddToken(FTextToken::Create(LOCTEXT("TraitFailedValidation", "trait-specific validation failed")));
		}
	}
	else if (EventName == UE::Mass::Debug::TraitIgnored)
	{
		if (const FMassGenericDebugEvent* GenericEvent = Payload.GetPtr<const FMassGenericDebugEvent>())
		{
			const UMassEntityTraitBase* Trait = Cast<UMassEntityTraitBase>(GenericEvent->Context);
			FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);
			MessageLog.AddMessage(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Warning)))
				->AddToken(FUObjectToken::Create(Trait))
				->AddToken(FTextToken::Create(LOCTEXT("TraitIgnoredTrait", "trait was ignored. Check if it's not a duplicate.")));
		}
	}
	else if (EventName == UE::Mass::Debug::TraitOverridden)
	{
		if (const FMassGenericDebugEvent* GenericEvent = Payload.GetPtr<const FMassGenericDebugEvent>())
		{
			const UMassEntityTraitBase* Trait = Cast<UMassEntityTraitBase>(GenericEvent->Context);
			FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);
			MessageLog.AddMessage(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Info)))
				->AddToken(FUObjectToken::Create(Trait))
				->AddToken(FTextToken::Create(LOCTEXT("TraitOverridden", "trait was overridden by another instance. Make sure that's intended.")));
		}
	}
	else if (EventName == DuplicateElementsMessageName)
	{
		if (const FMassDuplicateElementsMessage* DuplicateElementsMessage = Payload.GetPtr<const FMassDuplicateElementsMessage>())
		{
			FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);
			MessageLog.AddMessage(FTokenizedMessage::Create(OVERRIDABLE_SEVERITY(EMessageSeverity::Warning)))
				->AddToken(FUObjectToken::Create(DuplicateElementsMessage->DuplicatingTrait))
				->AddToken(FTextToken::Create(FText::FormatOrdered(
					LOCTEXT("TraitFragmentDuplicationWarning", "trying to add fragment of type {0} while it has already been added by")
					, FText::FromName(GetFNameSafe(DuplicateElementsMessage->Element))
					)))
				->AddToken(FUObjectToken::Create(DuplicateElementsMessage->OriginalTrait))
				->AddToken(FTextToken::Create(LOCTEXT("TraitFragmentDuplicationWarningCheckConflicts", "Check your entity config for conflicting traits")));
		}
	}

#undef OVERRIDABLE_SEVERITY
}
#endif // WITH_MASSENTITY_DEBUG

TConstArrayView<FName> UMassTraitRepository::GetTraitsNameAddingElements(const FName ElementName)
{
	if (bIsRepositoryInitialized == false)
	{
		InitRepository();
	}

	if (TArray<FName>* FoundTraits = ElementTypeToTraitMap.Find(ElementName))
	{
		return MakeArrayView(*FoundTraits);
	}

	return TConstArrayView<FName>();
}

void UMassTraitRepository::InitRepository()
{
	if (bIsRepositoryInitialized == true)
	{
		return;
	}

#if WITH_MASSENTITY_DEBUG
	UWorld::InitializationValues IVS;
	IVS.InitializeScenes(false)
		.AllowAudioPlayback(false)
		.RequiresHitProxies(false)
		.CreatePhysicsScene(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.SetTransactional(false)
		.CreateFXSystem(false);

	InvestigationWorld = UWorld::CreateWorld(EWorldType::Inactive
		, /*bInformEngineOfWorld=*/false
		, /*WorldName=*/TEXT("MassTraitRepository_InvestigationWorld")
		, /*Package=*/nullptr
		, /*bAddToRoot=*/false
		, /*InFeatureLevel=*/ERHIFeatureLevel::Num
		, &IVS
		, /*bInSkipInitWorld=*/true);
	GlobalInvestigationWorld = InvestigationWorld;
	check(InvestigationWorld);
	InvestigationWorld->InitWorld(IVS);

	// Marking as "initialized" so that the OnNewTraitType calls below do their job as expected. 
	bIsRepositoryInitialized = true;

	// Using RF_NoFlags to include CDOs (filtered out by default), since that's all we care about.
	for (TObjectIterator<UMassEntityTraitBase> ClassIterator(/*AdditionalExclusionFlags=*/RF_NoFlags); ClassIterator; ++ClassIterator)
	{
		if (ClassIterator && ClassIterator->HasAnyFlags(RF_ClassDefaultObject))
		{
			OnNewTraitType(**ClassIterator);
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

void UMassTraitRepository::Deinitialize()
{
#if WITH_MASSENTITY_DEBUG
	UMassEntityTraitBase::GetOnNewTraitTypeEvent().Remove(OnNewTraitTypeHandle);

	if (IsValid(InvestigationWorld))
	{
		InvestigationWorld->DestroyWorld(/*bInformEngineOfWorld=*/false);
	}
#endif // WITH_MASSENTITY_DEBUG

	Super::Deinitialize();
}

#if WITH_MASSENTITY_DEBUG
void UMassTraitRepository::OnNewTraitType(UMassEntityTraitBase& Trait)
{
	if (bIsRepositoryInitialized == false)
	{
		// since the repository is not initialized yet we assume we don't need to collect the information about Trait just yet.
		// Once InitRepository is called all existing Trait CDOs will be collected and processed.
		return;
	}

	// simply ignore abstract classes, we don't care about these since the user will never be able to use them anyway.
	if (Trait.GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		return;
	}

	checkf(Trait.HasAnyFlags(RF_ClassDefaultObject), TEXT("Only CDOs are expected here."));
	check(InvestigationWorld);

	const FName TraitName = Trait.GetClass()->GetFName();
	FTraitAndElements TraitData;

	// first check if we have this one already. If so we need to remove and re-add in the type got updated
	if (TraitClassNameToDataMap.RemoveAndCopyValue(TraitName, TraitData))
	{
		for (const FName TypeName : TraitData.ElementNames)
		{
			ElementTypeToTraitMap.FindChecked(TypeName).RemoveSingleSwap(TraitName, EAllowShrinking::No);
		}
	}

	TraitData.TraitClass = Trait.GetClass();
	TraitData.ElementNames.Reset();

	FMassTraitInspectionContext InvestigationContext;
	InvestigationContext.BuildContext.SetTrait(Trait);
	Trait.BuildTemplate(InvestigationContext.BuildContext, *InvestigationWorld);

	const FMassArchetypeCompositionDescriptor& Composition = InvestigationContext.EntityTemplate.GetCompositionDescriptor();

	for (FMassFragmentBitSet::FIndexIterator It = Composition.GetElementsBitSet().GetIndexIterator(); It; ++It)
	{
		const FName StructName = Composition.GetElementsBitSet().DebugGetStructTypeName(*It);
		TraitData.ElementNames.Add(StructName);
	}

	for (const FName ElementTypeName : TraitData.ElementNames)
	{
		ElementTypeToTraitMap.FindOrAdd(ElementTypeName).Add(TraitName);
	}
	TraitClassNameToDataMap.Add(TraitName, MoveTemp(TraitData));
}
#endif // WITH_MASSENTITY_DEBUG

UWorld* UMassTraitRepository::GetInvestigationWorld()
{
	return GlobalInvestigationWorld.Get();
}

TWeakObjectPtr<UClass> UMassTraitRepository::GetTraitClass(const FName TraitClassName) const
{
	if (const FTraitAndElements* Data = TraitClassNameToDataMap.Find(TraitClassName))
	{
		return Data->TraitClass;
	}
	return TWeakObjectPtr<UClass>();
}

//-----------------------------------------------------------------------------
// UMassDebugEntitySubsystem
//-----------------------------------------------------------------------------
bool UMassDebugEntitySubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return Outer != nullptr && Outer == UMassTraitRepository::GetInvestigationWorld();
}

#undef LOCTEXT_NAMESPACE 
