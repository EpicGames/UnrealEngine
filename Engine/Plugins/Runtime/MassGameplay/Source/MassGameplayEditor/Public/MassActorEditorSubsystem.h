// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "MassActorEditorSubsystem.generated.h"

#define UE_API MASSGAMEPLAYEDITOR_API


struct FMassActorManager;

UCLASS(MinimalAPI)
class UMassActorEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	FMassActorManager& GetMutableActorManager() { return *ActorManager.Get(); }

protected:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	TSharedPtr<FMassActorManager> ActorManager;
};

#undef UE_API
