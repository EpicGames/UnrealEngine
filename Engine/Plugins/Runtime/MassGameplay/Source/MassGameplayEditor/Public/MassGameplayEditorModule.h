// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Modules/ModuleInterface.h"

#define UE_API MASSGAMEPLAYEDITOR_API

/**
* The public interface to this module
*/
class FMassGameplayEditorModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

protected:
	UE_API void RegisterSectionMappings();
};

#undef UE_API
