// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTemplateRegistry.h"
#include "EditorSubsystem.h"
#include "MassEntitySubsystem.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/StructView.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassDebugger.h"
#include "MassTraitRepository.generated.h"

class UMassEntityTraitBase;
class UWorld;

namespace UE::Mass::Editor
{
	extern MASSGAMEPLAYEDITOR_API TConstArrayView<FName> GetTraitsNameAddingElements(const FName ElementName);
}

struct FMassTraitInspectionContext
{
	struct FInvestigationContext : public FMassEntityTemplateBuildContext
	{
		using Super = FMassEntityTemplateBuildContext;
		explicit FInvestigationContext(FMassEntityTemplateData& InTemplate);
		void SetTrait(const UMassEntityTraitBase& Trait);
	};

	FMassTraitInspectionContext();

	FMassEntityTemplateData EntityTemplate;
	FInvestigationContext BuildContext;
};

/** 
 * Subsystem to store information about Mass traits so that we can make helpful suggestions to users like which
 * traits supply a fragment they need.
 * It also serves to post trait validation information and fix options to the MessageLog.
 */
UCLASS(MinimalAPI)
class UMassTraitRepository : public UEditorSubsystem
{
	GENERATED_BODY()
	
public:
	static MASSGAMEPLAYEDITOR_API UWorld* GetInvestigationWorld();

	MASSGAMEPLAYEDITOR_API TConstArrayView<FName> GetTraitsNameAddingElements(const FName ElementName);
	MASSGAMEPLAYEDITOR_API TWeakObjectPtr<UClass> GetTraitClass(const FName TraitClassName) const;

protected:
	MASSGAMEPLAYEDITOR_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSGAMEPLAYEDITOR_API virtual void Deinitialize() override;

#if WITH_MASSENTITY_DEBUG
	/** 
	 * Note that the function does nothing until InitRepository is called. This is done to avoid collecting trait
	 * data until it's actually necessary.
	 */
	MASSGAMEPLAYEDITOR_API void OnNewTraitType(UMassEntityTraitBase& Trait);

	MASSGAMEPLAYEDITOR_API void OnDebugEvent(const FName EventName, FConstStructView Payload, EMassDebugMessageSeverity SeverityOverride);
#endif // WITH_MASSENTITY_DEBUG
	/** 
	 * The method gathers all existing trait classes and processed them. We don't do that on subsystem's init since
	 * very often the data won't be needed during the given editor run. We call the function lazily the first time 
	 * the data is needed.
	 */
	MASSGAMEPLAYEDITOR_API void InitRepository();

	FDelegateHandle OnNewTraitTypeHandle;
	struct FTraitAndElements
	{
		TWeakObjectPtr<UClass> TraitClass;
		TArray<FName> ElementNames;
	};
	TMap<FName, FTraitAndElements> TraitClassNameToDataMap;
	TMap<FName, TArray<FName>> ElementTypeToTraitMap;


	/** the World we use to host all the subsystems required to process traits */
	UPROPERTY()
	TObjectPtr<UWorld> InvestigationWorld;

	/** We initialize the repository's data lazily and this property indicates whether it has been already done. */
	bool bIsRepositoryInitialized;

	static MASSGAMEPLAYEDITOR_API TWeakObjectPtr<UWorld> GlobalInvestigationWorld;
};


UCLASS()
class UMassDebugEntitySubsystem : public UMassEntitySubsystem
{
	GENERATED_BODY()
	
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
};
