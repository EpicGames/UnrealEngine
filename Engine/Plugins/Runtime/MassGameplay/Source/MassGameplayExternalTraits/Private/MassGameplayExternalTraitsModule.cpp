// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMassGameplayExternalTraitsModule.h"

class FMassGameplayExternalTraitsModule : public IMassGameplayExternalTraitsModule
{

};

IMPLEMENT_MODULE(FMassGameplayExternalTraitsModule, MassGameplayExternalTraits)
