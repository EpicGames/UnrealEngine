// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class MassGameplayTestSuite : ModuleRules
	{
		public MassGameplayTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AITestSuite",
					"MassCore",
					"MassEntity",
					"MassEntityTestSuite",
					"MassLOD",
					"MassMovement",
					"MassSpawner",
					"MassCharacterTrajectory",
					"PoseSearch",
				}
			);
		}
	}
}