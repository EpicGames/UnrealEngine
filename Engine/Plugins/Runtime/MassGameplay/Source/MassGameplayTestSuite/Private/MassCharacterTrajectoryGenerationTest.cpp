// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTestTypes.h"
#include "MassCommonFragments.h"
#include "MassEntityBuilder.h"
#include "MassMovementFragments.h"
#include "MassProcessingContext.h"
#include "MassSpringMovementFragments.h"
#include "MassCharacterTrajectoryFragments.h"
#include "MassCharacterTrajectoryGenerationProcessors.h"
#include "MassCharacterTrajectoryMovementProcessor.h"
#include "MassSpringMovementProcessors.h"
#include "Movement/MassMovementProcessors.h"
#include "MassLODFragments.h"
#include "MassObserverManager.h"
#include "MassExecutor.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"

#define LOCTEXT_NAMESPACE "MassTest"

namespace UE::Mass::CharacterTrajectory::Tests
{

//----------------------------------------------------------------------//
// Helpers
//----------------------------------------------------------------------//

static FCharacterTrajectoryParameters MakeDefaultCharacterTrajectoryParams()
{
	FCharacterTrajectoryParameters Params;
	Params.NumHistorySamples = 3;
	Params.NumPredictionSamples = 5;
	Params.PredictionSamplingInterval = 0.1f;
	Params.Offset = FTransform::Identity;
	return Params;
}

static FSpringMovementSettings MakeDefaultSpringSettings()
{
	FSpringMovementSettings Settings;
	Settings.VelocitySmoothingTime = 0.1f;
	Settings.FacingSmoothingTime = 0.1f;
	Settings.VelocityDeadzoneThreshold = 0.1f;
	return Settings;
}

static FMassEntityHandle CreateCharacterTrajectoryEntity(
	FMassEntityManager& EntityManager,
	const FCharacterTrajectoryParameters& TrajectoryParams,
	bool bWithSpring = false,
	const FSpringMovementSettings* SpringSettings = nullptr)
{
	TArray<const UScriptStruct*> Fragments;
	Fragments.Add(FCharacterTrajectoryFragment::StaticStruct());
	Fragments.Add(FTransformFragment::StaticStruct());
	Fragments.Add(FMassDesiredMovementFragment::StaticStruct());
	Fragments.Add(FMassVelocityFragment::StaticStruct());
	if (bWithSpring)
	{
		Fragments.Add(FSpringMovementRuntime::StaticStruct());
	}

	FMassArchetypeHandle Archetype = EntityManager.CreateArchetype(Fragments);
	TArray<FMassEntityHandle> Entities;
	EntityManager.BatchCreateEntities(Archetype, 1, Entities);
	FMassEntityHandle Entity = Entities[0];

	// Add const shared fragments
	const FConstSharedStruct TrajectoryParamsShared = EntityManager.GetOrCreateConstSharedFragment(TrajectoryParams);
	EntityManager.AddConstSharedFragmentToEntity(Entity, TrajectoryParamsShared);

	if (bWithSpring && SpringSettings)
	{
		const FConstSharedStruct SpringSettingsShared = EntityManager.GetOrCreateConstSharedFragment(*SpringSettings);
		EntityManager.AddConstSharedFragmentToEntity(Entity, SpringSettingsShared);
	}

	return Entity;
}

static void RunProcessor(UMassProcessor& Processor, FMassEntityManager& EntityManager, float DeltaTime)
{
	FMassProcessingContext ProcessingContext(EntityManager, DeltaTime);
	UE::Mass::Executor::Run(Processor, ProcessingContext);
}

static void CreateMovementBenchmarkEntities(
	FMassEntityManager& EntityManager,
	int32 Count,
	bool bSpring,
	const FSpringMovementSettings& SpringSettings,
	TArray<FMassEntityHandle>& OutEntities)
{
	TArray<const UScriptStruct*> Fragments;
	Fragments.Add(FTransformFragment::StaticStruct());
	Fragments.Add(FMassDesiredMovementFragment::StaticStruct());
	Fragments.Add(FMassVelocityFragment::StaticStruct());

	if (bSpring)
	{
		Fragments.Add(FSpringMovementRuntime::StaticStruct());
	}
	else
	{
		Fragments.Add(FMassCodeDrivenMovementTag::StaticStruct());
	}

	FMassArchetypeHandle Archetype = EntityManager.CreateArchetype(Fragments);

	if (bSpring)
	{
		FMassArchetypeSharedFragmentValues SharedValues;
		const FConstSharedStruct SpringSettingsShared = EntityManager.GetOrCreateConstSharedFragment(SpringSettings);
		SharedValues.Add(SpringSettingsShared);
		SharedValues.Sort();
		EntityManager.BatchCreateEntities(Archetype, SharedValues, Count, OutEntities);
	}
	else
	{
		EntityManager.BatchCreateEntities(Archetype, Count, OutEntities);
	}

	for (int32 Index = 0; Index < OutEntities.Num(); ++Index)
	{
		FMassEntityHandle Entity = OutEntities[Index];

		FTransformFragment& Transform = EntityManager.GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(Index * 10.0f, 0.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager.GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(300.0f, 100.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		if (bSpring)
		{
			FSpringMovementRuntime& SpringRuntime = EntityManager.GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
			SpringRuntime.CurrentPosition = FVector(Index * 10.0f, 0.0f, 0.0f);
			SpringRuntime.CurrentVelocity = FVector(200.0f, 50.0f, 0.0f);
			SpringRuntime.CurrentAccel = FVector::ZeroVector;
			SpringRuntime.CurrentFacing = FQuat::Identity;
			SpringRuntime.CurrentAngularVelocity = FVector::ZeroVector;
		}
	}
}

//----------------------------------------------------------------------//
// Test 1: Sample Layout Correctness (non-spring)
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_SampleLayout : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const float DeltaTime = 0.016f;
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();

		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		// Set a known transform and desired velocity
		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(100.0f, 200.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(300.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		// Run the non-spring trajectory processor
		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*Processor, *EntityManager, DeltaTime);

		// Verify sample layout
		const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		const int32 NumHistory = Params.NumHistorySamples;
		const int32 ExpectedSamples = NumHistory + 1 + Params.NumPredictionSamples;

		AITEST_EQUAL("Total sample count", Trajectory.Trajectory.Samples.Num(), ExpectedSamples);

		// [NumHistory] should be at t=DeltaTime with the predicted position
		const FTransformTrajectorySample& CurrentSample = Trajectory.Trajectory.Samples[NumHistory];
		AITEST_TRUE("Current sample time equals DeltaTime", FMath::IsNearlyEqual(CurrentSample.TimeInSeconds, DeltaTime, 1e-5f));

		const FVector ExpectedPosition = FVector(100.0f, 200.0f, 0.0f) + FVector(300.0f, 0.0f, 0.0f) * DeltaTime;
		AITEST_TRUE("Current sample position is extrapolated",
			CurrentSample.Position.Equals(ExpectedPosition, 1.0f));

		// [NumHistory+1] should be at t=DeltaTime + PredictionInterval
		if (Params.NumPredictionSamples > 0)
		{
			const FTransformTrajectorySample& FirstPrediction = Trajectory.Trajectory.Samples[NumHistory + 1];
			const float ExpectedTime = DeltaTime + Params.PredictionSamplingInterval;
			AITEST_TRUE("First prediction time", FMath::IsNearlyEqual(FirstPrediction.TimeInSeconds, ExpectedTime, 1e-5f));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_SampleLayout, "System.Mass.CharacterTrajectory.SampleLayout");

//----------------------------------------------------------------------//
// Test 2: Variable Delta Time History
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_VariableDeltaTime : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();
		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(0.0f, 0.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(100.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		const float DeltaTimes[] = { 0.016f, 0.033f, 0.008f, 0.050f, 0.016f };
		FVector AccumulatedPosition = FVector::ZeroVector;

		for (int32 Frame = 0; Frame < UE_ARRAY_COUNT(DeltaTimes); ++Frame)
		{
			const float DT = DeltaTimes[Frame];

			// Simulate movement: advance position by desired velocity * previous DT
			// (In the real flow, the movement processor does this AFTER trajectory generation)
			if (Frame > 0)
			{
				AccumulatedPosition += DesiredMovement.DesiredVelocity * DeltaTimes[Frame - 1];
				Transform.GetMutableTransform().SetTranslation(AccumulatedPosition);
			}

			RunProcessor(*Processor, *EntityManager, DT);

			const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
			const int32 NumHistory = Params.NumHistorySamples;

			// Verify current sample is at t=DeltaTime
			AITEST_TRUE(FString::Printf(TEXT("Frame %d: current sample time == DT"), Frame),
				FMath::IsNearlyEqual(Trajectory.Trajectory.Samples[NumHistory].TimeInSeconds, DT, 1e-5f));

			// After first frame, verify history sample [NumHistory-1] has t <= 0 (it's a past position)
			if (Frame > 0)
			{
				AITEST_TRUE(FString::Printf(TEXT("Frame %d: most recent history at t <= 0"), Frame),
					Trajectory.Trajectory.Samples[NumHistory - 1].TimeInSeconds <= 1e-5f);
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_VariableDeltaTime, "System.Mass.CharacterTrajectory.VariableDeltaTime");

//----------------------------------------------------------------------//
// Test 3: First Frame Initialization
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_FirstFrameInit : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();
		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		// Verify trajectory starts empty
		const FCharacterTrajectoryFragment& TrajectoryBefore = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		AITEST_EQUAL("Trajectory starts with 0 samples", TrajectoryBefore.Trajectory.Samples.Num(), 0);

		// Run processor
		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*Processor, *EntityManager, 0.016f);

		// Verify correct sample count after first frame
		const FCharacterTrajectoryFragment& TrajectoryAfter = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		const int32 ExpectedSamples = Params.NumHistorySamples + 1 + Params.NumPredictionSamples;
		AITEST_EQUAL("Trajectory initialized with correct sample count", TrajectoryAfter.Trajectory.Samples.Num(), ExpectedSamples);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_FirstFrameInit, "System.Mass.CharacterTrajectory.FirstFrameInit");

//----------------------------------------------------------------------//
// Test 4: Execution Order Exclusion (spring vs non-spring)
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_ProcessorExclusion : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const FCharacterTrajectoryParameters TrajectoryParams = MakeDefaultCharacterTrajectoryParams();
		const FSpringMovementSettings SpringSettings = MakeDefaultSpringSettings();

		// Create a spring entity
		FMassEntityHandle SpringEntity = CreateCharacterTrajectoryEntity(*EntityManager, TrajectoryParams, /*bWithSpring=*/true, &SpringSettings);

		// Create a non-spring entity
		FMassEntityHandle NonSpringEntity = CreateCharacterTrajectoryEntity(*EntityManager, TrajectoryParams, /*bWithSpring=*/false);

		// Run the non-spring processor — should only process the non-spring entity
		UMovementToCharacterTrajectoryProcessor* NonSpringProcessor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		NonSpringProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*NonSpringProcessor, *EntityManager, 0.016f);

		// The non-spring entity should have trajectory data, the spring entity should not
		const FCharacterTrajectoryFragment& NonSpringTrajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(NonSpringEntity);
		const FCharacterTrajectoryFragment& SpringTrajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(SpringEntity);

		AITEST_TRUE("Non-spring entity has trajectory samples", NonSpringTrajectory.Trajectory.Samples.Num() > 0);
		AITEST_EQUAL("Spring entity was not processed by non-spring processor", SpringTrajectory.Trajectory.Samples.Num(), 0);

		// Now run the spring processor — should only process the spring entity
		USpringMovementToCharacterTrajectoryProcessor* SpringProcessor = NewObject<USpringMovementToCharacterTrajectoryProcessor>();
		SpringProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*SpringProcessor, *EntityManager, 0.016f);

		const FCharacterTrajectoryFragment& SpringTrajectoryAfter = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(SpringEntity);
		AITEST_TRUE("Spring entity now has trajectory samples", SpringTrajectoryAfter.Trajectory.Samples.Num() > 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_ProcessorExclusion, "System.Mass.CharacterTrajectory.ProcessorExclusion");

//----------------------------------------------------------------------//
// Test 5: Non-Spring Uses DesiredVelocity for Position
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_DesiredVelocityExtrapolation : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const float DeltaTime = 0.020f;
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();

		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(500.0f, 100.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(200.0f, -50.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*Processor, *EntityManager, DeltaTime);

		const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		const int32 NumHistory = Params.NumHistorySamples;

		// Current sample should use DesiredVelocity
		const FVector StartPos(500.0f, 100.0f, 0.0f);
		const FVector ExpectedCurrent = StartPos + DesiredMovement.DesiredVelocity * DeltaTime;
		AITEST_TRUE("Current sample uses DesiredVelocity",
			Trajectory.Trajectory.Samples[NumHistory].Position.Equals(ExpectedCurrent, 1.0f));

		// Prediction samples should extrapolate at DesiredVelocity
		for (int32 i = 0; i < Params.NumPredictionSamples; ++i)
		{
			const int32 SampleIndex = NumHistory + 1 + i;
			if (SampleIndex < Trajectory.Trajectory.Samples.Num())
			{
				const float FutureTime = (i + 1) * Params.PredictionSamplingInterval;
				const FVector ExpectedPrediction = ExpectedCurrent + DesiredMovement.DesiredVelocity * FutureTime;
				AITEST_TRUE(FString::Printf(TEXT("Prediction %d extrapolates correctly"), i),
					Trajectory.Trajectory.Samples[SampleIndex].Position.Equals(ExpectedPrediction, 1.0f));
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_DesiredVelocityExtrapolation, "System.Mass.CharacterTrajectory.DesiredVelocityExtrapolation");

//----------------------------------------------------------------------//
// Test 6: Spring Trajectory Uses Spring State
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_SpringTrajectory : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const float DeltaTime = 0.016f;
		const FCharacterTrajectoryParameters TrajectoryParams = MakeDefaultCharacterTrajectoryParams();
		const FSpringMovementSettings SpringSettings = MakeDefaultSpringSettings();

		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, TrajectoryParams, /*bWithSpring=*/true, &SpringSettings);

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(0.0f, 0.0f, 0.0f));

		// Set up spring runtime with a known state
		FSpringMovementRuntime& SpringRuntime = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		SpringRuntime.CurrentPosition = FVector(10.0f, 0.0f, 0.0f);
		SpringRuntime.CurrentVelocity = FVector(200.0f, 0.0f, 0.0f);
		SpringRuntime.CurrentAccel = FVector::ZeroVector;
		SpringRuntime.CurrentFacing = FQuat::Identity;
		SpringRuntime.CurrentAngularVelocity = FVector::ZeroVector;

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(200.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		USpringMovementToCharacterTrajectoryProcessor* Processor = NewObject<USpringMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*Processor, *EntityManager, DeltaTime);

		const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		const int32 NumHistory = TrajectoryParams.NumHistorySamples;

		// [NumHistory] should be at t=DeltaTime using the spring state, not the transform
		AITEST_TRUE("Current sample time equals DeltaTime",
			FMath::IsNearlyEqual(Trajectory.Trajectory.Samples[NumHistory].TimeInSeconds, DeltaTime, 1e-5f));

		// Position should be derived from spring CurrentPosition (10,0,0), not transform (0,0,0)
		AITEST_TRUE("Current sample uses spring position not transform",
			FMath::Abs(Trajectory.Trajectory.Samples[NumHistory].Position.X - SpringRuntime.CurrentPosition.X) < 50.0f);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_SpringTrajectory, "System.Mass.CharacterTrajectory.SpringTrajectory");

//----------------------------------------------------------------------//
// Test 7: Zero DeltaTime Safety
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_ZeroDeltaTime : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();
		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(100.0f, 0.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(300.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		// Run with DeltaTime = 0 — should not crash or produce NaN
		RunProcessor(*Processor, *EntityManager, 0.0f);

		const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		const int32 NumHistory = Params.NumHistorySamples;

		// Verify no NaN in any sample
		for (int32 SampleIndex = 0; SampleIndex < Trajectory.Trajectory.Samples.Num(); ++SampleIndex)
		{
			const FTransformTrajectorySample& Sample = Trajectory.Trajectory.Samples[SampleIndex];
			AITEST_FALSE(FString::Printf(TEXT("Sample %d position is not NaN"), SampleIndex),
				Sample.Position.ContainsNaN());
			AITEST_FALSE(FString::Printf(TEXT("Sample %d time is not NaN"), SampleIndex),
				FMath::IsNaN(Sample.TimeInSeconds));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_ZeroDeltaTime, "System.Mass.CharacterTrajectory.ZeroDeltaTime");

//----------------------------------------------------------------------//
// Test 8: History Monotonicity Over Many Frames
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_HistoryMonotonicity : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();
		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector::ZeroVector);

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(100.0f, 50.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		UMovementToCharacterTrajectoryProcessor* Processor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		Processor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		// Run enough frames to fill the history buffer
		const int32 NumFrames = Params.NumHistorySamples + 5;
		FVector Position = FVector::ZeroVector;

		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			const float DeltaTime = 0.016f + (Frame % 3) * 0.005f; // Varying DT
			Transform.GetMutableTransform().SetTranslation(Position);
			RunProcessor(*Processor, *EntityManager, DeltaTime);
			Position += DesiredMovement.DesiredVelocity * DeltaTime;
		}

		// After many frames, verify history times are monotonically increasing
		const FCharacterTrajectoryFragment& Trajectory = EntityManager->GetFragmentDataChecked<FCharacterTrajectoryFragment>(Entity);
		for (int32 SampleIndex = 1; SampleIndex < Params.NumHistorySamples; ++SampleIndex)
		{
			AITEST_TRUE(FString::Printf(TEXT("History sample %d time > sample %d time"), SampleIndex, SampleIndex - 1),
				Trajectory.Trajectory.Samples[SampleIndex].TimeInSeconds >
				Trajectory.Trajectory.Samples[SampleIndex - 1].TimeInSeconds);
		}

		// Verify current sample time is positive
		AITEST_TRUE("Current sample time is positive",
			Trajectory.Trajectory.Samples[Params.NumHistorySamples].TimeInSeconds > 0.0f);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_HistoryMonotonicity, "System.Mass.CharacterTrajectory.HistoryMonotonicity");

//----------------------------------------------------------------------//
// Test 9: Trajectory Movement Processor Moves Entity
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_TrajectoryMovement : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const float DeltaTime = 0.016f;
		const FCharacterTrajectoryParameters Params = MakeDefaultCharacterTrajectoryParams();

		FMassEntityHandle Entity = CreateCharacterTrajectoryEntity(*EntityManager, Params);

		// Add FCharacterTrajectoryMovementTag
		EntityManager->AddTagToEntity(Entity, FCharacterTrajectoryMovementTag::StaticStruct());

		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(0.0f, 0.0f, 0.0f));

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(500.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		// First generate a trajectory
		UMovementToCharacterTrajectoryProcessor* GenProcessor = NewObject<UMovementToCharacterTrajectoryProcessor>();
		GenProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*GenProcessor, *EntityManager, DeltaTime);

		// Record position before movement
		const FVector PositionBeforeMove = Transform.GetTransform().GetLocation();

		// Now run trajectory movement processor
		UCharacterTrajectoryToMovementProcessor* MoveProcessor = NewObject<UCharacterTrajectoryToMovementProcessor>();
		MoveProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
		RunProcessor(*MoveProcessor, *EntityManager, DeltaTime);

		// Position should have changed
		const FVector PositionAfterMove = Transform.GetTransform().GetLocation();
		const float DistanceMoved = FVector::Dist(PositionBeforeMove, PositionAfterMove);

		AITEST_TRUE("Entity moved after trajectory movement", DistanceMoved > 1.0f);

		// Velocity should be non-zero
		const FMassVelocityFragment& Velocity = EntityManager->GetFragmentDataChecked<FMassVelocityFragment>(Entity);
		AITEST_TRUE("Velocity is non-zero after trajectory movement", Velocity.Value.SizeSquared() > 1.0f);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_TrajectoryMovement, "System.Mass.CharacterTrajectory.TrajectoryMovement");

//----------------------------------------------------------------------//
// Test 10: Movement Benchmark — Vanilla vs Spring (500 entities)
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_MovementBenchmark : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 EntityCount = 500;
		constexpr int32 WarmUpFrames = 10;
		constexpr int32 BenchmarkFrames = 100;
		constexpr float DeltaTime = 0.016f;

		const FSpringMovementSettings SpringSettings = MakeDefaultSpringSettings();

		// Create 500 vanilla entities and 500 spring entities
		TArray<FMassEntityHandle> VanillaEntities;
		TArray<FMassEntityHandle> SpringEntities;
		CreateMovementBenchmarkEntities(*EntityManager, EntityCount, /*bSpring=*/false, SpringSettings, VanillaEntities);
		CreateMovementBenchmarkEntities(*EntityManager, EntityCount, /*bSpring=*/true, SpringSettings, SpringEntities);

		AITEST_EQUAL("Vanilla entity count", VanillaEntities.Num(), EntityCount);
		AITEST_EQUAL("Spring entity count", SpringEntities.Num(), EntityCount);

		// Initialize processors
		UMassApplyMovementProcessor* VanillaProcessor = NewObject<UMassApplyMovementProcessor>();
		VanillaProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		USpringUpdateProcessor* SpringUpdate = NewObject<USpringUpdateProcessor>();
		SpringUpdate->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		USpringApplyMovementProcessor* SpringApply = NewObject<USpringApplyMovementProcessor>();
		SpringApply->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		// Warm up
		for (int32 Frame = 0; Frame < WarmUpFrames; ++Frame)
		{
			RunProcessor(*VanillaProcessor, *EntityManager, DeltaTime);
			RunProcessor(*SpringUpdate, *EntityManager, DeltaTime);
			RunProcessor(*SpringApply, *EntityManager, DeltaTime);
		}

		// Benchmark vanilla path
		const double VanillaStart = FPlatformTime::Seconds();
		for (int32 Frame = 0; Frame < BenchmarkFrames; ++Frame)
		{
			RunProcessor(*VanillaProcessor, *EntityManager, DeltaTime);
		}
		const double VanillaEnd = FPlatformTime::Seconds();
		const double VanillaTotalMs = (VanillaEnd - VanillaStart) * 1000.0;
		const double VanillaPerFrameMs = VanillaTotalMs / BenchmarkFrames;

		// Benchmark spring path
		const double SpringStart = FPlatformTime::Seconds();
		for (int32 Frame = 0; Frame < BenchmarkFrames; ++Frame)
		{
			RunProcessor(*SpringUpdate, *EntityManager, DeltaTime);
			RunProcessor(*SpringApply, *EntityManager, DeltaTime);
		}
		const double SpringEnd = FPlatformTime::Seconds();
		const double SpringTotalMs = (SpringEnd - SpringStart) * 1000.0;
		const double SpringPerFrameMs = SpringTotalMs / BenchmarkFrames;

		// Compute delta
		const double DeltaMs = SpringPerFrameMs - VanillaPerFrameMs;
		const double Ratio = (VanillaPerFrameMs > 0.0) ? (SpringPerFrameMs / VanillaPerFrameMs) : 0.0;

		// Log results
		UE_LOG(LogTemp, Display, TEXT("=== Movement Benchmark: %d entities, %d frames ==="), EntityCount, BenchmarkFrames);
		UE_LOG(LogTemp, Display, TEXT("Vanilla (UMassApplyMovementProcessor):                          %.4f ms/frame (total: %.2f ms)"), VanillaPerFrameMs, VanillaTotalMs);
		UE_LOG(LogTemp, Display, TEXT("Spring  (USpringUpdateProcessor + USpringApplyMovementProcessor): %.4f ms/frame (total: %.2f ms)"), SpringPerFrameMs, SpringTotalMs);
		UE_LOG(LogTemp, Display, TEXT("Delta: %.4f ms/frame | Ratio: %.2fx"), DeltaMs, Ratio);

		// Correctness assertions — verify processors actually ran
		{
			const FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(VanillaEntities[0]);
			const FMassVelocityFragment& Velocity = EntityManager->GetFragmentDataChecked<FMassVelocityFragment>(VanillaEntities[0]);
			AITEST_FALSE("Vanilla entity position is not NaN", Transform.GetTransform().GetLocation().ContainsNaN());
			AITEST_TRUE("Vanilla entity velocity is non-zero", Velocity.Value.SizeSquared() > 1.0f);
		}
		{
			const FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(SpringEntities[0]);
			const FMassVelocityFragment& Velocity = EntityManager->GetFragmentDataChecked<FMassVelocityFragment>(SpringEntities[0]);
			const FSpringMovementRuntime& SpringRuntime = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(SpringEntities[0]);
			AITEST_FALSE("Spring entity position is not NaN", Transform.GetTransform().GetLocation().ContainsNaN());
			AITEST_TRUE("Spring entity velocity is non-zero", Velocity.Value.SizeSquared() > 1.0f);
			AITEST_TRUE("Spring runtime velocity is non-zero", SpringRuntime.CurrentVelocity.SizeSquared() > 1.0f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_MovementBenchmark, "System.Mass.CharacterTrajectory.MovementBenchmark");

//----------------------------------------------------------------------//
// Test 11: Spring LOD Handling — skip when OffLOD, resync on transition
//----------------------------------------------------------------------//
struct FCharacterTrajectoryTest_SpringLODHandling : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		const float DeltaTime = 0.016f;
		const FSpringMovementSettings SpringSettings = MakeDefaultSpringSettings();

		// Create a spring entity
		TArray<const UScriptStruct*> Fragments;
		Fragments.Add(FTransformFragment::StaticStruct());
		Fragments.Add(FMassDesiredMovementFragment::StaticStruct());
		Fragments.Add(FMassVelocityFragment::StaticStruct());
		Fragments.Add(FSpringMovementRuntime::StaticStruct());

		FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Fragments);
		FMassArchetypeSharedFragmentValues SharedValues;
		const FConstSharedStruct SpringSettingsShared = EntityManager->GetOrCreateConstSharedFragment(SpringSettings);
		SharedValues.Add(SpringSettingsShared);
		SharedValues.Sort();

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(Archetype, SharedValues, 1, Entities);
		FMassEntityHandle Entity = Entities[0];

		// Set known initial state
		FTransformFragment& Transform = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		Transform.GetMutableTransform().SetTranslation(FVector(100.0f, 200.0f, 0.0f));
		Transform.GetMutableTransform().SetRotation(FQuat::Identity);

		FMassDesiredMovementFragment& DesiredMovement = EntityManager->GetFragmentDataChecked<FMassDesiredMovementFragment>(Entity);
		DesiredMovement.DesiredVelocity = FVector(300.0f, 0.0f, 0.0f);
		DesiredMovement.DesiredFacing = FQuat::Identity;

		FMassVelocityFragment& Velocity = EntityManager->GetFragmentDataChecked<FMassVelocityFragment>(Entity);
		Velocity.Value = FVector::ZeroVector;

		FSpringMovementRuntime& SpringRuntime = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		SpringRuntime.CurrentPosition = FVector(100.0f, 200.0f, 0.0f);
		SpringRuntime.CurrentVelocity = FVector::ZeroVector;
		SpringRuntime.CurrentAccel = FVector::ZeroVector;
		SpringRuntime.CurrentFacing = FQuat::Identity;
		SpringRuntime.CurrentAngularVelocity = FVector::ZeroVector;

		// Initialize processors
		USpringUpdateProcessor* SpringUpdate = NewObject<USpringUpdateProcessor>();
		SpringUpdate->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		USpringApplyMovementProcessor* SpringApply = NewObject<USpringApplyMovementProcessor>();
		SpringApply->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

		// --- Step 1: Verify spring runs without OffLOD ---
		RunProcessor(*SpringUpdate, *EntityManager, DeltaTime);
		RunProcessor(*SpringApply, *EntityManager, DeltaTime);

		const FVector PositionAfterFirstFrame = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity).CurrentPosition;
		AITEST_FALSE("Spring ran: position changed from initial",
			PositionAfterFirstFrame.Equals(FVector(100.0f, 200.0f, 0.0f), 0.01f));

		// --- Step 2: Add OffLOD tag, verify spring skips ---
		EntityManager->AddTagToEntity(Entity, FMassOffLODTag::StaticStruct());

		// Set spring state to a stale sentinel value
		FSpringMovementRuntime& RuntimeBeforeSkip = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		RuntimeBeforeSkip.CurrentPosition = FVector(999.0f, 999.0f, 999.0f);
		RuntimeBeforeSkip.CurrentVelocity = FVector(999.0f, 999.0f, 999.0f);

		RunProcessor(*SpringUpdate, *EntityManager, DeltaTime);
		RunProcessor(*SpringApply, *EntityManager, DeltaTime);

		const FSpringMovementRuntime& RuntimeAfterSkip = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		AITEST_TRUE("Spring skipped: position unchanged while OffLOD",
			RuntimeAfterSkip.CurrentPosition.Equals(FVector(999.0f, 999.0f, 999.0f), 0.01f));
		AITEST_TRUE("Spring skipped: velocity unchanged while OffLOD",
			RuntimeAfterSkip.CurrentVelocity.Equals(FVector(999.0f, 999.0f, 999.0f), 0.01f));

		// --- Step 3: Remove OffLOD tag, verify observer resyncs spring state ---
		// Set the transform/velocity to known values that differ from the stale spring state
		FTransformFragment& TransformBeforeResync = EntityManager->GetFragmentDataChecked<FTransformFragment>(Entity);
		TransformBeforeResync.GetMutableTransform().SetTranslation(FVector(500.0f, 600.0f, 0.0f));
		TransformBeforeResync.GetMutableTransform().SetRotation(FQuat(FVector::UpVector, PI / 4.0f));

		FMassVelocityFragment& VelocityBeforeResync = EntityManager->GetFragmentDataChecked<FMassVelocityFragment>(Entity);
		VelocityBeforeResync.Value = FVector(150.0f, 75.0f, 0.0f);

		// Register the observer and remove the tag via deferred path to trigger it
		// Note: AddObserverInstance calls CallInitialize internally, do not double-init
		USpringMovementLODResyncObserver* Observer = NewObject<USpringMovementLODResyncObserver>();

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(Observer);

		EntityManager->Defer().RemoveTag<FMassOffLODTag>(Entity);
		EntityManager->FlushCommands();

		// Verify resync
		const FSpringMovementRuntime& RuntimeAfterResync = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		AITEST_TRUE("Resync: position matches transform",
			RuntimeAfterResync.CurrentPosition.Equals(FVector(500.0f, 600.0f, 0.0f), 0.01f));
		AITEST_TRUE("Resync: velocity matches velocity fragment",
			RuntimeAfterResync.CurrentVelocity.Equals(FVector(150.0f, 75.0f, 0.0f), 0.01f));
		AITEST_TRUE("Resync: accel zeroed",
			RuntimeAfterResync.CurrentAccel.Equals(FVector::ZeroVector, 0.01f));
		AITEST_TRUE("Resync: angular velocity zeroed",
			RuntimeAfterResync.CurrentAngularVelocity.Equals(FVector::ZeroVector, 0.01f));
		AITEST_TRUE("Resync: facing matches transform rotation",
			RuntimeAfterResync.CurrentFacing.Equals(FQuat(FVector::UpVector, PI / 4.0f), 0.01f));

		// --- Step 4: Verify spring resumes processing ---
		const FVector PositionBeforeResume = RuntimeAfterResync.CurrentPosition;

		RunProcessor(*SpringUpdate, *EntityManager, DeltaTime);
		RunProcessor(*SpringApply, *EntityManager, DeltaTime);

		const FSpringMovementRuntime& RuntimeAfterResume = EntityManager->GetFragmentDataChecked<FSpringMovementRuntime>(Entity);
		AITEST_FALSE("Spring resumed: position changed after OffLOD removed",
			RuntimeAfterResume.CurrentPosition.Equals(PositionBeforeResume, 0.01f));

		// Cleanup observer
		ObserverManager.RemoveObserverInstance(Observer);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCharacterTrajectoryTest_SpringLODHandling, "System.Mass.CharacterTrajectory.SpringLODHandling");

} // namespace UE::Mass::CharacterTrajectory::Tests

#undef LOCTEXT_NAMESPACE
