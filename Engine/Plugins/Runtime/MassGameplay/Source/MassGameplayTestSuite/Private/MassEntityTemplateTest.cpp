// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityTypes.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTemplate.h"
#include "MassEntityTemplateRegistry.h"
#include "MassObserverNotificationTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassEntityTemplateTest
{

struct FEntityTemplateBase : FExecutionTestBase
{
	FMassEntityTemplateRegistry TemplateRegistry;

	virtual bool SetUp() override
	{
		if (FExecutionTestBase::SetUp())
		{
			TemplateRegistry.Initialize(EntityManager);
			return true;
		}
		return false;
	}
};


struct FEntityTemplate_Empty : FEntityTemplateBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityTemplateID InitialTemplateID = FMassEntityTemplateIDFactory::Make(FGuid::NewGuid());
		FMassEntityTemplateData TemplateData;

		const TSharedRef<FMassEntityTemplate> FinalizedTemplate = TemplateRegistry.FindOrAddTemplate(InitialTemplateID, MoveTemp(TemplateData));
		
		const FMassEntityTemplateID TemplateID = FinalizedTemplate->GetTemplateID();
		AITEST_TRUE("Empty template is expected to be registered as a valid template", TemplateID.IsValid());
		AITEST_EQUAL("Empty template's ID is expected to be the same as the input template", TemplateID, InitialTemplateID);

		const FMassArchetypeHandle& ArchetypeHandle = FinalizedTemplate->GetArchetype();
		AITEST_TRUE("Empty template is expected to produce a valid (if empty) archetype", ArchetypeHandle.IsValid());

		const FMassArchetypeCompositionDescriptor& ArchetypeDescription = EntityManager->GetArchetypeComposition(ArchetypeHandle);
		AITEST_TRUE("Empty template is expected to produce an empty-composition archetype", ArchetypeDescription.IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTemplate_Empty, "System.Mass.EntityTemplate.Empty");


struct FEntityTemplate_Composition : FEntityTemplateBase
{
	virtual bool InstantTest() override
	{
		FMassEntityTemplateData TemplateData;

		TemplateData.AddFragment<FTestFragment_Int>();
		TemplateData.AddFragment<FTestFragment_Float>();
		TemplateData.AddChunkFragment<FTestChunkFragment_Int>();
		{
			FSharedStruct SharedFragment = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(0);
			TemplateData.AddSharedFragment(SharedFragment);
		}
		{
			FSharedStruct SharedFragment = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Float>(1);
			TemplateData.AddSharedFragment(SharedFragment);
		}
		TemplateData.Sort();

		FMassArchetypeCompositionDescriptor ExpectedComposition;
		ExpectedComposition.Add<FTestFragment_Int>();
		ExpectedComposition.Add<FTestFragment_Float>();
		ExpectedComposition.Add<FTestChunkFragment_Int>();
		ExpectedComposition.Add<FTestSharedFragment_Int>();
		ExpectedComposition.Add<FTestSharedFragment_Float>();

		AITEST_TRUE("The composition should end up being the same regardless of whether the data is added via a template data or a composition descriptor"
			, ExpectedComposition.IsEquivalent(TemplateData.GetCompositionDescriptor()));

		const FMassEntityTemplateID TemplateID = FMassEntityTemplateIDFactory::Make(FGuid::NewGuid());
		const TSharedRef<FMassEntityTemplate> FinalizedTemplate = TemplateRegistry.FindOrAddTemplate(TemplateID, MoveTemp(TemplateData));

		const FMassArchetypeHandle& ArchetypeHandle = FinalizedTemplate->GetArchetype();
		AITEST_TRUE("Empty template is expected to produce a valid (if empty) archetype", ArchetypeHandle.IsValid());

		const FMassArchetypeCompositionDescriptor& ArchetypeDescription = EntityManager->GetArchetypeComposition(ArchetypeHandle);
		AITEST_TRUE("Empty template is expected to produce an empty-composition archetype", ArchetypeDescription.IsEquivalent(ExpectedComposition));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTemplate_Composition, "System.Mass.EntityTemplate.Composition");


struct FEntityTemplate_Trivial : FEntityTemplateBase
{
	virtual bool InstantTest() override
	{
		FMassEntityTemplateData EmptyTemplateData;
		FMassEntityTemplateData TemplateDataA;
		FMassEntityTemplateData TemplateDataB;

		TemplateDataA.AddFragment<FTestFragment_Int>();
		TemplateDataB.AddFragment<FTestFragment_Int>();
		const FGuid NonEmptyTemplateGuid = FGuid::NewGuid();

		const TSharedRef<FMassEntityTemplate> FinalizedEmptyTemplate = TemplateRegistry.FindOrAddTemplate(FMassEntityTemplateIDFactory::Make(FGuid::NewGuid()), MoveTemp(EmptyTemplateData));
		const TSharedRef<FMassEntityTemplate> FinalizedTemplateA = TemplateRegistry.FindOrAddTemplate(FMassEntityTemplateIDFactory::Make(NonEmptyTemplateGuid), MoveTemp(TemplateDataA));
		const TSharedRef<FMassEntityTemplate> FinalizedTemplateB = TemplateRegistry.FindOrAddTemplate(FMassEntityTemplateIDFactory::Make(NonEmptyTemplateGuid), MoveTemp(TemplateDataB));

		AITEST_NOT_EQUAL("Non-empty template data should result in a finalized template different from the empty one", FinalizedEmptyTemplate->GetTemplateID(), FinalizedTemplateA->GetTemplateID());
		AITEST_EQUAL("Non-empty template data should result in the very same finalized template", FinalizedTemplateA, FinalizedTemplateB);

		const FMassArchetypeHandle& ArchetypeHandle = FinalizedTemplateA->GetArchetype();
		AITEST_TRUE("Non-empty template is expected to produce a valid archetype", ArchetypeHandle.IsValid());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTemplate_Trivial, "System.Mass.EntityTemplate.Trivial");


struct FEntityTemplate_Modified : FEntityTemplateBase
{
	virtual bool InstantTest() override
	{
		UScriptStruct* IntFragmentType = FTestFragment_Int::StaticStruct();
		FMassArchetypeHandle IntsArchetype = EntityManager->CreateArchetype(MakeArrayView(&IntFragmentType, 1));

		FMassEntityTemplateData OriginalTemplateData;
		FTestFragment_Int& IntFragment = OriginalTemplateData.AddFragment_GetRef<FTestFragment_Int>();
		IntFragment.Value = FTestFragment_Int::TestIntValue;

		FMassEntityTemplateData MovedOriginalTemplateData = OriginalTemplateData;
		const TSharedRef<FMassEntityTemplate> FinalizedOriginalTemplate = TemplateRegistry.FindOrAddTemplate(FMassEntityTemplateIDFactory::Make(FGuid::NewGuid()), MoveTemp(MovedOriginalTemplateData));

		FMassEntityTemplateData NewTemplateData(*FinalizedOriginalTemplate);

		AITEST_TRUE("Template data created from a finalized template should match the composition of the original data"
			, NewTemplateData.GetCompositionDescriptor().IsEquivalent(OriginalTemplateData.GetCompositionDescriptor()));

		// need to set a new name, otherwise the hash calculated while creating finalized template will end up being the 
		// same as the original template's. This is the case since we're not using struct's contents while calculating hash (yet).
		NewTemplateData.SetTemplateName(TEXT("ModifiedTemplate"));
		FTestFragment_Int* CopiedFragment = NewTemplateData.GetMutableFragment<FTestFragment_Int>();
		AITEST_NOT_NULL("The fragment instance is expected to be found", CopiedFragment);
		AITEST_EQUAL("The fragment instance is expected to have the same value as the original one", CopiedFragment->Value, IntFragment.Value);
		CopiedFragment->Value = FTestFragment_Int::TestIntValue + 1;
		AITEST_NOT_EQUAL("Modifying the coppied instance of the fragment doesn't affect the original", CopiedFragment->Value, IntFragment.Value);

		const FMassEntityTemplateID NewTemplateID = FMassEntityTemplateIDFactory::MakeFlavor(FinalizedOriginalTemplate->GetTemplateID(), 1);
		const TSharedRef<FMassEntityTemplate> FinalizedModifiedTemplate = TemplateRegistry.FindOrAddTemplate(NewTemplateID, MoveTemp(NewTemplateData));

		AITEST_NOT_EQUAL("The original and modified templates should end up resulting in two different templates", FinalizedModifiedTemplate->GetTemplateID(), FinalizedOriginalTemplate->GetTemplateID());
		AITEST_EQUAL("The original and modified templates should still point at the same archetype", FinalizedModifiedTemplate->GetArchetype(), FinalizedOriginalTemplate->GetArchetype());

		AITEST_EQUAL("The resulting archetype should match the IntArchetypeHandle", FinalizedModifiedTemplate->GetArchetype(), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTemplate_Modified, "System.Mass.EntityTemplate.Modification");


//struct FEntityTemplate_Modified : FEntityTemplateBase
//{
//	virtual bool InstantTest() override
//	{
//		FMassEntityTemplateData OriginalTemplateData;
//		OriginalTemplateData.AddFragment<FTestFragment_Int>();
//
//		FMassEntityTemplateData OriginalTemplateDataCopy = OriginalTemplateData;
//		const TSharedRef<FMassEntityTemplate> FinalizedOriginalTemplate = TemplateRegistry.FindOrAddTemplate(MoveTemp(OriginalTemplateData));
//		
//
//		AITEST_NOT_EQUAL("Non-empty template data should result in a finalized template different from the empty one", FinalizedEmptyTemplate->GetTemplateID(), FinalizedTemplateA->GetTemplateID());
//		AITEST_EQUAL("Non-empty template data should result in the very same finalized template", FinalizedTemplateA, FinalizedTemplateB);
//
//		const FMassArchetypeHandle& ArchetypeHandle = FinalizedTemplateA->GetArchetype();
//		AITEST_TRUE("Non-empty template is expected to produce a valid archetype", ArchetypeHandle.IsValid());*/
//
//		return true;
//	}
//};
//IMPLEMENT_AI_INSTANT_TEST(FEntityTemplate_Modified, "System.Mass.EntityTemplate.Modification");


struct FEntityTest_BatchSetMissingFragment : FEntityTemplateBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumToSpawn = 3;

		FMassEntityTemplateData TemplateData;
		TemplateData.AddFragment<FTestFragment_Int>();
		TemplateData.AddFragment_GetRef<FTestFragment_Float>().Value = 3.14f;
		TemplateData.Sort();

		const FMassEntityTemplateID TemplateID = FMassEntityTemplateIDFactory::Make(FGuid::NewGuid());
		const TSharedRef<FMassEntityTemplate> EntityTemplate = TemplateRegistry.FindOrAddTemplate(TemplateID, MoveTemp(TemplateData));

		TArray<FMassEntityHandle> CreatedEntities;
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext
			= EntityManager->BatchCreateEntities(EntityTemplate->GetArchetype(), EntityTemplate->GetSharedFragmentValues(), NumToSpawn, CreatedEntities);

		// forcing archetype change for the created entities. 
		EntityManager->BatchChangeFragmentCompositionForEntities(CreationContext->GetEntityCollections(*EntityManager.Get())
			, {}, FMassFragmentBitSet(FTestFragment_Float::StaticStruct()));

		{
			MASS_SCOPED_ENSURE_TEST("Attempting to set value of fragment of type", 1);
			TConstArrayView<FInstancedStruct> FragmentInstances = EntityTemplate->GetInitialFragmentValues();
			EntityManager->BatchSetEntityFragmentValues(CreationContext->GetEntityCollections(*EntityManager.Get()), FragmentInstances);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_BatchSetMissingFragment, "System.Mass.EntityTemplate.BatchSetMissingFragment");

} // FMassEntityTemplateTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
