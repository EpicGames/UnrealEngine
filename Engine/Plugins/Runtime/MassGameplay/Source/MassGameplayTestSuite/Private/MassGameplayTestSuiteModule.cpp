// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassGameplayTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "MassTest"

class FMassGameplayTestSuiteModule : public IMassGameplayTestSuiteModule
{
};

IMPLEMENT_MODULE(FMassGameplayTestSuiteModule, MassGameplayTestSuite)

#undef LOCTEXT_NAMESPACE
