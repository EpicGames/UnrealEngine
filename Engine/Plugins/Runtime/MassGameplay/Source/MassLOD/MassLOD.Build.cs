// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassLOD : ModuleRules
	{
		public MassLOD(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassSimulation",
					"MassSpawner"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
