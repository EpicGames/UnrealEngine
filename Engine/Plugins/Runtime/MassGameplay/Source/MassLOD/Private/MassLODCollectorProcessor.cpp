// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODCollectorProcessor.h"
#include "MassLODUtils.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"
#include "MassSimulationLOD.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLODCollectorProcessor)

UMassLODCollectorProcessor::UMassLODCollectorProcessor()
{
	bAutoRegisterWithProcessingPhases = false;

	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LODCollector;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SyncWorldToMass);
}

void UMassLODCollectorProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseQuery(EntityManager);
	BaseQuery.AddTagRequirement<FMassCollectLODViewerInfoTag>(EMassFragmentPresence::All);
	BaseQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadWrite);
	BaseQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	BaseQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	BaseQuery.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		return FMassVisualizationChunkFragment::IsChunkHandledThisFrame(Context)
			|| FMassSimulationVariableTickChunkFragment::IsChunkHandledThisFrame(Context);
	});

	EntityQuery_VisibleRangeAndOnLOD = BaseQuery;
	EntityQuery_VisibleRangeAndOnLOD.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	EntityQuery_VisibleRangeAndOnLOD.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery_VisibleRangeAndOnLOD.RegisterWithProcessor(*this);

	EntityQuery_VisibleRangeOnly = BaseQuery;
	EntityQuery_VisibleRangeOnly.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	EntityQuery_VisibleRangeOnly.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	EntityQuery_VisibleRangeOnly.RegisterWithProcessor(*this);

	EntityQuery_OnLODOnly = BaseQuery;
	EntityQuery_OnLODOnly.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	EntityQuery_OnLODOnly.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery_OnLODOnly.RegisterWithProcessor(*this);

	EntityQuery_NotVisibleRangeAndOffLOD = BaseQuery;
	EntityQuery_NotVisibleRangeAndOffLOD.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	EntityQuery_NotVisibleRangeAndOffLOD.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	EntityQuery_NotVisibleRangeAndOffLOD.RegisterWithProcessor(*this);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

template <bool bLocalViewersOnly>
void UMassLODCollectorProcessor::CollectLODForChunk(FMassExecutionContext& Context)
{
	TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
	TArrayView<FMassViewerInfoFragment> ViewerInfoList = Context.GetMutableFragmentView<FMassViewerInfoFragment>();

	Collector.CollectLODInfo<FTransformFragment, FMassViewerInfoFragment, bLocalViewersOnly, true/*bCollectDistanceToFrustum*/>(Context, LocationList, ViewerInfoList);
}

template <bool bLocalViewersOnly>
void UMassLODCollectorProcessor::ExecuteInternal(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Close);
		EntityQuery_VisibleRangeAndOnLOD.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
		EntityQuery_VisibleRangeOnly.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
		EntityQuery_OnLODOnly.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Far);
		EntityQuery_NotVisibleRangeAndOffLOD.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
	}
}

void UMassLODCollectorProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();
	Collector.PrepareExecution(Viewers);

	UWorld* World = EntityManager.GetWorld();
	check(World);
	if (World->GetNetMode() != NM_Client)
	{
		ExecuteInternal<false/*bLocalViewersOnly*/>(EntityManager, Context);
	}
	else
	{
		ExecuteInternal<true/*bLocalViewersOnly*/>(EntityManager, Context);
	}

}
