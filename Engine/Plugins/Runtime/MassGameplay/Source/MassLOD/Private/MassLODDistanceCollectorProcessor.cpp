// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODDistanceCollectorProcessor.h"
#include "MassLODUtils.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"
#include "MassSimulationLOD.h"


//-----------------------------------------------------------------------------
// UMassLODDistanceCollectorProcessor
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLODDistanceCollectorProcessor)
UMassLODDistanceCollectorProcessor::UMassLODDistanceCollectorProcessor()
{
	bAutoRegisterWithProcessingPhases = true;

	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LODCollector;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SyncWorldToMass);
}

void UMassLODDistanceCollectorProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseQuery(EntityManager);
	BaseQuery.AddTagRequirement<FMassCollectDistanceLODViewerInfoTag>(EMassFragmentPresence::All);
	BaseQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadWrite);
	BaseQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	BaseQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	BaseQuery.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		return FMassVisualizationChunkFragment::IsChunkHandledThisFrame(Context)
			|| FMassSimulationVariableTickChunkFragment::IsChunkHandledThisFrame(Context);
	});

	EntityQuery_RelevantRangeAndOnLOD = BaseQuery;
	EntityQuery_RelevantRangeAndOnLOD.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	EntityQuery_RelevantRangeAndOnLOD.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery_RelevantRangeAndOnLOD.RegisterWithProcessor(*this);

	EntityQuery_RelevantRangeOnly = BaseQuery;
	EntityQuery_RelevantRangeOnly.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	EntityQuery_RelevantRangeOnly.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	EntityQuery_RelevantRangeOnly.RegisterWithProcessor(*this);

	EntityQuery_OnLODOnly = BaseQuery;
	EntityQuery_OnLODOnly.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	EntityQuery_OnLODOnly.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery_OnLODOnly.RegisterWithProcessor(*this);

	EntityQuery_NotRelevantRangeAndOffLOD = BaseQuery;
	EntityQuery_NotRelevantRangeAndOffLOD.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	EntityQuery_NotRelevantRangeAndOffLOD.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	EntityQuery_NotRelevantRangeAndOffLOD.RegisterWithProcessor(*this);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

template <bool bLocalViewersOnly>
void UMassLODDistanceCollectorProcessor::CollectLODForChunk(FMassExecutionContext& Context)
{
	TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
	TArrayView<FMassViewerInfoFragment> ViewerInfoList = Context.GetMutableFragmentView<FMassViewerInfoFragment>();

	Collector.CollectLODInfo<FTransformFragment, FMassViewerInfoFragment, bLocalViewersOnly, false/*bCollectDistanceToFrustum*/>(Context, LocationList, ViewerInfoList);
}

template <bool bLocalViewersOnly>
void UMassLODDistanceCollectorProcessor::ExecuteInternal(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LODDistanceCollector_Close);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LODDistanceCollector_Close_RelevantRangeAndOnLOD);
			EntityQuery_RelevantRangeAndOnLOD.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LODDistanceCollector_Close_OnLODOnly);
			EntityQuery_RelevantRangeOnly.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LODDistanceCollector_Close_OnLODOnly);
			EntityQuery_OnLODOnly.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LODDistanceCollector_Far);
		EntityQuery_NotRelevantRangeAndOffLOD.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) { CollectLODForChunk<bLocalViewersOnly>(Context); });
	}
}

void UMassLODDistanceCollectorProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();
	Collector.PrepareExecution(Viewers);

	UWorld* World = EntityManager.GetWorld();
	check(World);
	if (World->GetNetMode() != NM_Client)
	{
		ExecuteInternal<false/*bLocalViewersOnly*/>(EntityManager, Context);
	}
	else
	{
		ExecuteInternal<true/*bLocalViewersOnly*/>(EntityManager, Context);
	}
}
