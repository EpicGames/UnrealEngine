// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODLogic.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

void FMassLODBaseLogic::CacheViewerInformation(TConstArrayView<FViewerInfo> ViewerInfos)
{
	if(Viewers.Num() < ViewerInfos.Num())
	{
		Viewers.AddDefaulted(ViewerInfos.Num() - Viewers.Num());
	}

	// Cache viewer info
	for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
	{
		const FViewerInfo& Viewer =  ViewerInfos.IsValidIndex(ViewerIdx) ? ViewerInfos[ViewerIdx] : FViewerInfo();
		const FMassViewerHandle ViewerHandle =  Viewer.bEnabled ? Viewer.Handle : FMassViewerHandle();

		// Check if it is the same client as before
		FViewerLODInfo& ViewerLOD = Viewers[ViewerIdx];
		ViewerLOD.bClearData = Viewers[ViewerIdx].Handle != ViewerHandle;
		ViewerLOD.Handle = ViewerHandle;
		ViewerLOD.bLocal = Viewer.IsLocal();

		if (ViewerHandle.IsValid())
		{
			ViewerLOD.Location = Viewer.Location;
			ViewerLOD.Direction = Viewer.Rotation.Vector();

			if (bBuildFrustumData)
			{
				const float HalfHorizontalFOVAngle = Viewer.FOV * 0.5f;
				const float HalfVerticalFOVAngle = FMath::RadiansToDegrees(FMath::Atan(FMath::Tan(FMath::DegreesToRadians(HalfHorizontalFOVAngle)) / Viewer.AspectRatio));

				const FVector RightPlaneNormal = Viewer.Rotation.RotateVector(FRotator(0.0f, HalfHorizontalFOVAngle, 0.0f).RotateVector(FVector::RightVector));
				const FVector LeftPlaneNormal = Viewer.Rotation.RotateVector(FRotator(0.0f, -HalfHorizontalFOVAngle, 0.0f).RotateVector(FVector::LeftVector));
				const FVector TopPlaneNormal = Viewer.Rotation.RotateVector(FRotator(HalfVerticalFOVAngle, 0.0f, 0.0f).RotateVector(FVector::UpVector));
				const FVector BottomPlaneNormal = Viewer.Rotation.RotateVector(FRotator(-HalfVerticalFOVAngle, 0.0f, 0.0f).RotateVector(FVector::DownVector));

				TArray<FPlane, TInlineAllocator<6>> Planes;
				Planes.Emplace(ViewerLOD.Location, RightPlaneNormal);
				Planes.Emplace(ViewerLOD.Location, LeftPlaneNormal);
				Planes.Emplace(ViewerLOD.Location, TopPlaneNormal);
				Planes.Emplace(ViewerLOD.Location, BottomPlaneNormal);

				ViewerLOD.Frustum = FConvexVolume(Planes);
			}
		}
	}
}