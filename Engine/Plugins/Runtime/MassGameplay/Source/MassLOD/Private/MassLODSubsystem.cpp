// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODSubsystem.h"
#include "MassLODTypes.h"
#include "GameFramework/PlayerController.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "Engine/World.h"
#include "MassSimulationSubsystem.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Pawn.h"
#include "SceneView.h"
#if WITH_EDITOR
#include "CoreGlobals.h" // GIsEditor
#include "Editor.h" // GEditor
#include "LevelEditorViewport.h"
#include "Editor/EditorEngine.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLODSubsystem)

namespace UE::Mass::ProcessorGroupNames
{
	const FName LODCollector = TEXT("LODCollector");
	const FName LOD = TEXT("LOD");
}

namespace UE::MassLOD
{
	FColor LODColors[] =
	{
		FColor::Red,
		FColor::Yellow,
		FColor::Emerald,
		FColor::White,
	};
	
	namespace Tweakables
	{

		bool bLODSubsystemIncludeAllPlayerControllers = true;
		namespace
		{
			static FAutoConsoleVariableRef AnonymousCVars[] = {
				{ TEXT("mass.LODSubsystem.IncludeAllPlayerControllers"), bLODSubsystemIncludeAllPlayerControllers, TEXT("Include all player controllers, even those without a camera or pawn."), ECVF_Default }
			};
		}
	}  // UE::Mass::Tweakables

#if WITH_MASSGAMEPLAY_DEBUG
	namespace Debug
	{
		/** Returns whether getting the UMassLODSubsystem and the bool parameter was successful */
		bool GetSubsystemAndBoolArgument(const TArray<FString>& Args, UWorld* World, UMassLODSubsystem*& OutMassLODSubsystem, bool& bOutBool)
		{
			if (!World)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: invalid world");
				return false;
			}

			OutMassLODSubsystem = World->GetSubsystem<UMassLODSubsystem>();
			if (OutMassLODSubsystem == nullptr)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: Unable to fetch MassLODSubsystem instance");
				return false;
			}

			if (Args.Num() < 1)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: Expecting 1 parameter");
				return false;
			}

			if (!LexTryParseString<bool>(bOutBool, *Args[0]))
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: parameter must be an integer or a boolean");
				return false;
			}

			return true;
		}

		FAutoConsoleCommandWithWorldArgsAndOutputDevice ToggleUsePlayerLocationCmd(
			TEXT("mass.debug.LODSubsystem.UsePlayerLocation"),
			TEXT("Sets UMassLODSubsystem::bUsePlayerPawnLocationInsteadOfCamera. Note that this is a command that doesn't retain state and usually needs running both for the client and the server"),
			FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
				{
					UMassLODSubsystem* MassLODSubsystem = nullptr;
					bool bNewValue = false;

					if (GetSubsystemAndBoolArgument(Args, World, MassLODSubsystem, bNewValue))
					{
						MassLODSubsystem->DebugSetUsePlayerPawnLocationInsteadOfCamera(bNewValue);
					}
				}));

		FAutoConsoleCommandWithWorldArgsAndOutputDevice ToggleGatherPlayers(
			TEXT("mass.debug.LODSubsystem.GatherPlayers"),
			TEXT("Sets UMassLODSubsystem::bGatherPlayerControllers. Note that this is a command that doesn't retain state and usually needs running both for the client and the server"),
			FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
				{
					UMassLODSubsystem* MassLODSubsystem = nullptr;
					bool bNewValue = false;

					if (GetSubsystemAndBoolArgument(Args, World, MassLODSubsystem, bNewValue))
					{
						MassLODSubsystem->DebugSetGatherPlayers(bNewValue);
					}
				})); 
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}

//-----------------------------------------------------------------------------
// UMassLODSubsystem
//-----------------------------------------------------------------------------
void UMassLODSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency(UMassSimulationSubsystem::StaticClass());

	Super::Initialize(Collection);
	
	if (UWorld* World = GetWorld())
	{
		UMassSimulationSubsystem* SimSystem = World->GetSubsystem<UMassSimulationSubsystem>();
		check(SimSystem);
		SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassLODSubsystem::OnPrePhysicsPhaseStarted);
#if WITH_EDITOR
		bIgnorePlayerControllersDueToSimulation = (GEditor && GEditor->IsSimulateInEditorInProgress());
		bUseEditorLevelViewports = bIgnorePlayerControllersDueToSimulation || (GIsEditor && World->WorldType == EWorldType::Editor);
#endif // WITH_EDITOR
	}

	SynchronizeViewers();
}

void UMassLODSubsystem::OnPrePhysicsPhaseStarted(float DeltaTime)
{
	SynchronizeViewers();
}

void UMassLODSubsystem::Deinitialize()
{
	// Remove all viewers from the viewer info list
	for (FViewerInfo& ViewerInfo : Viewers)
	{
		if (ViewerInfo.Handle.IsValid())
		{
			// Safe to remove while iterating as it is a sparse array with a free list
			RemoveViewer(ViewerInfo.Handle);
		}
	}
	
	if (UWorld* World = GetWorld())
	{
		if (UMassSimulationSubsystem* SimSystem = World->GetSubsystem<UMassSimulationSubsystem>())
		{
			SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).RemoveAll(this);
		}
	}

	Super::Deinitialize();
}

int32 UMassLODSubsystem::GetValidViewerIdx(const FMassViewerHandle& ViewerHandle) const
{
	// Check against invalid handle
	if (!ViewerHandle.IsValid())
	{
		return INDEX_NONE;
	}

	// Check against removed viewers
	const int32 ViewerIdx = ViewerHandle.GetIndex();
	if (ViewerIdx >= Viewers.Num() || ViewerHandle != Viewers[ViewerIdx].Handle)
	{
		return INDEX_NONE;
	}

	return ViewerIdx;
}

const TArray<FViewerInfo>& UMassLODSubsystem::GetSynchronizedViewers()
{
	SynchronizeViewers();

	return Viewers;
}

FMassViewerHandle UMassLODSubsystem::GetViewerHandleFromActor(const AActor& Actor) const
{
	const FMassViewerHandle* Handle = ViewerMap.Find(GetTypeHash(Actor.GetFName()));
	return Handle ? *Handle : FMassViewerHandle();
}


FMassViewerHandle UMassLODSubsystem::GetViewerHandleFromStreamingSource(const FName StreamingSourceName) const
{
	const FMassViewerHandle* Handle = ViewerMap.Find(GetTypeHash(StreamingSourceName));
	return Handle ? *Handle : FMassViewerHandle();
}

APlayerController* UMassLODSubsystem::GetPlayerControllerFromViewerHandle(const FMassViewerHandle& ViewerHandle) const
{
	const int32 ViewerIdx = GetValidViewerIdx(ViewerHandle);
	return ViewerIdx != INDEX_NONE ? Viewers[ViewerIdx].GetPlayerController() : nullptr;
}

void UMassLODSubsystem::SynchronizeViewers()
{
	if (LastSynchronizedFrame == GFrameCounter)
	{
		return;
	}
	LastSynchronizedFrame = GFrameCounter;

	bool bNeedShrinking = false;

	const UWorld* World = GetWorld();
	UWorldPartition* WorldPartition = World ? World->GetWorldPartition() : nullptr;
	const TArray<FWorldPartitionStreamingSource>& StreamingSources = WorldPartition ? WorldPartition->GetStreamingSources() : TArray<FWorldPartitionStreamingSource>();

	// Go through the list and check validity and store the valid one into a map
	TMap<uint32, FMassViewerHandle> LocalViewerMap;
	for (int32 ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
	{
		FViewerInfo& ViewerInfo = Viewers[ViewerIdx];
		if (!ViewerInfo.Handle.IsValid())
		{
			check(ViewerFreeIndices.Find(ViewerIdx) != INDEX_NONE);
			continue;
		}

		APlayerController* ViewerAsPlayerController = ViewerInfo.GetPlayerController();
		if (ViewerAsPlayerController != nullptr
#if WITH_EDITOR
			&& bIgnorePlayerControllersDueToSimulation == false
#endif // WITH_EDITOR
			)
		{
			LocalViewerMap.Add(GetTypeHash(ViewerAsPlayerController->GetFName()), ViewerInfo.Handle);
		}
		else if (!ViewerInfo.StreamingSourceName.IsNone() && StreamingSources.FindByPredicate([&ViewerInfo](const FWorldPartitionStreamingSource& Source){ return Source.Name == ViewerInfo.StreamingSourceName; }) != nullptr)
		{
			LocalViewerMap.Add(GetTypeHash(ViewerInfo.StreamingSourceName), ViewerInfo.Handle);
		}
#if WITH_EDITOR
		else if (bUseEditorLevelViewports && ViewerInfo.EditorViewportClientIndex != INDEX_NONE
			&& GEditor && GEditor->GetLevelViewportClients().IsValidIndex(ViewerInfo.EditorViewportClientIndex)
			&& GEditor->GetLevelViewportClients()[ViewerInfo.EditorViewportClientIndex])
		{
			
			const int32 HashValue = GetTypeHash(GEditor->GetLevelViewportClients()[ViewerInfo.EditorViewportClientIndex]);
			LocalViewerMap.Add(HashValue, ViewerInfo.Handle);
		}
#endif // WITH_EDITOR
		else
		{
			// Safe to remove while iterating as it is a sparse array with a free list
			RemoveViewer(ViewerInfo.Handle);
			bNeedShrinking |= ViewerIdx == Viewers.Num() - 1;
		}
	}

	if (World)
	{
		if (bGatherPlayerControllers)
		{
			// Now go through all current player controllers and add if they do not exist
			for (FConstPlayerControllerIterator PlayerIterator = World->GetPlayerControllerIterator(); PlayerIterator; ++PlayerIterator)
			{
				APlayerController* PlayerController = (*PlayerIterator).Get();
				check(PlayerController);

				// Check if the controller already exists by trying to remove it from the map which was filled up with controllers we were tracking
				if (LocalViewerMap.Remove(GetTypeHash(PlayerController->GetFName())) == 0)
				{
					// If not add it to the list
					AddPlayerViewer(*PlayerController);
				}
			}
		}

		if (bGatherStreamingSources)
		{
			// Now go through all current streaming source and add if they do not exist
			for (const FWorldPartitionStreamingSource& StreamingSource : StreamingSources)
			{
				if (LocalViewerMap.Remove(GetTypeHash(StreamingSource.Name)) == 0)
				{
					AddStreamingSourceViewer(StreamingSource.Name);
				}
			}
		}

		if (bAllowNonPlayerViwerActors)
		{
			for (int32 ActorViewerIndex = RegisteredActorViewers.Num() - 1; ActorViewerIndex >= 0; --ActorViewerIndex)
			{
				if (RegisteredActorViewers[ActorViewerIndex])
				{
					AddActorViewer(*RegisteredActorViewers[ActorViewerIndex]);
				}
				else
				{
					RegisteredActorViewers.RemoveAtSwap(ActorViewerIndex, EAllowShrinking::No);
				}
			}
		}
	}
#if WITH_EDITOR
	if (bUseEditorLevelViewports)
	{
		CA_ASSUME(GEditor);
		for (int32 ClientIndex = 0; ClientIndex < GEditor->GetLevelViewportClients().Num(); ++ClientIndex)
		{
			const FLevelEditorViewportClient* LevelVC = GEditor->GetLevelViewportClients()[ClientIndex];
			if (LevelVC && LevelVC->IsPerspective())
			{
				const int32 HashValue = GetTypeHash(LevelVC);
				if (LocalViewerMap.Remove(HashValue) == 0)
				{
					AddEditorViewer(HashValue, ClientIndex);
				}
			}
		}
	}
	
#endif // WITH_EDITOR

	// Anything left in the map need to be removed from the list
	for (TMap<uint32, FMassViewerHandle>::TIterator Itr = LocalViewerMap.CreateIterator(); Itr; ++Itr)
	{
		const int32 ViewerIdx = Itr->Value.GetIndex();
		RemoveViewer(Viewers[ViewerIdx].Handle);
		bNeedShrinking |= ViewerIdx == Viewers.Num() - 1;
	}

	if (bNeedShrinking)
	{
		// Check to shrink the array of viewers if possible
		while (Viewers.Num() > 0 && ViewerFreeIndices.Num() > 0 && ViewerFreeIndices.Remove(Viewers.Num() - 1))
		{
			Viewers.Pop(EAllowShrinking::No);
		}
	}

	// Update location and direction for every type of viewer
	for (FViewerInfo& ViewerInfo : Viewers)
	{
		if (!ViewerInfo.Handle.IsValid())
		{
			continue;
		}

		if (APlayerController* ViewerAsPlayerController = ViewerInfo.GetPlayerController())
		{
			ViewerInfo.bEnabled = !WorldPartition || ViewerAsPlayerController->bEnableStreamingSource;

			// Note: Using bUsePlayerPawnLocationInsteadOfCamera will not work correctly with FOV based LOD, since the
			// camera will be at wrong location.  
			// @todo: separate "player location" and "view location", and use the player location on distance based LOD 
			// calculations for stability, and view location in FOV based LOD for view precision.
			if (bUsePlayerPawnLocationInsteadOfCamera && ViewerAsPlayerController->GetPawn())
			{
				ViewerInfo.Location = ViewerAsPlayerController->GetPawn()->GetActorLocation();
				ViewerInfo.Rotation = ViewerAsPlayerController->GetPawn()->GetActorRotation();
			}
			else
			{
				FVector PlayerCameraLocation(ForceInitToZero);
				FRotator PlayerCameraRotation(FRotator::ZeroRotator);
				ViewerAsPlayerController->GetPlayerViewPoint(PlayerCameraLocation, PlayerCameraRotation);
				ViewerInfo.Location = PlayerCameraLocation;
				ViewerInfo.Rotation = PlayerCameraRotation;
			}

			// LocalPlayer is null on a dedicated server; leave FOV / AspectRatio at their defaults.
			const ULocalPlayer* LocalPlayer = ViewerAsPlayerController->GetLocalPlayer();
			if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->Viewport)
			{
				FSceneViewProjectionData ProjectionData;
				if (LocalPlayer->GetProjectionData(LocalPlayer->ViewportClient->Viewport, ProjectionData))
				{
					const FMatrix& Proj = ProjectionData.ProjectionMatrix;
					// M[3][3] == 0 for perspective matrices, 1 for orthographic. FOV / aspect
					// recovery from M[0][0] / M[1][1] only makes sense for perspective.
					if (FMath::Abs(Proj.M[3][3]) < UE_KINDA_SMALL_NUMBER)
					{
						const double TanHalfH = (FMath::Abs(Proj.M[0][0]) > UE_KINDA_SMALL_NUMBER) ? 1.0 / FMath::Abs(Proj.M[0][0]) : 1.0;
						const double TanHalfV = (FMath::Abs(Proj.M[1][1]) > UE_KINDA_SMALL_NUMBER) ? 1.0 / FMath::Abs(Proj.M[1][1]) : 1.0;
						ViewerInfo.FOV = static_cast<float>(FMath::RadiansToDegrees(FMath::Atan(TanHalfH) * 2.0));
						ViewerInfo.AspectRatio = static_cast<float>(TanHalfH / TanHalfV);
					}
				}
			}
		}
		else if (AActor* Actor = ViewerInfo.ActorViewer.Get())
		{
			ViewerInfo.Location = Actor->GetActorLocation();
			ViewerInfo.Rotation = Actor->GetActorRotation();
		}
#if WITH_EDITOR
		else if (bUseEditorLevelViewports && ViewerInfo.EditorViewportClientIndex != INDEX_NONE)
		{
			CA_ASSUME(GEditor);
			const FLevelEditorViewportClient* LevelVC = GEditor->GetLevelViewportClients()[ViewerInfo.EditorViewportClientIndex];
			checkSlow(LevelVC);
			ViewerInfo.bEnabled = LevelVC && LevelVC->IsPerspective();
			ViewerInfo.Location = LevelVC->GetViewLocation();
			ViewerInfo.Rotation = LevelVC->GetViewRotation();
			
		}
#endif // WITH_EDITOR
		else
		{
			checkf(!ViewerInfo.StreamingSourceName.IsNone(), TEXT("Expecting to have a streamingsourcename if the playercontroller is null"));
			const FWorldPartitionStreamingSource* StreamingSource = StreamingSources.FindByPredicate([&ViewerInfo](const FWorldPartitionStreamingSource& Source) { return Source.Name == ViewerInfo.StreamingSourceName; });
			checkf(StreamingSource, TEXT("Expecting to be pointing to a valid streaming source"));
			ViewerInfo.bEnabled = StreamingSource != nullptr;
			if (StreamingSource)
			{
				ViewerInfo.Location = StreamingSource->Location;
				ViewerInfo.Rotation = StreamingSource->Rotation;
			}
		}
	}
}

void UMassLODSubsystem::AddPlayerViewer(APlayerController& PlayerController)
{
#if WITH_EDITOR
	if (bIgnorePlayerControllersDueToSimulation)
	{
		return;
	}
#endif // WITH_EDITOR

	// ignore players that don't have a pawn nor a camera
	if (UE::MassLOD::Tweakables::bLODSubsystemIncludeAllPlayerControllers == false &&
		PlayerController.GetPawn() == nullptr
		&& (bool(PlayerController.PlayerCameraManager) == false
			|| PlayerController.PlayerCameraManager->GetLastFrameCameraCacheTime() == 0.f)
		)	
	{
		return;
	}

	const int32 HashValue = GetTypeHash(PlayerController.GetFName());

	FMassViewerHandle& ViewerHandle = ViewerMap.FindOrAdd(HashValue, FMassViewerHandle());
	if (ViewerHandle.IsValid())
	{
		// We are only interested to set the player controller if it was not already set.
		const int32 ViewerHandleIdx = GetValidViewerIdx(ViewerHandle);
		check(ViewerHandleIdx != INDEX_NONE);

		FViewerInfo& ViewerInfo = Viewers[ViewerHandleIdx];
		check(ViewerInfo.ActorViewer == nullptr);
		ViewerInfo.ActorViewer = &PlayerController;
	}
	else
	{
		// Add new viewer
#if UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER
		//for debugging / profiling purposes create DebugNumberViwersPerController
		//in this case ViewerMap will only contain a hash to the most recent viewer handle created.
		for (int Idx = 0; Idx < UE::MassLOD::DebugNumberViewersPerController; ++Idx)
#endif //UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER
		{
			const bool bAddNew = ViewerFreeIndices.Num() == 0;
			const int NewIdx = bAddNew ? Viewers.Num() : ViewerFreeIndices.Pop();
			FViewerInfo& NewViewerInfo = bAddNew ? Viewers.AddDefaulted_GetRef() : Viewers[NewIdx];
			NewViewerInfo.ActorViewer = &PlayerController;
			NewViewerInfo.Handle.Index = NewIdx;
			NewViewerInfo.Handle.SerialNumber = GetNextViewerSerialNumber();
			NewViewerInfo.HashValue = HashValue;

			ViewerHandle = NewViewerInfo.Handle;

			OnViewerAddedDelegate.Broadcast(NewViewerInfo);
		}
	}

	PlayerController.OnEndPlay.AddUniqueDynamic(this, &UMassLODSubsystem::OnPlayerControllerEndPlay);
}

void UMassLODSubsystem::AddStreamingSourceViewer(const FName StreamingSourceName)
{
	const int32 HashValue = GetTypeHash(StreamingSourceName);

	FMassViewerHandle& ViewerHandle = ViewerMap.FindOrAdd(HashValue, FMassViewerHandle());
	// only add new viewer if it hasn't been added yet
	if (ViewerHandle.IsValid() == false)
	{
		const bool bAddNew = ViewerFreeIndices.Num() == 0;
		const int NewIdx = bAddNew ? Viewers.Num() : ViewerFreeIndices.Pop();
		FViewerInfo& NewViewerInfo = bAddNew ? Viewers.AddDefaulted_GetRef() : Viewers[NewIdx];
		NewViewerInfo.StreamingSourceName = StreamingSourceName;
		NewViewerInfo.Handle.Index = NewIdx;
		NewViewerInfo.Handle.SerialNumber = GetNextViewerSerialNumber();
		NewViewerInfo.HashValue = HashValue;

		ViewerHandle = NewViewerInfo.Handle;

		OnViewerAddedDelegate.Broadcast(NewViewerInfo);
	}
}

void UMassLODSubsystem::AddActorViewer(AActor& ActorViewer)
{
	// @todo we might need to use PathName instead 
	const int32 HashValue = GetTypeHash(ActorViewer.GetFName());

	FMassViewerHandle& ViewerHandle = ViewerMap.FindOrAdd(HashValue, FMassViewerHandle());
	if (ViewerHandle.IsValid())
	{
		// We are only interested to set the player controller if it was not already set.
		const int32 ViewerHandleIdx = GetValidViewerIdx(ViewerHandle);
		check(ViewerHandleIdx != INDEX_NONE);

		FViewerInfo& ViewerInfo = Viewers[ViewerHandleIdx];
		ViewerInfo.ActorViewer = &ActorViewer;
	}
	else
	{
		// Add new viewer
		const bool bAddNew = ViewerFreeIndices.Num() == 0;
		const int NewIdx = bAddNew ? Viewers.Num() : ViewerFreeIndices.Pop();

		FViewerInfo& NewViewerInfo = bAddNew ? Viewers.AddDefaulted_GetRef() : Viewers[NewIdx];
		NewViewerInfo.ActorViewer = &ActorViewer;
		NewViewerInfo.Handle.Index = NewIdx;
		NewViewerInfo.Handle.SerialNumber = GetNextViewerSerialNumber();
		NewViewerInfo.HashValue = HashValue;
		ViewerHandle = NewViewerInfo.Handle;

		OnViewerAddedDelegate.Broadcast(NewViewerInfo);
	}
}

#if WITH_EDITOR
void UMassLODSubsystem::AddEditorViewer(const int32 HashValue, const int32 ClientIndex)
{
	using ClientIndexType = decltype(FViewerInfo::EditorViewportClientIndex);
	if (!ensureMsgf(ClientIndex >= std::numeric_limits<ClientIndexType>::min() && ClientIndex <= std::numeric_limits<ClientIndexType>::max()
		, TEXT("Editor viewport client index %d does not fit in storage type (range %d..%d); skipping LOD viewer registration.")
		, ClientIndex, int32(std::numeric_limits<ClientIndexType>::min()), int32(std::numeric_limits<ClientIndexType>::max())))
	{
		return;
	}

	FMassViewerHandle& ViewerHandle = ViewerMap.FindOrAdd(HashValue, FMassViewerHandle());
	// only add new viewer if it hasn't been added yet
	if (ViewerHandle.IsValid() == false)
	{
		const bool bAddNew = ViewerFreeIndices.Num() == 0;
		const int NewIdx = bAddNew ? Viewers.Num() : ViewerFreeIndices.Pop();
		FViewerInfo& NewViewerInfo = bAddNew ? Viewers.AddDefaulted_GetRef() : Viewers[NewIdx];

		NewViewerInfo.EditorViewportClientIndex = static_cast<ClientIndexType>(ClientIndex);
		NewViewerInfo.Handle.Index = NewIdx;
		NewViewerInfo.Handle.SerialNumber = GetNextViewerSerialNumber();
		NewViewerInfo.HashValue = HashValue;

		ViewerHandle = NewViewerInfo.Handle;

		OnViewerAddedDelegate.Broadcast(NewViewerInfo);
	}
}
#endif // WITH_EDITOR

void UMassLODSubsystem::RemoveViewer(const FMassViewerHandle& ViewerHandle)
{
#if UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER

	const int32 ViewerHandleIdx = GetValidViewerIdx(ViewerHandle);
	check(ViewerHandleIdx != INDEX_NONE);

	//find all the viewer handles the slow way and remove them
	for (int32 ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
	{
		FViewerInfo& ViewerInfo = Viewers[ViewerIdx];

		if (ViewerInfo.PlayerController == Viewers[ViewerHandleIdx].PlayerController)
		{
			RemoveViewerInternal(ViewerInfo.Handle);
		}
	}

#else

	RemoveViewerInternal(ViewerHandle);

#endif //UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER
}

void UMassLODSubsystem::RemoveViewerInternal(const FMassViewerHandle& ViewerHandle)
{
	const int32 ViewerIdx = GetValidViewerIdx(ViewerHandle);
	check(ViewerIdx != INDEX_NONE);
	FViewerInfo& ViewerInfo = Viewers[ViewerIdx];

	OnViewerRemovedDelegate.Broadcast(ViewerInfo);

	if (APlayerController* ViewerAsPlayerController = ViewerInfo.GetPlayerController())
	{
		ViewerAsPlayerController->OnEndPlay.RemoveDynamic(this, &UMassLODSubsystem::OnPlayerControllerEndPlay);
	}

	ViewerMap.Remove(ViewerInfo.HashValue);

	ViewerInfo.Reset();
	ViewerFreeIndices.Push(ViewerIdx);
}

void UMassLODSubsystem::OnPlayerControllerEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason)
{
	APlayerController* PlayerController = Cast<APlayerController>(Actor);
	if (ensure(PlayerController))
	{
		const FMassViewerHandle ViewerHandle = GetViewerHandleFromActor(*PlayerController);
		if (ensure(ViewerHandle.IsValid()))
		{
			RemoveViewer(ViewerHandle);
		}
	}
}

void UMassLODSubsystem::RegisterActorViewer(AActor& ActorViewer)
{
	RegisteredActorViewers.AddUnique(&ActorViewer);
}

void UMassLODSubsystem::UnregisterActorViewer(AActor& ActorViewer)
{
	if (RegisteredActorViewers.RemoveSingleSwap(&ActorViewer, EAllowShrinking::No))
	{
		const FMassViewerHandle ViewerHandle = GetViewerHandleFromActor(ActorViewer);
		if (ensure(ViewerHandle.IsValid()))
		{
			RemoveViewer(ViewerHandle);
		}
	}
}

#if WITH_MASSGAMEPLAY_DEBUG
void UMassLODSubsystem::DebugUnregisterActorViewer()
{
	for (const TObjectPtr<AActor>& RegisteredActorViewer : RegisteredActorViewers)
	{
		if (const AActor* ActorViewer = RegisteredActorViewer)
		{
			const FMassViewerHandle ViewerHandle = GetViewerHandleFromActor(*ActorViewer);
			if (ensure(ViewerHandle.IsValid()))
			{
				RemoveViewer(ViewerHandle);
			}
		}
	}
	RegisteredActorViewers.Reset();
}
#endif // WITH_MASSGAMEPLAY_DEBUG

//-----------------------------------------------------------------------------
// FViewerInfo
//-----------------------------------------------------------------------------
void FViewerInfo::Reset()
{
	Handle.Invalidate();
	ActorViewer = nullptr;
#if WITH_EDITOR
	EditorViewportClientIndex = INDEX_NONE;
#endif // WITH_EDITOR
	HashValue = 0;
}

bool FViewerInfo::IsLocal() const
{
	APlayerController* ViewerAsPlayerController = GetPlayerController();
	return (ViewerAsPlayerController && ViewerAsPlayerController->IsLocalController()) || !StreamingSourceName.IsNone()
#if WITH_EDITOR
		|| EditorViewportClientIndex != INDEX_NONE
#endif // WITH_EDITOR
		;
}

APlayerController* FViewerInfo::GetPlayerController() const
{
	return Cast<APlayerController>(ActorViewer.Get());
}
