// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "MassLODFragments.h"
#include "MassEntityUtils.h"
#include "MassLODCollectorProcessor.h"
#include "MassLODDistanceCollectorProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLODTrait)


namespace UE::Mass::Private
{
	bool ValidateRequiredProcessor(const TNotNull<const UMassEntityTraitBase*> Trait, const TSubclassOf<UMassProcessor>& ProcessorClass)
	{
		check(ProcessorClass);

		bool bValidCollectorActive = false;

		// @todo make this processor class configurable. Could be something like:
		// * every processor declared "identifying tag", a unique combination that would identify given processor class
		// * traits can add "required processors"
		// * tags of required processors are added automatically
		// * we can also report required processors that are marked as not auto-added.
		//	-- downside of that last, similar to the one bellow, is that disabling a given processor might be deliberate and it can still be enabled at runtime
		const UMassProcessor* LODCollectorProcessor = ProcessorClass->GetDefaultObject<UMassProcessor>();
		if (ensureMsgf(LODCollectorProcessor, TEXT("Failed to fetch CDO from class %s"), *ProcessorClass->GetName()))
		{
			if (LODCollectorProcessor->ShouldAutoAddToGlobalList())
			{
				bValidCollectorActive = true;
			}
			else
			{
				// look for subclasses
				TArray<UClass*> Results;
				GetDerivedClasses(ProcessorClass, Results, /*bRecursive=*/true);
				for (UClass* Class : Results)
				{
					check(Class);
					const UMassProcessor* LODCollectorSubclassProcessor = Class->GetDefaultObject<UMassProcessor>();
					check(LODCollectorSubclassProcessor);
					if (LODCollectorSubclassProcessor->ShouldAutoAddToGlobalList())
					{
						bValidCollectorActive = true;
						break;
					}
				}

				if (bValidCollectorActive == false)
				{
					TStringBuilder<256> SubclassesStringBuilder;
					if (Results.Num() > 0)
					{
						SubclassesStringBuilder << ", nor any of its subclasses: ";
						for (UClass* Class : Results)
						{
							SubclassesStringBuilder << GetNameSafe(Class) << ", ";
						}
					}

					UE_LOGF(LogMassLOD, Error, "Using %ls trait while the required processor %ls%ls is not enabled by default"
						, *Trait->GetName(), *GetNameSafe(LODCollectorProcessor->GetClass()), SubclassesStringBuilder.ToString());
				}
			}
		}
		
		return bValidCollectorActive;
	}
}

using UE::Mass::Private::ValidateRequiredProcessor;

//-----------------------------------------------------------------------------
// UMassLODCollectorTrait
//-----------------------------------------------------------------------------
void UMassLODCollectorTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassViewerInfoFragment>();
	BuildContext.AddTag<FMassCollectLODViewerInfoTag>();
	BuildContext.RequireFragment<FTransformFragment>();
}

bool UMassLODCollectorTrait::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	// if enabled, we require MassLODCollectorProcessor to be enabled.
	if (bTestCollectorProcessor && !ValidateRequiredProcessor(this, UMassLODCollectorProcessor::StaticClass()))
	{
		return false;
	}

	return Super::ValidateTemplate(BuildContext, World, OutTraitRequirements);
}

//-----------------------------------------------------------------------------
// UMassDistanceLODCollectorTrait
//-----------------------------------------------------------------------------
void UMassDistanceLODCollectorTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassViewerInfoFragment>();
	BuildContext.AddTag<FMassCollectDistanceLODViewerInfoTag>();
	BuildContext.RequireFragment<FTransformFragment>();
}

bool UMassDistanceLODCollectorTrait::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	// if enabled, we require UMassLODDistanceCollectorProcessor to be enabled.
	if (bTestCollectorProcessor && !ValidateRequiredProcessor(this, UMassLODDistanceCollectorProcessor::StaticClass()))
	{
		return false;
	}

	return Super::ValidateTemplate(BuildContext, World, OutTraitRequirements);
}

//-----------------------------------------------------------------------------
// UMassSimulationLODTrait
//-----------------------------------------------------------------------------
void UMassSimulationLODTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FMassViewerInfoFragment>();
	BuildContext.RequireFragment<FTransformFragment>();

	FMassSimulationLODFragment& LODFragment = BuildContext.AddFragment_GetRef<FMassSimulationLODFragment>();

	// Start all simulation LOD in the Off 
	if (Params.bSetLODTags || bEnableVariableTicking || BuildContext.IsInspectingData())
	{
		LODFragment.LOD = EMassLOD::Off;
		BuildContext.AddTag<FMassOffLODTag>();
	}

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	FConstSharedStruct ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(Params);
	BuildContext.AddConstSharedFragment(ParamsFragment);

	FSharedStruct SharedFragment = EntityManager.GetOrCreateSharedFragment<FMassSimulationLODSharedFragment>(FConstStructView::Make(Params), Params);
	BuildContext.AddSharedFragment(SharedFragment);

	// Variable ticking from simulation LOD
	if (bEnableVariableTicking || BuildContext.IsInspectingData())
	{
		BuildContext.AddFragment<FMassSimulationVariableTickFragment>();
		BuildContext.AddChunkFragment<FMassSimulationVariableTickChunkFragment>();

		FConstSharedStruct VariableTickParamsFragment = EntityManager.GetOrCreateConstSharedFragment(VariableTickParams);
		BuildContext.AddConstSharedFragment(VariableTickParamsFragment);

		FSharedStruct VariableTickSharedFragment = EntityManager.GetOrCreateSharedFragment<FMassSimulationVariableTickSharedFragment>(FConstStructView::Make(VariableTickParams), VariableTickParams);
		BuildContext.AddSharedFragment(VariableTickSharedFragment);
	}
}
