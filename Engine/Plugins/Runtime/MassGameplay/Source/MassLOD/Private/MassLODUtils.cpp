// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLODUtils.h"
#include "MassCommandBuffer.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY(LogMassLOD);
#if WITH_MASSGAMEPLAY_DEBUG
namespace UE::MassLOD::Debug
{
	bool bLODCalculationsPaused = false;
	
	FAutoConsoleVariableRef CVarLODPause(TEXT("mass.lod.pause"), bLODCalculationsPaused, TEXT("If non zero will pause all LOD calculations"));
} // UE::MassLOD::Debug
#endif // WITH_MASSGAMEPLAY_DEBUG

namespace UE::MassLOD
{
void PushSwapTagsCommand(FMassCommandBuffer& CommandBuffer, const FMassEntityHandle Entity, const EMassLOD::Type PrevLOD, const EMassLOD::Type NewLOD)
{
#define CASE_SWAP_TAGS(OldLOD, NewLOD) \
case NewLOD: \
	CommandBuffer.SwapTags<TMassLODTagForLevel<OldLOD>::FTag, TMassLODTagForLevel<NewLOD>::FTag>(Entity); \
	break

#define CASE_ADD_TAG(NewLOD) \
case NewLOD: \
	CommandBuffer.AddTag<TMassLODTagForLevel<NewLOD>::FTag>(Entity); \
	break

#define DEFAULT_REMOVE_TAG(OldLOD) \
case EMassLOD::Max: /* fall through on purpose */ \
default: \
	CommandBuffer.RemoveTag<TMassLODTagForLevel<OldLOD>::FTag>(Entity); \
	break

	check(PrevLOD != NewLOD);

	switch (PrevLOD)
	{
	case EMassLOD::High:
		switch (NewLOD)
		{
			CASE_SWAP_TAGS(EMassLOD::High, EMassLOD::Medium);
			CASE_SWAP_TAGS(EMassLOD::High, EMassLOD::Low);
			CASE_SWAP_TAGS(EMassLOD::High, EMassLOD::Off);
			DEFAULT_REMOVE_TAG(EMassLOD::High);
		}
		break;
	case EMassLOD::Medium:
		switch (NewLOD)
		{
			CASE_SWAP_TAGS(EMassLOD::Medium, EMassLOD::High);
			CASE_SWAP_TAGS(EMassLOD::Medium, EMassLOD::Low);
			CASE_SWAP_TAGS(EMassLOD::Medium, EMassLOD::Off);
			DEFAULT_REMOVE_TAG(EMassLOD::Medium);
		}
		break;
	case EMassLOD::Low:
		switch (NewLOD)
		{
			CASE_SWAP_TAGS(EMassLOD::Low, EMassLOD::High);
			CASE_SWAP_TAGS(EMassLOD::Low, EMassLOD::Medium);
			CASE_SWAP_TAGS(EMassLOD::Low, EMassLOD::Off);
			DEFAULT_REMOVE_TAG(EMassLOD::Low);
		}
		break;
	case EMassLOD::Off:
		switch (NewLOD)
		{
			CASE_SWAP_TAGS(EMassLOD::Off, EMassLOD::High);
			CASE_SWAP_TAGS(EMassLOD::Off, EMassLOD::Medium);
			CASE_SWAP_TAGS(EMassLOD::Off, EMassLOD::Low);
			DEFAULT_REMOVE_TAG(EMassLOD::Off);
		}
		break;
	case EMassLOD::Max:
		switch (NewLOD)
		{
			CASE_ADD_TAG(EMassLOD::High);
			CASE_ADD_TAG(EMassLOD::Medium);
			CASE_ADD_TAG(EMassLOD::Low);
			CASE_ADD_TAG(EMassLOD::Off);
			default:
				checkf(false, TEXT("Unsupported LOD types!"));
				break;
		}
		break;
	default:
		checkf(false, TEXT("Unsupported LOD type!"));
		break;
	}

#undef CASE_SWAP_TAGS
#undef DEFAULT_REMOVE_TAG
}
}