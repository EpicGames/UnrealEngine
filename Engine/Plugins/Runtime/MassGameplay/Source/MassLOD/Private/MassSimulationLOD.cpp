// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSimulationLOD.h"

#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "MassLODUtils.h"
#include "MassDebugger.h"

//-----------------------------------------------------------------------------
// FMassSimulationLODParameters
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSimulationLOD)
FMassSimulationLODParameters::FMassSimulationLODParameters()
{
	LODDistance[EMassLOD::High] = 0.0f;
	LODDistance[EMassLOD::Medium] = 5000.0f;
	LODDistance[EMassLOD::Low] = 10000.0f;
	LODDistance[EMassLOD::Off] = 30000.0f;

	LODMaxCount[EMassLOD::High] = 100;
	LODMaxCount[EMassLOD::Medium] = 200;
	LODMaxCount[EMassLOD::Low] = 300;
	LODMaxCount[EMassLOD::Off] = INT_MAX;
}

//-----------------------------------------------------------------------------
// FMassSimulationVariableTickParameters
//-----------------------------------------------------------------------------
FMassSimulationVariableTickParameters::FMassSimulationVariableTickParameters()
{
	TickRates[EMassLOD::High] = 0.0f;
	TickRates[EMassLOD::Medium] = 0.5f;
	TickRates[EMassLOD::Low] = 1.0f;
	TickRates[EMassLOD::Off] = 1.5f;
}

//-----------------------------------------------------------------------------
// FMassSimulationLODSharedFragment
//-----------------------------------------------------------------------------
FMassSimulationLODSharedFragment::FMassSimulationLODSharedFragment(const FMassSimulationLODParameters& LODParams)
{
	LODCalculator.Initialize(LODParams.LODDistance, LODParams.BufferHysteresisOnDistancePercentage / 100.0f, LODParams.LODMaxCount);
}

//-----------------------------------------------------------------------------
// FMassSimulationLODSharedFragment
//-----------------------------------------------------------------------------
FMassSimulationVariableTickSharedFragment::FMassSimulationVariableTickSharedFragment(const FMassSimulationVariableTickParameters& TickRateParams)
{
	LODTickRateController.Initialize(TickRateParams.TickRates, TickRateParams.bSpreadFirstSimulationUpdate);
}

//-----------------------------------------------------------------------------
// UMassSimulationLODProcessor
//-----------------------------------------------------------------------------

namespace UE::MassLOD
{
	int32 bDebugSimulationLOD = 0;
	FAutoConsoleVariableRef CVarDebugSimulationLODTest(TEXT("mass.debug.SimulationLOD"), bDebugSimulationLOD, TEXT("Debug Simulation LOD"), ECVF_Cheat);
} // UE::MassLOD

UMassSimulationLODProcessor::UMassSimulationLODProcessor()
	: EntityQuery(*this)
	, EntityQueryCalculateLOD(*this)
	, EntityQueryAdjustDistances(*this)
	, EntityQueryVariableTick(*this)
	, EntityQuerySetLODTag(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LOD;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);
}

void UMassSimulationLODProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassSimulationLODParameters>();
	EntityQuery.AddSharedRequirement<FMassSimulationLODSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddSharedRequirement<FMassSimulationVariableTickSharedFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

	EntityQueryCalculateLOD = EntityQuery;
	EntityQueryCalculateLOD.SetChunkFilter(FMassSimulationVariableTickSharedFragment::ShouldCalculateLODForChunk);

	EntityQueryAdjustDistances = EntityQuery;
	EntityQueryAdjustDistances.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		const FMassSimulationLODSharedFragment& LODSharedFragment = Context.GetSharedFragment<FMassSimulationLODSharedFragment>();
		return LODSharedFragment.bHasAdjustedDistancesFromCount && FMassSimulationVariableTickSharedFragment::ShouldAdjustLODFromCountForChunk(Context);
	});

	EntityQueryVariableTick.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQueryVariableTick.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadWrite);
	EntityQueryVariableTick.AddConstSharedRequirement<FMassSimulationVariableTickParameters>();
	EntityQueryVariableTick.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQueryVariableTick.AddSharedRequirement<FMassSimulationVariableTickSharedFragment>(EMassFragmentAccess::ReadWrite);

	// In case where the variableTick isn't enabled, we might need to set LOD tags as if the users still wants them
	EntityQuerySetLODTag.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuerySetLODTag.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::None);
	EntityQuerySetLODTag.AddConstSharedRequirement<FMassSimulationLODParameters>();
	EntityQuerySetLODTag.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		const FMassSimulationLODParameters& LODParams = Context.GetConstSharedFragment<FMassSimulationLODParameters>();
		return LODParams.bSetLODTags;
	});

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassSimulationLODProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SimulationLOD)

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareExecution);

		const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
		const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();

		EntityManager.ForEachSharedFragment<FMassSimulationLODSharedFragment>([&Viewers](FMassSimulationLODSharedFragment& LODSharedFragment)
		{
			LODSharedFragment.LODCalculator.PrepareExecution(Viewers);
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CalculateLOD);
		EntityQueryCalculateLOD.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			FMassSimulationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassSimulationLODSharedFragment>();
			TConstArrayView<FMassViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			TArrayView<FMassSimulationLODFragment> SimulationLODFragments = Context.GetMutableFragmentView<FMassSimulationLODFragment>();
			LODSharedFragment.LODCalculator.CalculateLOD(Context, ViewersInfoList, SimulationLODFragments);
		});
	}

	if (bDoAdjustmentFromCount)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AdjustDistancesAndLODFromCount);
		EntityManager.ForEachSharedFragment<FMassSimulationLODSharedFragment>([](FMassSimulationLODSharedFragment& LODSharedFragment)
		{
			LODSharedFragment.bHasAdjustedDistancesFromCount = LODSharedFragment.LODCalculator.AdjustDistancesFromCount();
		});

		EntityQueryAdjustDistances.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			FMassSimulationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassSimulationLODSharedFragment>();
			TConstArrayView<FMassViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			TArrayView<FMassSimulationLODFragment> SimulationLODFragments = Context.GetMutableFragmentView<FMassSimulationLODFragment>();
			LODSharedFragment.LODCalculator.AdjustLODFromCount(Context, ViewersInfoList, SimulationLODFragments);
		});
	}

	UWorld* World = EntityManager.GetWorld();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VariableTickRates)
		check(World);
		const double Time = World->GetTimeSeconds();
		EntityQueryVariableTick.ForEachEntityChunk(Context, [Time](FMassExecutionContext& Context)
		{
			FMassSimulationVariableTickSharedFragment& TickRateSharedFragment = Context.GetMutableSharedFragment<FMassSimulationVariableTickSharedFragment>();
			TConstArrayView<FMassSimulationLODFragment> SimulationLODFragments = Context.GetFragmentView<FMassSimulationLODFragment>();
			TArrayView<FMassSimulationVariableTickFragment> SimulationVariableTickFragments = Context.GetMutableFragmentView<FMassSimulationVariableTickFragment>();

			TickRateSharedFragment.LODTickRateController.UpdateTickRateFromLOD(Context, SimulationLODFragments, SimulationVariableTickFragments, Time);
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SetLODTags)
		check(World);
		EntityQuerySetLODTag.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			TConstArrayView<FMassSimulationLODFragment> SimulationLODFragments = Context.GetFragmentView<FMassSimulationLODFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassSimulationLODFragment& EntityLOD = SimulationLODFragments[EntityIt];
				if (EntityLOD.PrevLOD != EntityLOD.LOD)
				{
					const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
					UE::MassLOD::PushSwapTagsCommand(Context.Defer(), Entity, EntityLOD.PrevLOD, EntityLOD.LOD);
				}
			}
		});
	}

#if WITH_MASSGAMEPLAY_DEBUG
	// Optional debug display
	if (UE::MassLOD::bDebugSimulationLOD)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayLOD);
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			EntityQuery.ForEachEntityChunk(Context, [DebugDrawingWorld](FMassExecutionContext& Context)
			{
				FMassSimulationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassSimulationLODSharedFragment>();
				TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
				TConstArrayView<FMassSimulationLODFragment> SimulationLODList = Context.GetFragmentView<FMassSimulationLODFragment>();
				LODSharedFragment.LODCalculator.DebugDisplayLOD(Context, SimulationLODList, LocationList, DebugDrawingWorld);
		   });
		}
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}
