// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODLogic.h"
#include "MassLODUtils.h"
#include "DrawDebugHelpers.h"
#include "VisualLogger/VisualLogger.h"

/**
 * Helper struct to calculate LOD for each agent and maximize count per LOD
 *   Requires TViewerInfoFragment fragment collected by the TMassLODCollector.
 *   Stores information in TLODFragment fragment.
*/
template <typename FLODLogic = FLODDefaultLogic >
struct TMassLODCalculator : public FMassLODBaseLogic
{
	TMassLODCalculator()
		: FMassLODBaseLogic(/*bShouldBuildFrustumData=*/FLODLogic::bDoVisibilityLogic)
	{}

	/**
	 * Initializes the LOD calculator, needed to be called once at initialization time
	 * @Param InBaseLODDistance distances used to calculate LOD
	 * @Param InBufferHysteresisOnFOVRatio distance hysteresis used to calculate LOD
	 * @Param InLODMaxCount the maximum count for each LOD - Supports nullptr being passed in now and will put INT_MAX everywhere by default
	 * @Param InLODMaxCountPerViewer the maximum count for each LOD per viewer (Only when FLODLogic::bMaximizeCountPerViewer is enabled)
	 * @Param InVisibleDistanceToFrustum is the distance from the frustum to start considering this entity is visible (Only when FLODLogic::bDoVisibilityLogic is enabled)
	 * @Param InVisibleDistanceToFrustumHysteresis once visible, what extra distance the entity need to be before considered not visible anymore (Only when FLODLogic::bDoVisibilityLogic is enabled)
	 * @Param InVisibleLODDistance the maximum count for each LOD per viewer (Only when FLODLogic::bDoVisibilityLogic is enabled)
	 */
	void Initialize(const float InBaseLODDistance[EMassLOD::Max], 
					const float InBufferHysteresisOnDistanceRatio, 
					const int32 InLODMaxCount[EMassLOD::Max], 
					const int32 InLODMaxCountPerViewer[EMassLOD::Max] = nullptr,
					const float InVisibleDistanceToFrustum = 0.0f,
					const float InVisibleDistanceToFrustumHysteresis = 0.0f,
					const float InVisibleLODDistance[EMassLOD::Max] = nullptr);

	/**
	 * Prepares execution for the current frame, needed to be called before every execution
	 * @Param Viewers is the array of all the known viewers
	 */
	void PrepareExecution(TConstArrayView<FViewerInfo> Viewers);

	/**
	 * Calculate LOD, called for each entity chunks
	 * Use next method when FLODLogic::bStoreInfoPerViewer is enabled
	 * @Param Context of the chunk execution
	 * @Param ViewersInfoList is the source information fragment for LOD calculation
	 * @Param LODList is the fragment where calculation are stored
	 */
	template <typename TViewerInfoFragment,
			  typename TLODFragment,
			  bool bCalculateLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCalculateVisibility = FLODLogic::bDoVisibilityLogic>
	inline void CalculateLOD(FMassExecutionContext& Context,
					  			  TConstArrayView<TViewerInfoFragment> ViewersInfoList,
					  			  TArrayView<TLODFragment> LODList)
	{
		CalculateLOD<TViewerInfoFragment,
			TLODFragment,
			/*TPerViewerInfoFragment*/void*,
			bCalculateLocalViewers,
			bCalculateVisibility,
			/*bCalculateLODPerViewer*/false,
			/*bCalculateVisibilityPerViewer*/false,
			/*bMaximizeCountPerViewer*/false>(Context, ViewersInfoList, LODList, TConstArrayView<void*>());
	}

	/**
	 * Calculate LOD, called for each entity chunks
	 * Use this version when FLODLogic::bStoreInfoPerViewer is enabled
	 * It calculates a LOD per viewer and needs information per viewer via PerViewerInfoList fragments
	 * @Param Context of the chunk execution
	 * @Param ViewersInfoList is the source information fragment for LOD calculation
	 * @Param LODList is the fragment where calculation are stored
	 * @Param PerViewerInfoList is the Per viewer source information
	 */
	template <typename TViewerInfoFragment,
			  typename TLODFragment,
			  typename TPerViewerInfoFragment,
			  bool bCalculateLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCalculateVisibility = FLODLogic::bDoVisibilityLogic,
			  bool bCalculateLODPerViewer = FLODLogic::bCalculateLODPerViewer,
			  bool bCalculateVisibilityPerViewer = FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer,
			  bool bMaximizeCountPerViewer = FLODLogic::bMaximizeCountPerViewer>
	void CalculateLOD(FMassExecutionContext& Context,
					  TConstArrayView<TViewerInfoFragment> ViewersInfoList,
					  TArrayView<TLODFragment> LODList,
					  TConstArrayView<TPerViewerInfoFragment> PerViewerInfoList);

	/**
	 * Adjust LOD distances by clamping them to respect the maximum LOD count
	 * @Return true if any LOD distances clamping was done
	 */
	template <bool bCalculateVisibility = FLODLogic::bDoVisibilityLogic,
			  bool bCalculateVisibilityPerViewer = FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer,
			  bool bMaximizeCountPerViewer = FLODLogic::bMaximizeCountPerViewer>
	bool AdjustDistancesFromCount();

	/**
	 * Adjust LOD from newly adjusted distances, only needed to be called when AdjustDistancesFromCount return true, called for each entity chunks
	 * Use next method when FLODLogic::bStoreInfoPerViewer is enabled
	 * @Param Context of the chunk execution
	 * @Param ViewersInfoList is the source information fragment for LOD calculation
	 * @Param LODList is the fragment where calculation are stored
	 */
	template <typename TViewerInfoFragment, 
			  typename TLODFragment,
			  bool bCalculateLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCalculateVisibility = FLODLogic::bDoVisibilityLogic>
	inline void AdjustLODFromCount(FMassExecutionContext& Context,
										TConstArrayView<TViewerInfoFragment> ViewersInfoList,
										TArrayView<TLODFragment> LODList)
	{
		AdjustLODFromCount<TViewerInfoFragment,
			TLODFragment,
			/*TPerViewerInfoFragment*/void*,
			bCalculateLocalViewers,
			bCalculateVisibility,
			/*bCalculateLODPerViewer*/false,
			/*bCalculateVisibilityPerViewer*/false,
			/*bMaximizeCountPerViewer*/false>(Context, ViewersInfoList, LODList, TConstArrayView<void*>());
	}


	/**
	 * Adjust LOD from newly adjusted distances, only needed to be called when AdjustDistancesFromCount return true, called for each entity chunks
	 * Use this version when FLODLogic::bStoreInfoPerViewer is enabled
	 * It calculates a LOD per viewer and needs information per viewer via PerViewerInfoList fragments
	 * @Param Context of the chunk execution
	 * @Param ViewersInfoList is the source information fragment for LOD calculation
	 * @Param LODList is the fragment where calculation are stored
	 * @Param PerViewerInfoList is the Per viewer source information
	 */
	template <typename TViewerInfoFragment,
			  typename TLODFragment,
			  typename TPerViewerInfoFragment,
			  bool bCalculateLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCalculateVisibility = FLODLogic::bDoVisibilityLogic,
			  bool bCalculateLODPerViewer = FLODLogic::bCalculateLODPerViewer,
			  bool bCalculateVisibilityPerViewer = FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer,
			  bool bMaximizeCountPerViewer = FLODLogic::bMaximizeCountPerViewer>
	void AdjustLODFromCount(FMassExecutionContext& Context,
							TConstArrayView<TViewerInfoFragment> ViewersInfoList,
							TArrayView<TLODFragment> LODList,
							TConstArrayView<TPerViewerInfoFragment> PerViewerInfoList);

	/**
	 * Turn Off all LOD, called for each entity chunks
	 * @Param Context of the chunk execution
	 * @Param LODList is the fragment where calculation are stored
	 */
	template <typename TLODFragment>
	void ForceOffLOD(FMassExecutionContext& Context, TArrayView<TLODFragment> LODList);

#if WITH_MASSGAMEPLAY_DEBUG
	/**
	 * Debug draw the current state of each agent as a color coded square
	 * @Param Context of the chunk execution
	 * @Param LODList is the fragment where calculation are stored
	 * @Param LocationList is the fragment transforms of the entities
	 * @Param World where the debug display should be drawn
	 */
	template <typename TLODFragment, typename TTransformFragment>
	void DebugDisplayLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UWorld* World);

	/**
	 * Debug draw the current state of each agent as a color coded square, within MaxLODSignificance range
	 * @Param Context of the chunk execution
	 * @Param LODList is the fragment where calculation are stored
	 * @Param LocationList is the fragment transforms of the entities
	 * @Param World where the debug display should be drawn
	 * @Param MaxLODSignificance is the max allowed value of LODList[i].LODSignificance for an agent's state to debug draw
	 */
	template <typename TLODFragment, typename TTransformFragment>
	void DebugDisplaySignificantLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UWorld* World, float MaxLODSignificance);

	/**
	 * Add Visual Log entries for the current state of each agent as a color coded location
	 * @Param Context of the chunk execution
	 * @Param LODList is the fragment where calculation are stored
	 * @Param LocationList is the fragment transforms of the entities
	 * @Param World where the debug display should be drawn
	 */
	template <typename TLODFragment, typename TTransformFragment>
	void VisLogLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UObject* LogOwner);

	/**
	 * Add Visual Log entries for the current state of each agent as a color coded location, within MaxLODSignificance range
	 * @Param Context of the chunk execution
	 * @Param LODList is the fragment where calculation are stored
	 * @Param LocationList is the fragment transforms of the entities
	 * @Param World where the debug display should be drawn
	 * @Param MaxLODSignificance is the max allowed value of LODList[i].LODSignificance for an agent's state to vislog
	 */
	template <typename TLODFragment, typename TTransformFragment>
	void VisLogSignificantLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UObject* LogOwner, float MaxLODSignificance);
#endif // WITH_MASSGAMEPLAY_DEBUG

	/**
	 * Return the maximum distance at which the LOD will be turn off
	 */
	 float GetMaxLODDistance() const { return MaxLODDistance; }

protected:

	struct FMassLODRuntimeData
	{
		/** Reset values to default */
		void Reset(const TStaticArray<float, EMassLOD::Max>& InBaseLODDistance, const TStaticArray<float, EMassLOD::Max>& InVisibleLODDistance)
		{
			// Reset the AdjustedLODDistances as they might have been changed by the max count calculation previous frame
			for (int32 LODDistIdx = 0; LODDistIdx < EMassLOD::Max; LODDistIdx++)
			{
				AdjustedBaseLODDistance[LODDistIdx] = InBaseLODDistance[LODDistIdx];
				AdjustedBaseLODDistanceSq[LODDistIdx] = FMath::Square(AdjustedBaseLODDistance[LODDistIdx]);
				if (FLODLogic::bDoVisibilityLogic)
				{
					AdjustedVisibleLODDistance[LODDistIdx] = InVisibleLODDistance[LODDistIdx];
					AdjustedVisibleLODDistanceSq[LODDistIdx] = FMath::Square(AdjustedVisibleLODDistance[LODDistIdx]);
				}
			}
			FMemory::Memzero(BaseBucketCounts);
			if (FLODLogic::bDoVisibilityLogic)
			{
				FMemory::Memzero(VisibleBucketCounts);
			}
		}

		/** Distance where each LOD becomes relevant (Squared and Normal) */
		TStaticArray<float, EMassLOD::Max> AdjustedBaseLODDistanceSq;
		TStaticArray<float, EMassLOD::Max> AdjustedVisibleLODDistanceSq;
		TStaticArray<float, EMassLOD::Max> AdjustedBaseLODDistance;
		TStaticArray<float, EMassLOD::Max> AdjustedVisibleLODDistance;

		/** Count of entities in each subdivision */
		TStaticArray< TStaticArray<int32, UE::MassLOD::MaxBucketsPerLOD>, EMassLOD::Max > BaseBucketCounts;
		TStaticArray< TStaticArray<int32, UE::MassLOD::MaxBucketsPerLOD>, EMassLOD::Max > VisibleBucketCounts;

#if WITH_MASSGAMEPLAY_DEBUG
		/* Last calculation count per LOD */
		TStaticArray<int32, EMassLOD::Max> LastCalculatedLODCount;
#endif // WITH_MASSGAMEPLAY_DEBUG
	};


	template <bool bCalculateVisibility, bool bCalculateLODSignificance>
	float AccumulateCountInRuntimeData(const EMassLOD::Type LOD, const float ViewerDistanceSq, const bool bIsVisible, FMassLODRuntimeData& Data) const;

	template <bool bCalculateVisibility>
	bool AdjustDistancesFromCountForRuntimeData(const TStaticArray<int32, EMassLOD::Max>& MaxCount, FMassLODRuntimeData& RuntimeData) const;

	template<bool bCalculateVisibility>
	EMassLOD::Type ComputeLODFromSettings(const EMassLOD::Type PrevLOD, const float DistanceToViewerSq, const bool bIsVisible, bool* bIsInAVisibleRange, const FMassLODRuntimeData& Data) const;

	bool CalculateVisibility(const bool bWasVisible, const float DistanceToFrustum) const;

	/** LOD distances */
	TStaticArray<float, EMassLOD::Max> BaseLODDistance;
	TStaticArray<float, EMassLOD::Max> VisibleLODDistance;

	/** MaxCount total */
	TStaticArray<int32, EMassLOD::Max> LODMaxCount;

	/** MaxCount total per viewers*/
	TStaticArray<int32, EMassLOD::Max> LODMaxCountPerViewer;

	/** Ratio for Buffer Distance Hysteresis */
	float BufferHysteresisOnDistanceRatio = 0.1f;

	/** How far away from frustum does this entities are considered visible */
	float VisibleDistanceToFrustum = 0.0f;

	/** Once visible how much further than distance to frustum does the entities need to be before being consider not visible */
	float VisibleDistanceToFrustumWithHysteresis = 0.0f;

	/** The size of each subdivision per LOD (LOD Size/MaxBucketsPerLOD) */
	TStaticArray<float, EMassLOD::Max> BaseBucketSize;
	TStaticArray<float, EMassLOD::Max> VisibleBucketSize;

	/** Maximum LOD Distance  */
	float MaxLODDistance = 0.0f;

	/** Runtime data for LOD calculation */
	FMassLODRuntimeData RuntimeData;

	/** Runtime data for each viewer specific LOD calculation, used only when bMaximizeCountPerViewer is true */
	TArray<FMassLODRuntimeData> RuntimeDataPerViewer;
};

template <typename FLODLogic>
void TMassLODCalculator<FLODLogic>::Initialize(const float InBaseLODDistance[EMassLOD::Max],
											   const float InBufferHysteresisOnDistanceRatio,
											   const int32 InLODMaxCount[EMassLOD::Max],
											   const int32 InLODMaxCountPerViewer[EMassLOD::Max] /*= nullptr*/,
											   const float InVisibleDistanceToFrustum /*= 0.0f*/,
											   const float InVisibleDistanceToFrustumHysteresis /*= 0.0f*/,
											   const float InVisibleLODDistance[EMassLOD::Max] /*= nullptr*/ )
{
	static_assert(!FLODLogic::bCalculateLODPerViewer || FLODLogic::bStoreInfoPerViewer, "Need to enable store info per viewer to be able to calculate LOD per viewer");
	static_assert(!FLODLogic::bMaximizeCountPerViewer || FLODLogic::bCalculateLODPerViewer, "Need to enable CalculatedLODPerviewer in order to maximize count per viewer");

	checkf(FLODLogic::bMaximizeCountPerViewer == (InLODMaxCountPerViewer != nullptr), TEXT("Missmatched between expected parameter InLODMaxCountPerViewer and LOD logic trait bMaximizeCountPerViewer."));
	checkf(FLODLogic::bDoVisibilityLogic == (InVisibleLODDistance != nullptr), TEXT("Missmatched between expected parameter InVisibleLODDistance and LOD logic trait bDoVisibilityLogic."));

	// Make a copy of all the settings
	for (int x = 0; x < EMassLOD::Max; x++)
	{
		BaseLODDistance[x] = InBaseLODDistance[x];
		
		// @todo Treat InLODMaxCount as a possible nullptr by default for this Initialize function, would need to come as an option from FLODLogic as well
		LODMaxCount[x] = (InLODMaxCount != nullptr) ? InLODMaxCount[x] : INT_MAX;
		if (FLODLogic::bDoVisibilityLogic && InVisibleLODDistance)
		{
			VisibleLODDistance[x] = InVisibleLODDistance[x];
		}
		if (FLODLogic::bMaximizeCountPerViewer && InLODMaxCountPerViewer)
		{
			LODMaxCountPerViewer[x] = InLODMaxCountPerViewer[x];
		}
	}

	// Some values should always be constant
	BaseLODDistance[EMassLOD::High] = 0.0f;
	BaseBucketSize[EMassLOD::Off] = FLT_MAX;
	VisibleLODDistance[EMassLOD::High] = 0.0f;
	VisibleBucketSize[EMassLOD::Off] = FLT_MAX;
	LODMaxCount[EMassLOD::Off] = INT_MAX;
	LODMaxCountPerViewer[EMassLOD::Off] = INT_MAX;
	BufferHysteresisOnDistanceRatio = InBufferHysteresisOnDistanceRatio;

	// Calculate the size for each LOD buckets
	float BasePrevLODDistance = BaseLODDistance[0];
	float VisiblePrevLODDistance = VisibleLODDistance[0];
	for (int32 LODDistIdx = 1; LODDistIdx < EMassLOD::Max; LODDistIdx++)
	{
		BaseBucketSize[LODDistIdx - 1] = (BaseLODDistance[LODDistIdx] - BasePrevLODDistance) / UE::MassLOD::MaxBucketsPerLOD;
		BasePrevLODDistance = BaseLODDistance[LODDistIdx];

		if (FLODLogic::bDoVisibilityLogic)
		{
			VisibleBucketSize[LODDistIdx - 1] = (VisibleLODDistance[LODDistIdx] - VisiblePrevLODDistance) / UE::MassLOD::MaxBucketsPerLOD;
			VisiblePrevLODDistance = VisibleLODDistance[LODDistIdx];
		}
	}

	// Assuming that off is the farthest distance, calculate the max LOD distance
	MaxLODDistance = !FLODLogic::bDoVisibilityLogic || BaseLODDistance[EMassLOD::Off] >= VisibleLODDistance[EMassLOD::Off] ? BaseLODDistance[EMassLOD::Off] : VisibleLODDistance[EMassLOD::Off];

	// Distance to frustum settings
	VisibleDistanceToFrustum = InVisibleDistanceToFrustum;
	VisibleDistanceToFrustumWithHysteresis = InVisibleDistanceToFrustum + InVisibleDistanceToFrustumHysteresis;
}

template <typename FLODLogic>
bool TMassLODCalculator<FLODLogic>::CalculateVisibility(const bool bWasVisible, const float DistanceToFrustum) const
{
	return DistanceToFrustum < (bWasVisible ? VisibleDistanceToFrustumWithHysteresis : VisibleDistanceToFrustum);
}

template <typename FLODLogic>
void TMassLODCalculator<FLODLogic>::PrepareExecution(TConstArrayView<FViewerInfo> ViewersInfo)
{
	CacheViewerInformation(ViewersInfo);

	if (FLODLogic::bMaximizeCountPerViewer)
	{
		RuntimeDataPerViewer.SetNum(Viewers.Num());
		for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
		{
			// Reset viewer data
			if (Viewers[ViewerIdx].Handle.IsValid())
			{
				RuntimeDataPerViewer[ViewerIdx].Reset(BaseLODDistance, VisibleLODDistance);
			}
		}
	}

	RuntimeData.Reset(BaseLODDistance, VisibleLODDistance);
}


template <typename FLODLogic>
template <bool bCalculateVisibility, bool bCalculateLODSignificance>
float TMassLODCalculator<FLODLogic>::AccumulateCountInRuntimeData(const EMassLOD::Type LOD, const float ViewerDistanceSq, const bool bIsVisible, FMassLODRuntimeData& Data) const
{
	TStaticArray< TStaticArray<int32, UE::MassLOD::MaxBucketsPerLOD>, EMassLOD::Max>& BucketCounts = bCalculateVisibility && bIsVisible ? Data.VisibleBucketCounts : Data.BaseBucketCounts;

	// Cumulate LOD in buckets for Max LOD count calculation
	if (LOD == EMassLOD::Off)
	{
		// A single bucket for Off LOD
		BucketCounts[EMassLOD::Off][0]++;
		if (bCalculateLODSignificance)
		{
			return float(EMassLOD::Off);
		}
	}
	else
	{
		const TStaticArray<float, EMassLOD::Max>& BucketSize = bCalculateVisibility && bIsVisible ? VisibleBucketSize : BaseBucketSize;
		const TStaticArray<float, EMassLOD::Max>& AdjustedLODDistance = bCalculateVisibility && bIsVisible ? Data.AdjustedVisibleLODDistance : Data.AdjustedBaseLODDistance;

		const int32 LODDistIdx = (int32)LOD;

		// Need to clamp as the Sqrt is not precise enough and always end up with floating calculation errors
		const int32 BucketIdx = FMath::Clamp((int32)((FMath::Sqrt(ViewerDistanceSq) - AdjustedLODDistance[LODDistIdx]) / BucketSize[LODDistIdx]), 0, UE::MassLOD::MaxBucketsPerLOD - 1);
		BucketCounts[LODDistIdx][BucketIdx]++;

		if (bCalculateLODSignificance)
		{
			// Derive significance from LODDistIdx combined with BucketIdx
			const float PartialLODSignificance = float(BucketIdx) / float(UE::MassLOD::MaxBucketsPerLOD);
			return float(LODDistIdx) + PartialLODSignificance;
		}
	}
	return 0.0f;
}

template <typename FLODLogic>
template <typename TViewerInfoFragment, typename TLODFragment, typename TPerViewerInfoFragment,
		  bool bCalculateLocalViewers, bool bCalculateVisibility, bool bCalculateLODPerViewer, bool bCalculateVisibilityPerViewer,bool bMaximizeCountPerViewer>
void TMassLODCalculator<FLODLogic>::CalculateLOD(FMassExecutionContext& Context, 
												 TConstArrayView<TViewerInfoFragment> ViewersInfoList, 
												 TArrayView<TLODFragment> LODList, 
												 TConstArrayView<TPerViewerInfoFragment> PerViewerInfoList)
{
	static_assert(!bCalculateVisibility || FLODLogic::bDoVisibilityLogic, "FLODLogic must have bDoVisibilityLogic enabled to calculate visibility.");
	static_assert(!bCalculateLODPerViewer || FLODLogic::bCalculateLODPerViewer, "FLODLogic must have bCalculateLODPerViewer enabled to calculate LOD Per viewer.");
	static_assert(!bCalculateVisibilityPerViewer || (FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer), "FLODLogic must have bDoVisibilityLogic and bStoreInfoPerViewer enabled to calculate visibility per viewer.");
	static_assert(!bMaximizeCountPerViewer || FLODLogic::bMaximizeCountPerViewer, "FLODLogic must have bMaximizeCountPerViewer enabled to maximize count per viewer.");

#if WITH_MASSGAMEPLAY_DEBUG
	if (UE::MassLOD::Debug::bLODCalculationsPaused)
	{
		return;
	}
#endif // WITH_MASSGAMEPLAY_DEBUG

	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		// Calculate the LOD purely upon distances
		const TViewerInfoFragment& EntityViewersInfo = ViewersInfoList[EntityIt];
		TLODFragment& EntityLOD = LODList[EntityIt];
		const float ClosestDistanceToFrustum = GetClosestDistanceToFrustum<bCalculateVisibility>(EntityViewersInfo, FLT_MAX);
		const EMassVisibility PrevVisibility = GetVisibility<bCalculateVisibility>(EntityLOD, EMassVisibility::Max);
		const bool bIsVisibleByAViewer = CalculateVisibility(PrevVisibility == EMassVisibility::CanBeSeen, ClosestDistanceToFrustum);
		bool bIsInAVisibleRange = false;

		// Find new LOD
		EntityLOD.PrevLOD = EntityLOD.LOD;
		EntityLOD.LOD = ComputeLODFromSettings<bCalculateVisibility>(EntityLOD.PrevLOD, EntityViewersInfo.ClosestViewerDistanceSq, bIsVisibleByAViewer, &bIsInAVisibleRange, RuntimeData);

		// Set visibility
		SetPrevVisibility<bCalculateVisibility>(EntityLOD, PrevVisibility);
		SetVisibility<bCalculateVisibility>(EntityLOD, bIsInAVisibleRange ? (bIsVisibleByAViewer ? EMassVisibility::CanBeSeen : EMassVisibility::CulledByFrustum) : EMassVisibility::CulledByDistance);

		// Accumulate in buckets
		const float LODSignificance = AccumulateCountInRuntimeData<bCalculateVisibility, FLODLogic::bCalculateLODSignificance>(EntityLOD.LOD, EntityViewersInfo.ClosestViewerDistanceSq, bIsVisibleByAViewer, RuntimeData);
		SetLODSignificance<FLODLogic::bCalculateLODSignificance>(EntityLOD, LODSignificance);

		// Do per viewer logic if asked for
		if (bCalculateLODPerViewer || bCalculateVisibilityPerViewer)
		{
			const TPerViewerInfoFragment& EntityPerViewerInfo = PerViewerInfoList[EntityIt];

			SetLODPerViewerNum<bCalculateLODPerViewer>(EntityLOD, Viewers.Num());
			SetPrevLODPerViewerNum<bCalculateLODPerViewer>(EntityLOD, Viewers.Num());
			SetVisibilityPerViewerNum<bCalculateVisibilityPerViewer>(EntityLOD, Viewers.Num());
			SetPrevVisibilityPerViewerNum<bCalculateVisibilityPerViewer>(EntityLOD, Viewers.Num());

			for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
			{
				const FViewerLODInfo& Viewer = Viewers[ViewerIdx];
				if (Viewer.bClearData)
				{
					SetLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EMassLOD::Max);
					SetPrevLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EMassLOD::Max);
					SetPrevVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max);
					SetVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max);
				}

				// Check to see if we want only local viewer only
				if (bCalculateLocalViewers && !Viewer.bLocal)
				{
					continue;
				}

				if (Viewer.Handle.IsValid())
				{
					const float DistanceToFrustum = GetDistanceToFrustum<bCalculateVisibilityPerViewer>(EntityPerViewerInfo, ViewerIdx, FLT_MAX);
					const EMassVisibility PrevVisibilityPerViewer = GetVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max);
					const bool bIsVisibleByViewer = CalculateVisibility(PrevVisibilityPerViewer == EMassVisibility::CanBeSeen, DistanceToFrustum);
					bool bIsInVisibleRange = false;

					if (bCalculateLODPerViewer)
					{
						const float DistanceToViewerSq = GetDistanceToViewerSq<bCalculateLODPerViewer>(EntityPerViewerInfo, ViewerIdx, FLT_MAX);
						const EMassLOD::Type PrevLODPerViewer = GetLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EntityLOD.LOD);

						// Find new LOD
						const EMassLOD::Type LODPerViewer = ComputeLODFromSettings<bCalculateVisibilityPerViewer>(PrevLODPerViewer, DistanceToViewerSq, bIsVisibleByViewer, &bIsInVisibleRange, RuntimeData);

						// Set Per viewer LOD
						SetPrevLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, PrevLODPerViewer);
						SetLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, LODPerViewer);

						if (bMaximizeCountPerViewer)
						{
							// Accumulate in buckets
							AccumulateCountInRuntimeData<bCalculateVisibilityPerViewer, false>(LODPerViewer, DistanceToViewerSq, bIsVisibleByViewer, RuntimeDataPerViewer[ViewerIdx]);
						}
					}
					else if (bCalculateVisibilityPerViewer)
					{
						// No per-viewer LOD computation, but visibility-per-viewer still needs to know if the entity is in the visible LOD range
						const float DistanceToViewerSq = GetDistanceToViewerSq<bCalculateVisibilityPerViewer>(EntityPerViewerInfo, ViewerIdx, FLT_MAX);
						bIsInVisibleRange = DistanceToViewerSq < RuntimeData.AdjustedVisibleLODDistanceSq[EMassLOD::Off];
					}

					// Set visibility
					SetPrevVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, PrevVisibilityPerViewer);
					SetVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, bIsInVisibleRange ? (bIsVisibleByViewer ? EMassVisibility::CanBeSeen : EMassVisibility::CulledByFrustum) : EMassVisibility::CulledByDistance);
				}
			}
		}
	}
}

template <typename FLODLogic>
template <bool bCalculateVisibility>
bool TMassLODCalculator<FLODLogic>::AdjustDistancesFromCountForRuntimeData(const TStaticArray<int32, EMassLOD::Max>& MaxCount, FMassLODRuntimeData& Data) const
{
	int32 Count = 0;
	int32 ProcessingLODIdx = EMassLOD::High;

	bool bNeedAdjustments = false;

	// Go through all LOD can start counting from the high LOD
	for (int32 BucketLODIdx = 0; BucketLODIdx < EMassLOD::Max - 1; ++BucketLODIdx)
	{
		// Switch to next LOD if we have not reach the max
		if (ProcessingLODIdx < BucketLODIdx)
		{
#if WITH_MASSGAMEPLAY_DEBUG
			// Save the count of this LOD for this frame
			Data.LastCalculatedLODCount[ProcessingLODIdx] = Count;
#endif // WITH_MASSGAMEPLAY_DEBUG

			// Switch to next LOD
			ProcessingLODIdx = BucketLODIdx;

			// Restart the count
			Count = 0;
		}

		// Count entities through all buckets of this LOD
		for (int32 BucketIdx = 0; BucketIdx < UE::MassLOD::MaxBucketsPerLOD; ++BucketIdx)
		{
			if (bCalculateVisibility)
			{
				// Do visible count first to prioritize them over none visible for that bucket idx
				Count += Data.VisibleBucketCounts[BucketLODIdx][BucketIdx];

				while (Count > MaxCount[ProcessingLODIdx])
				{
#if WITH_MASSGAMEPLAY_DEBUG
					// Save the count of this LOD for this frame
					Data.LastCalculatedLODCount[ProcessingLODIdx] = Count - Data.VisibleBucketCounts[BucketLODIdx][BucketIdx];
#endif // WITH_MASSGAMEPLAY_DEBUG

					// Switch to next LOD
					ProcessingLODIdx++;

					// Adjust distance for this LOD
					Data.AdjustedBaseLODDistance[ProcessingLODIdx] = BaseLODDistance[BucketLODIdx] + (static_cast<float>(BucketIdx) * BaseBucketSize[BucketLODIdx]);
					Data.AdjustedBaseLODDistanceSq[ProcessingLODIdx] = FMath::Square(Data.AdjustedBaseLODDistance[ProcessingLODIdx]);
					Data.AdjustedVisibleLODDistance[ProcessingLODIdx] = VisibleLODDistance[BucketLODIdx] + (static_cast<float>(BucketIdx) * VisibleBucketSize[BucketLODIdx]);
					Data.AdjustedVisibleLODDistanceSq[ProcessingLODIdx] = FMath::Square(Data.AdjustedVisibleLODDistance[ProcessingLODIdx]);

					// Check if we are done
					if (ProcessingLODIdx == EMassLOD::Off)
					{
						return true;
					}

					// Start the next LOD count with the bucket count that made it go over
					Count = Data.VisibleBucketCounts[BucketLODIdx][BucketIdx];

					bNeedAdjustments = true;
				}
			}

			// Add base count
			Count += Data.BaseBucketCounts[BucketLODIdx][BucketIdx];

			// Check if the count is going over max
			while (Count > MaxCount[ProcessingLODIdx])
			{
#if WITH_MASSGAMEPLAY_DEBUG
				// Save the count of this LOD for this frame
				Data.LastCalculatedLODCount[ProcessingLODIdx] = Count - Data.BaseBucketCounts[BucketLODIdx][BucketIdx];
#endif // WITH_MASSGAMEPLAY_DEBUG

				// Switch to next LOD
				ProcessingLODIdx++;

				// Adjust distance for this LOD
				Data.AdjustedBaseLODDistance[ProcessingLODIdx] = BaseLODDistance[BucketLODIdx] + (static_cast<float>(BucketIdx) * BaseBucketSize[BucketLODIdx]);
				Data.AdjustedBaseLODDistanceSq[ProcessingLODIdx] = FMath::Square(Data.AdjustedBaseLODDistance[ProcessingLODIdx]);
				if (bCalculateVisibility)
				{
					Data.AdjustedVisibleLODDistance[ProcessingLODIdx] = VisibleLODDistance[BucketLODIdx] + (static_cast<float>(BucketIdx) * VisibleBucketSize[BucketLODIdx]);
					Data.AdjustedVisibleLODDistanceSq[ProcessingLODIdx] = FMath::Square(Data.AdjustedVisibleLODDistance[ProcessingLODIdx]);
				}

				// Check if we are done
				if (ProcessingLODIdx == EMassLOD::Off)
				{
					return true;
				}

				// Start the next LOD count with the bucket count that made it go over
				Count = Data.BaseBucketCounts[BucketLODIdx][BucketIdx];

				bNeedAdjustments = true;
			}
		}
	}

#if WITH_MASSGAMEPLAY_DEBUG
	if (ProcessingLODIdx < EMassLOD::Max - 1)
	{
		// Save the count of this LOD for this frame
		Data.LastCalculatedLODCount[ProcessingLODIdx] = Count;
	}
#endif // WITH_MASSGAMEPLAY_DEBUG

	return bNeedAdjustments;
}


template <typename FLODLogic>
template <bool bCalculateVisibility, bool bCalculateVisibilityPerViewer, bool bMaximizeCountPerViewer>
bool TMassLODCalculator<FLODLogic>::AdjustDistancesFromCount()
{
	static_assert(!bCalculateVisibility || FLODLogic::bDoVisibilityLogic, "FLODLogic must have bDoVisibilityLogic enabled to calculate visibility.");
	static_assert(!bCalculateVisibilityPerViewer || (FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer), "FLODLogic must have bDoVisibilityLogic and bStoreInfoPerViewer enabled to calculate visibility per viewer.");
	static_assert(!bMaximizeCountPerViewer || FLODLogic::bMaximizeCountPerViewer, "FLODLogic must have bMaximizeCountPerViewer enabled to maximize count per viewer.");

	bool bDistanceAdjusted = false;
	if (bMaximizeCountPerViewer)
	{
		for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
		{
			if (!Viewers[ViewerIdx].Handle.IsValid())
			{
				continue;
			}

			bDistanceAdjusted |= AdjustDistancesFromCountForRuntimeData<bCalculateVisibilityPerViewer>(LODMaxCountPerViewer, RuntimeDataPerViewer[ViewerIdx]);
		}
	}

	bDistanceAdjusted |= AdjustDistancesFromCountForRuntimeData<bCalculateVisibility>(LODMaxCount, RuntimeData);
	return bDistanceAdjusted;
}

template <typename FLODLogic>
template <typename TViewerInfoFragment, typename TLODFragment, typename TPerViewerInfoFragment,
		  bool bCalculateLocalViewers, bool bCalculateVisibility, bool bCalculateLODPerViewer, bool bCalculateVisibilityPerViewer, bool bMaximizeCountPerViewer>
void TMassLODCalculator<FLODLogic>::AdjustLODFromCount(FMassExecutionContext& Context,
													   TConstArrayView<TViewerInfoFragment> ViewersInfoList,
													   TArrayView<TLODFragment> LODList,
													   TConstArrayView<TPerViewerInfoFragment> PerViewerInfoList)
{
	static_assert(!bCalculateVisibility || FLODLogic::bDoVisibilityLogic, "FLODLogic must have bDoVisibilityLogic enabled to calculate visibility.");
	static_assert(!bCalculateLODPerViewer || FLODLogic::bCalculateLODPerViewer, "FLODLogic must have bCalculateLODPerViewer enabled to calculate LOD Per viewer.");
	static_assert(!bCalculateVisibilityPerViewer || (FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer), "FLODLogic must have bDoVisibilityLogic and bStoreInfoPerViewer enabled to calculate visibility per viewer.");
	static_assert(!bMaximizeCountPerViewer || FLODLogic::bMaximizeCountPerViewer, "FLODLogic must have bMaximizeCountPerViewer enabled to maximize count per viewer.");

	// Adjust LOD for each viewer and remember the new highest
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const TViewerInfoFragment& EntityViewersInfo = ViewersInfoList[EntityIt];
		TLODFragment& EntityLOD = LODList[EntityIt];
		EMassLOD::Type HighestViewerLOD = EMassLOD::Off;
		if (bMaximizeCountPerViewer)
		{
			const TPerViewerInfoFragment& EntityPerViewerInfo = PerViewerInfoList[EntityIt];

			for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
			{
				const FViewerLODInfo& Viewer = Viewers[ViewerIdx];
				if (!Viewer.Handle.IsValid())
				{
					continue;
				}

				// Check to see if we want only local viewer only
				if (bCalculateLocalViewers && !Viewer.bLocal)
				{
					continue;
				}

				const float DistanceToViewerSq = GetDistanceToViewerSq<bMaximizeCountPerViewer>(EntityPerViewerInfo, ViewerIdx, FLT_MAX);
				// Using the prev visibility here as it was already updated for this frame in the CalculateLOD method and we want the previous one
				const bool bIsVisibleByViewer = GetPrevVisibilityPerViewer<bCalculateVisibilityPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max) == EMassVisibility::CanBeSeen;
				EMassLOD::Type LODPerViewer = GetLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EntityLOD.LOD);

				LODPerViewer = ComputeLODFromSettings<bCalculateVisibilityPerViewer>(LODPerViewer, DistanceToViewerSq, bIsVisibleByViewer, nullptr, RuntimeDataPerViewer[ViewerIdx]);

				if (LODPerViewer < HighestViewerLOD)
				{
					HighestViewerLOD = LODPerViewer;
				}

				SetLODPerViewer<bCalculateLODPerViewer>(EntityLOD, ViewerIdx, LODPerViewer);
			}
		}

		// Using the prev visibility here as it was already updated for this frame in the CalculateLOD method and we want the previous one
		const bool bIsVisibleByAViewer = GetPrevVisibility<bCalculateVisibility>(EntityLOD, EMassVisibility::Max) == EMassVisibility::CanBeSeen;
		EMassLOD::Type NewLOD = ComputeLODFromSettings<bCalculateVisibility>(EntityLOD.PrevLOD, EntityViewersInfo.ClosestViewerDistanceSq, bIsVisibleByAViewer, nullptr, RuntimeData);

		// Maybe the highest of all the viewers is now lower than the global entity LOD, make sure to update it accordingly
		if (bMaximizeCountPerViewer && NewLOD > HighestViewerLOD)
		{
			NewLOD = HighestViewerLOD;
		}
		if (EntityLOD.LOD != NewLOD)
		{
			EntityLOD.LOD = NewLOD;
			if (FLODLogic::bCalculateLODSignificance)
			{
				float LODSignificance = 0.f;
				if (NewLOD == EMassLOD::Off)
				{
					LODSignificance = float(EMassLOD::Off);
				}
				else
				{
					const TStaticArray<float, EMassLOD::Max>& AdjustedLODDistance = bCalculateVisibility && bIsVisibleByAViewer ? RuntimeData.AdjustedVisibleLODDistance : RuntimeData.AdjustedBaseLODDistance;

					// Need to clamp as the Sqrt is not precise enough and always end up with floating calculation errors
					const float DistanceBetweenLODThresholdAndEntity = FMath::Max(FMath::Sqrt(EntityViewersInfo.ClosestViewerDistanceSq) - AdjustedLODDistance[NewLOD], 0.f);

					// Derive significance from distance between viewer and where the agent stands between both LOD threshold
					const float AdjustedDistanceBetweenCurrentLODAndNext = AdjustedLODDistance[NewLOD + 1] - AdjustedLODDistance[NewLOD];
					const float PartialLODSignificance = DistanceBetweenLODThresholdAndEntity / AdjustedDistanceBetweenCurrentLODAndNext;
					LODSignificance = float(NewLOD) + PartialLODSignificance;
				}

				SetLODSignificance<FLODLogic::bCalculateLODSignificance>(EntityLOD, LODSignificance);
			}
		}
	}
}

template <typename FLODLogic>
template <typename TLODFragment>
void TMassLODCalculator<FLODLogic>::ForceOffLOD(FMassExecutionContext& Context, TArrayView<TLODFragment> LODList)
{
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		TLODFragment& EntityLOD = LODList[EntityIt];

		// Force off LOD
		EntityLOD.PrevLOD = EntityLOD.LOD;
		EntityLOD.LOD = EMassLOD::Off;

		// Set visibility as not calculated
		SetPrevVisibility<FLODLogic::bDoVisibilityLogic>(EntityLOD, EMassVisibility::Max);
		SetVisibility<FLODLogic::bDoVisibilityLogic>(EntityLOD, EMassVisibility::Max);

		if (FLODLogic::bStoreInfoPerViewer)
		{
			for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
			{
				if (!Viewers[ViewerIdx].Handle.IsValid())
				{
					continue;
				}

				SetLODPerViewer<FLODLogic::bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EMassLOD::Off);
				SetPrevLODPerViewer<FLODLogic::bCalculateLODPerViewer>(EntityLOD, ViewerIdx, EMassLOD::Off);
				SetPrevVisibilityPerViewer<FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max);
				SetVisibilityPerViewer<FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer>(EntityLOD, ViewerIdx, EMassVisibility::Max);
			}
		}

		SetLODSignificance<FLODLogic::bCalculateLODSignificance>(EntityLOD, float(EMassLOD::Off));
	}
}

#if WITH_MASSGAMEPLAY_DEBUG
template <typename FLODLogic>
template <typename TLODFragment, typename TTransformFragment>
void TMassLODCalculator<FLODLogic>::DebugDisplayLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UWorld* World)
{
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const TTransformFragment& EntityLocation = LocationList[EntityIt];
		const TLODFragment& EntityLOD = LODList[EntityIt];
		int32 LODIdx = (int32)EntityLOD.LOD;
		DrawDebugSolidBox(World, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 120.0f), FVector(25.0f), UE::MassLOD::LODColors[LODIdx]);
	}
}

template <typename FLODLogic>
template <typename TLODFragment, typename TTransformFragment>
void TMassLODCalculator<FLODLogic>::DebugDisplaySignificantLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UWorld* World, const float MaxLODSignificance)
{
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const TLODFragment& EntityLOD = LODList[EntityIt];
		if (EntityLOD.LODSignificance <= MaxLODSignificance)
		{
			const TTransformFragment& EntityLocation = LocationList[EntityIt];
			int32 LODIdx = (int32)EntityLOD.LOD;
			DrawDebugSolidBox(World, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 120.0f), FVector(25.0f), UE::MassLOD::LODColors[LODIdx]);
		}
	}
}

template <typename FLODLogic>
template <typename TLODFragment, typename TTransformFragment>
void TMassLODCalculator<FLODLogic>::VisLogLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UObject* LogOwner)
{
#if ENABLE_VISUAL_LOG
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const TTransformFragment& EntityLocation = LocationList[EntityIt];
		const TLODFragment& EntityLOD = LODList[EntityIt];
		int32 LODIdx = (int32)EntityLOD.LOD;
		UE_VLOG_LOCATION(LogOwner, LogMassLOD, Verbose, EntityLocation.GetTransform().GetLocation(), 20.0f, UE::MassLOD::LODColors[LODIdx], TEXT("%s %d"), *Context.GetEntity(EntityIt).DebugGetDescription(), LODIdx);
	}
#endif
}

template <typename FLODLogic>
template <typename TLODFragment, typename TTransformFragment>
void TMassLODCalculator<FLODLogic>::VisLogSignificantLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TConstArrayView<TTransformFragment> LocationList, UObject* LogOwner, const float MaxLODSignificance)
{
#if ENABLE_VISUAL_LOG
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const TLODFragment& EntityLOD = LODList[EntityIt];
		if (EntityLOD.LODSignificance <= MaxLODSignificance)
		{
			const TTransformFragment& EntityLocation = LocationList[EntityIt];
			int32 LODIdx = (int32)EntityLOD.LOD;
			UE_VLOG_LOCATION(LogOwner, LogMassLOD, Verbose, EntityLocation.GetTransform().GetLocation(), 20, UE::MassLOD::LODColors[LODIdx], TEXT("%s %d"), *Context.GetEntity(EntityIt).DebugGetDescription(), LODIdx);
		}
	}
#endif
}
#endif // WITH_MASSGAMEPLAY_DEBUG

template <typename FLODLogic>
template<bool bCalculateVisibility>
EMassLOD::Type TMassLODCalculator<FLODLogic>::ComputeLODFromSettings(const EMassLOD::Type PrevLOD, const float DistanceToViewerSq, const bool bIsVisible, bool* bIsInAVisibleRange, const FMassLODRuntimeData& Data) const
{
	const TStaticArray<float, EMassLOD::Max>& AdjustedLODDistanceSq = bCalculateVisibility && bIsVisible ? Data.AdjustedVisibleLODDistanceSq : Data.AdjustedBaseLODDistanceSq;
	int32 LODDistIdx = EMassLOD::Max - 1;
	for (; LODDistIdx > 0; LODDistIdx--)
	{
		if (DistanceToViewerSq >= AdjustedLODDistanceSq[LODDistIdx])
		{
			// Validate that we allow going to a single higher LOD level after considering an extended buffer hysteresis on distance for the LOD level we are about to quit to prevent oscillating LOD states
			if (PrevLOD != EMassLOD::Max && (PrevLOD - (EMassLOD::Type)LODDistIdx) == 1)
			{
				const TStaticArray<float, EMassLOD::Max>& AdjustedLODDistance = bCalculateVisibility && bIsVisible ? Data.AdjustedVisibleLODDistance : Data.AdjustedBaseLODDistance;
				float HysteresisDistance = FMath::Lerp(AdjustedLODDistance[LODDistIdx], AdjustedLODDistance[LODDistIdx + 1], 1.f - BufferHysteresisOnDistanceRatio);
				if (DistanceToViewerSq >= FMath::Square(HysteresisDistance))
				{
					// Keep old
					LODDistIdx = PrevLOD;
				}
			}

			break;
		}
	}

	EMassLOD::Type NewLOD = (EMassLOD::Type)LODDistIdx;
	if(bCalculateVisibility && bIsInAVisibleRange)
	{
		*bIsInAVisibleRange = bIsVisible ? (NewLOD != EMassLOD::Off) : (DistanceToViewerSq < Data.AdjustedVisibleLODDistanceSq[EMassLOD::Off]);
	}

	return NewLOD;
}