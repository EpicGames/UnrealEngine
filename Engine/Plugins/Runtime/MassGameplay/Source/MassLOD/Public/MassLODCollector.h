// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODLogic.h"
#include "MassExecutionContext.h"
#include "DrawDebugHelpers.h"


/**
 * Helper struct to collect needed information on each agent that will be needed later for LOD calculation
 *   Requires TTransformFragment fragment.
 *   Stores information in TViewerInfoFragment fragment.
 */
template <typename FLODLogic = FLODDefaultLogic >
struct TMassLODCollector : public FMassLODBaseLogic
{
	TMassLODCollector()
		: FMassLODBaseLogic(/*bShouldBuildFrustumData=*/FLODLogic::bDoVisibilityLogic)
	{}

	/**
	 * Prepares execution for the current frame, needed to be called before every execution
	 * @Param Viewers is the array of all the known viewers
	 */
	void PrepareExecution(TConstArrayView<FViewerInfo> Viewers);

	/**
	 * Collects the information for LOD calculation, called for each entity chunks
	 * Use next method when FLODLogic::bStoreInfoPerViewer is enabled
	 * @Param Context of the chunk execution
	 * @Param TransformList is the fragment transforms of the entities
	 * @Param ViewersInfoList is the fragment where to store source information for LOD calculation
	 */
	template <typename TTransformFragment, 
			  typename TViewerInfoFragment,
			  bool bCollectLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCollectDistanceToFrustum = FLODLogic::bDoVisibilityLogic>
	inline void CollectLODInfo(FMassExecutionContext& Context, 
									TConstArrayView<TTransformFragment> TransformList, 
									TArrayView<TViewerInfoFragment> ViewersInfoList)
	{
		CollectLODInfo<TTransformFragment,
			TViewerInfoFragment,
			/*TPerViewerInfoFragment*/void*,
			bCollectLocalViewers,
			bCollectDistanceToFrustum,
			/*bCollectDistancePerViewer*/false,
			/*bCollectDistanceToFrustumPerViewer*/false>(Context, TransformList, ViewersInfoList, TArrayView<void*>());
	}


	/**
	 * Collects the information for LOD calculation, called for each entity chunks
	 * Use this version when FLODLogic::bStoreInfoPerViewer is enabled
	 * It collects information per viewer into the PerViewerInfoList fragments
	 * @Param Context of the chunk execution
	 * @Param TransformList is the fragment transforms of the entities
	 * @Param ViewersInfoList is the fragment where to store source information for LOD calculation
	 * @Param PerViewerInfoList is the per viewer information
	 */
	template <typename TTransformFragment,
			  typename TViewerInfoFragment,
			  typename TPerViewerInfoFragment,
			  bool bCollectLocalViewers = FLODLogic::bLocalViewersOnly,
			  bool bCollectDistanceToFrustum = FLODLogic::bDoVisibilityLogic,
			  bool bCollectDistancePerViewer = FLODLogic::bStoreInfoPerViewer,
			  bool bCollectDistanceToFrustumPerViewer = FLODLogic::bDoVisibilityLogic && FLODLogic::bStoreInfoPerViewer>
	void CollectLODInfo(FMassExecutionContext& Context, 
						TConstArrayView<TTransformFragment> TransformList, 
						TArrayView<TViewerInfoFragment> ViewersInfoList, 
						TArrayView<TPerViewerInfoFragment> PerViewerInfoList);
};

template <typename FLODLogic>
void TMassLODCollector<FLODLogic>::PrepareExecution(TConstArrayView<FViewerInfo> ViewersInfo)
{
	CacheViewerInformation(ViewersInfo);
}

template <typename FLODLogic>
template <typename TTransformFragment, 
		  typename TViewerInfoFragment, 
		  typename TPerViewerInfoFragment,
		  bool bCollectLocalViewers,
		  bool bCollectDistanceToFrustum,
		  bool bCollectDistancePerViewer,
		  bool bCollectDistanceToFrustumPerViewer>
void TMassLODCollector<FLODLogic>::CollectLODInfo(FMassExecutionContext& Context, 
												  TConstArrayView<TTransformFragment> TransformList, 
												  TArrayView<TViewerInfoFragment> ViewersInfoList, 
												  TArrayView<TPerViewerInfoFragment> PerViewerInfoList)
{
	static TPerViewerInfoFragment DummyFragment;
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		float ClosestViewerDistanceSq = FLT_MAX;
		float ClosestDistanceToFrustum = FLT_MAX;
		const TTransformFragment& EntityTransform = TransformList[EntityIt];
		TViewerInfoFragment& EntityViewerInfo = ViewersInfoList[EntityIt];
		TPerViewerInfoFragment& EntityInfoPerViewer = FLODLogic::bStoreInfoPerViewer ? PerViewerInfoList[EntityIt] : DummyFragment;

		SetDistanceToViewerSqNum<bCollectDistancePerViewer>(EntityInfoPerViewer, Viewers.Num());
		SetDistanceToFrustumNum<bCollectDistanceToFrustumPerViewer>(EntityInfoPerViewer, Viewers.Num());
		for (int ViewerIdx = 0; ViewerIdx < Viewers.Num(); ++ViewerIdx)
		{
			const FViewerLODInfo& Viewer = Viewers[ViewerIdx];
			if (Viewer.bClearData)
			{
				SetDistanceToViewerSq<bCollectDistancePerViewer>(EntityInfoPerViewer, ViewerIdx, FLT_MAX);
				SetDistanceToFrustum<bCollectDistanceToFrustumPerViewer>(EntityInfoPerViewer, ViewerIdx, FLT_MAX);
			}

			// Check to see if we want only local viewer only
			if (bCollectLocalViewers && !Viewer.bLocal)
			{
				continue;
			}

			if (Viewer.Handle.IsValid())
			{
				const FVector& EntityLocation = EntityTransform.GetTransform().GetLocation();
				const FVector ViewerToEntity = EntityLocation - Viewer.Location;
				const float DistanceToViewerSq = static_cast<float>(ViewerToEntity.SizeSquared()); // float precision is acceptable for LOD
				if (ClosestViewerDistanceSq > DistanceToViewerSq)
				{
					ClosestViewerDistanceSq = DistanceToViewerSq;
				}
				SetDistanceToViewerSq<bCollectDistancePerViewer>(EntityInfoPerViewer, ViewerIdx, DistanceToViewerSq);

				if constexpr (bCollectDistanceToFrustum)
				{
					const float DistanceToFrustum = Viewer.Frustum.DistanceTo(EntityLocation);
					SetDistanceToFrustum<bCollectDistanceToFrustumPerViewer>(EntityInfoPerViewer, ViewerIdx, DistanceToFrustum);
					if (ClosestDistanceToFrustum > DistanceToFrustum)
					{
						ClosestDistanceToFrustum = DistanceToFrustum;
					}
				}
			}
		}
		EntityViewerInfo.ClosestViewerDistanceSq = ClosestViewerDistanceSq;
		SetClosestDistanceToFrustum<bCollectDistanceToFrustum>(EntityViewerInfo, ClosestDistanceToFrustum);
	}
}