// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassProcessor.h"
#include "MassLODCollector.h"
#include "MassEntityQuery.h"
#include "MassLODCollectorProcessor.generated.h"


struct FMassGenericCollectorLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVisibilityLogic = true,
	};
};

/*
 * LOD collector which combines collection of LOD information for both Viewer and Simulation LODing when possible.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "LOD Collector"))
class UMassLODCollectorProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSLOD_API UMassLODCollectorProcessor();

protected:
	MASSLOD_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSLOD_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	template <bool bLocalViewersOnly>
	void CollectLODForChunk(FMassExecutionContext& Context);

	template <bool bLocalViewersOnly>
	void ExecuteInternal(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

	TMassLODCollector<FMassGenericCollectorLogic> Collector;

	// Queries for visualization and simulation calculation
	/** All entities that are in visible range and are On LOD*/
	FMassEntityQuery EntityQuery_VisibleRangeAndOnLOD;
	/** All entities that are in visible range but are Off LOD */
	FMassEntityQuery EntityQuery_VisibleRangeOnly;
	/** All entities that are NOT in visible range but are On LOD */
	FMassEntityQuery EntityQuery_OnLODOnly;
	/** All entities that are Not in visible range and are at Off LOD */
	FMassEntityQuery EntityQuery_NotVisibleRangeAndOffLOD;
};
