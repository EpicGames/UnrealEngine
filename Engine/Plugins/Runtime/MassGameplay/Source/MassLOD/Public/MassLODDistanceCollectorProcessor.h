// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassProcessor.h"
#include "MassLODCollector.h"
#include "MassLODLogic.h"
#include "MassEntityQuery.h"
#include "MassLODDistanceCollectorProcessor.generated.h"

/*
 * LOD Distance collector which combines collection of LOD information for both Viewer and Simulation LODing.
 * This collector cares only about the entities' distance to LOD viewer location, nothing else. 
 * Matches MassDistanceLODProcessor logic which uses the same Calculator LODLogic
 */
UCLASS(MinimalAPI, meta = (DisplayName = "LOD Distance Collector"))
class UMassLODDistanceCollectorProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSLOD_API UMassLODDistanceCollectorProcessor();

protected:
	MASSLOD_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSLOD_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	template <bool bLocalViewersOnly>
	void CollectLODForChunk(FMassExecutionContext& Context);

	template <bool bLocalViewersOnly>
	void ExecuteInternal(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

	TMassLODCollector<FMassDistanceLODLogic> Collector;

	// Queries for visualization and simulation calculation
	/** All entities that are in relevant range and are On LOD*/
	FMassEntityQuery EntityQuery_RelevantRangeAndOnLOD;
	/** All entities that are in relevant range but are Off LOD */
	FMassEntityQuery EntityQuery_RelevantRangeOnly;
	/** All entities that are NOT in relevant range but are On LOD */
	FMassEntityQuery EntityQuery_OnLODOnly;
	/** All entities that are Not in relevant range and are at Off LOD */
	FMassEntityQuery EntityQuery_NotRelevantRangeAndOffLOD;
};
