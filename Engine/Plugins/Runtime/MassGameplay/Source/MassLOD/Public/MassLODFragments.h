// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODTypes.h"
#include "MassExecutionContext.h"
#include "MassLODFragments.generated.h"

USTRUCT()
struct FMassHighLODTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassMediumLODTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassLowLODTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassOffLODTag : public FMassTag
{
	GENERATED_BODY()
};

/*
 * Data fragment to store the calculated distances to viewers
 */
USTRUCT()
struct FMassViewerInfoFragment : public FMassFragment
{
	GENERATED_BODY()

	// Closest viewer distance
	UPROPERTY()
	float ClosestViewerDistanceSq = FLT_MAX;

	// Closest distance to frustum
	UPROPERTY()
	float ClosestDistanceToFrustum = FLT_MAX;
};

USTRUCT()
struct FMassVariableTickChunkFragment : public FMassChunkFragment
{
	GENERATED_BODY();

	bool ShouldTickThisFrame() const
	{
		return bShouldTickThisFrame;
	}

	float GetTimeUntilNextTick() const
	{
		return TimeUntilNextTick;
	}

	int32 GetLastChunkSerialModificationNumber() const
	{
		return LastChunkSerialModificationNumber;
	}

	EMassLOD::Type GetLOD() const
	{
		return LOD;
	}

	void SetLOD(EMassLOD::Type InLOD)
	{
		checkf(LOD == EMassLOD::Max, TEXT("Chunk LOD should never change, it is allowed to only set it once"))
		LOD = InLOD;
	}

	void Update(const bool bInShouldTickThisFrame, const float InTimeUntilNextTick, int32 ChunkSerialModificationNumber)
	{
		bShouldTickThisFrame = bInShouldTickThisFrame;
		TimeUntilNextTick = InTimeUntilNextTick;
		LastChunkSerialModificationNumber = ChunkSerialModificationNumber;
	}

private:
	UPROPERTY()
	bool bShouldTickThisFrame = true;

	UPROPERTY()
	TEnumAsByte<EMassLOD::Type> LOD = EMassLOD::Max;

	UPROPERTY()
	float TimeUntilNextTick = 0.0f;

	UPROPERTY()
	int32 LastChunkSerialModificationNumber = INDEX_NONE;
};

USTRUCT()
struct FMassCollectLODViewerInfoTag : public FMassTag
{
	GENERATED_BODY();
};

/*
 * Tag to use to trigger the collector processor that uses the LODCollector without Visibility Logic, so strictly based of distance
 */
USTRUCT()
struct FMassCollectDistanceLODViewerInfoTag : public FMassTag
{
	GENERATED_BODY();
};

USTRUCT()
struct FMassVisibilityCanBeSeenTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassVisibilityCulledByFrustumTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassVisibilityCulledByDistanceTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassVisualizationChunkFragment : public FMassChunkFragment
{
	GENERATED_BODY();

	/**
	 * Fetched existing chunk fragment to know if there is a possibility of an entity that is visible.
	 * In the case that there is not chunk information, we cannot assume that all entities are not visible.
	 *
	 * @param Context of the execution from the entity sub system
	 * @return true if there is a possibility that the chunk contains a visible entity
	 */
	static bool AreAnyEntitiesVisibleInChunk(const FMassExecutionContext& Context)
	{
		const FMassVisualizationChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassVisualizationChunkFragment>();
		return ChunkFragment == nullptr || ChunkFragment->AreAnyEntitiesVisible();
	}

	/**
	 * Returns if there could be a visible entities in that chunk
	 *
	 * @return true if that is the case
	 */
	bool AreAnyEntitiesVisible() const
	{
		return (Visibility != EMassVisibility::CulledByDistance && Visibility != EMassVisibility::CulledByFrustum) || bContainsNewlyVisibleEntity;
	}

	/**
	 * IsChunkHandledThisFrame
	 *
	 * This function is used by LOD collector query chunk filters to check that visual LOD will be updated this frame. 
	 * It defaults to false (no LOD update), if visualization chunk fragment is NOT present.
	 * 
	 * @param Context of the execution from the entity sub system
	 * @return true if the visual LOD will be updated this frame
	 */
	static bool IsChunkHandledThisFrame(const FMassExecutionContext& Context)
	{
		const FMassVisualizationChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassVisualizationChunkFragment>();
		return ChunkFragment != nullptr && ChunkFragment->ShouldUpdateVisualization();
	}

	/**
	 * ShouldUpdateVisualizationForChunk
	 * 
	 * This function is used by query chunk filters in processors that require variable visual LOD update. 
	 * It defaults to true (always updating) if visualization chunk fragment is NOT present.
	 *
	 * @param Context of the execution from the entity sub system
	 * @return true if the chunk should update the visual this frame
	 */
	static bool ShouldUpdateVisualizationForChunk(const FMassExecutionContext& Context)
	{
		const FMassVisualizationChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassVisualizationChunkFragment>();
		return ChunkFragment == nullptr || ChunkFragment->ShouldUpdateVisualization();
	}

	/**
	 * Representation type of all currently visible entities are always updated
	 * But as an optimization, we use a frequency check on the not visible one.
	 *
	 * @return true if we should update the representation type for this chunk
	 */
	bool ShouldUpdateVisualization() const
	{
		return Visibility != EMassVisibility::CulledByDistance || DeltaTime <= 0.0f;
	}

	void SetContainsNewlyVisibleEntity(bool bInContainsNewlyVisibleEntity)
	{
		if (bInContainsNewlyVisibleEntity)
		{
			checkfSlow(Visibility != EMassVisibility::CanBeSeen, TEXT("Something is not adding up, how can an entity be newly visible in a can be seen chunk?"));
			bContainsNewlyVisibleEntity = true;
		}
	}

	void SetVisibility(const EMassVisibility InVisibility)
	{
		checkf(Visibility == EMassVisibility::Max, TEXT("Chunk visibility should never change, it is allowed to only set it once"));
		Visibility = InVisibility;
	}

	EMassVisibility GetVisibility() const
	{
		return Visibility;
	}

	float GetDeltaTime() const
	{
		return DeltaTime;
	}

	void Update(float InDeltaTime)
	{
		bContainsNewlyVisibleEntity = false;
		DeltaTime = InDeltaTime;
	}

protected:

	/** Visibility of the current chunk, should never change */
	UPROPERTY()
	EMassVisibility Visibility = EMassVisibility::Max;

	/** Not visible chunks, might contains entity that are newly visible and not yet moved. */
	UPROPERTY()
	bool bContainsNewlyVisibleEntity = true;

	/** Not visible chunks delta time until next update */
	UPROPERTY()
	float DeltaTime = 0;
};
