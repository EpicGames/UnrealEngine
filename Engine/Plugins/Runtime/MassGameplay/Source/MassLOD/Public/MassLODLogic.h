// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODSubsystem.h"
#include "Containers/StaticArray.h"
#include "ConvexVolume.h"
#include "MassLODTypes.h"

#define UE_API MASSLOD_API

#define DECLARE_CONDITIONAL_MEMBER_ACCESSORS( Condition, MemberType, MemberName ) \
template <bool Condition, typename TemplateClass> static inline typename TEnableIf< Condition, MemberType>::Type Get##MemberName(TemplateClass& Obj, MemberType DefaultValue) { return Obj.MemberName; } \
template <bool Condition, typename TemplateClass> static inline typename TEnableIf<!Condition, MemberType>::Type Get##MemberName(TemplateClass& Obj, MemberType DefaultValue) { return DefaultValue; } \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName(TemplateClass& Obj, typename TEnableIf< Condition, MemberType>::Type Value) { Obj.MemberName = Value; } \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName(TemplateClass& Obj, typename TEnableIf<!Condition, MemberType>::Type Value) {}

#define DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS( Condition, MemberType, MemberName ) \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName##Num(TemplateClass& Obj, typename TEnableIf< Condition, int32>::Type Num) { Obj.MemberName.SetNum(Num); } \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName##Num(TemplateClass& Obj, typename TEnableIf<!Condition, int32>::Type Num) {} \
template <bool Condition, typename TemplateClass> static inline typename TEnableIf< Condition, MemberType>::Type Get##MemberName(TemplateClass& Obj, int32 Index, MemberType DefaultValue) { return Obj.MemberName[Index]; } \
template <bool Condition, typename TemplateClass> static inline typename TEnableIf<!Condition, MemberType>::Type Get##MemberName(TemplateClass& Obj, int32 Index, MemberType DefaultValue) { return DefaultValue; } \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName(TemplateClass& Obj, int32 Index, typename TEnableIf< Condition, MemberType>::Type Value) { Obj.MemberName[Index] = Value; } \
template <bool Condition, typename TemplateClass> static inline void Set##MemberName(TemplateClass& Obj, int32 Index, typename TEnableIf<!Condition, MemberType>::Type Value) {}

/**
 * Traits for LOD logic calculation behaviors
 */
struct FLODDefaultLogic
{
	enum
	{
		bStoreInfoPerViewer = false, // Enable to store all calculated information per viewer
		bCalculateLODPerViewer = false, // Enable to calculate and store the result LOD per viewer in the FMassLODResultInfo::LODPerViewer and FMassLODResultInfo::PrevLODPerViewer, requires bStoreInfoPerViewer to be true as well.
		bMaximizeCountPerViewer = false, // Enable to maximize count per viewer, requires a valid InLODMaxCountPerViewer parameter during initialization of TMassLODCalculator.
		bDoVisibilityLogic = false, // Enable to calculate visibility and apply its own LOD distances. Requires a valid InVisibleLODDistance parameter during initialization of TMassLODCalculator.
		bCalculateLODSignificance = false, // Enable to calculate and set the a more precise LOD floating point significance in member FMassLODResultInfo::LODSignificance.
		bLocalViewersOnly = false, // Enable to calculate LOD from LocalViewersOnly, otherwise will be done on all viewers.
	};
};

struct FMassSimulationLODLogic : public FLODDefaultLogic
{
};

struct FMassRepresentationLODLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVisibilityLogic = true,
		bCalculateLODSignificance = true,
		bLocalViewersOnly = true,
	};
};

struct FMassCombinedLODLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVisibilityLogic = true,
		bCalculateLODSignificance = true,
		bLocalViewersOnly = true,
	};
};

/** Simplest version of RepresentationLODLogic strictly based on Distance to Viewer
 *	Compared to FMassRepresentationLODLogic, we:
 *	* Do not care about doing the Visibility Logic
 *	* For now we will keep the Significance computation as it could allow for finer grained control later on
*/
struct FMassDistanceLODLogic : public FLODDefaultLogic
{
	enum
	{
		bCalculateLODSignificance = true,
		bLocalViewersOnly = true,
	};
};

/**
 * TMassLODCollector outputs
 *
struct FMassViewerInfoFragment
{
	// Closest viewer distance  (Always needed)
	float ClosestViewerDistanceSq;

	// Square distances to each valid viewers (Required when FLODLogic::bStoreInfoPerViewer is enabled)
	TArray<float> DistanceToViewerSq;
};

struct FMassInfoPerViewerFragment
{
	// Square distances to each valid viewers (Required when FLODLogic::bStoreInfoPerViewer is enabled)
	TArray<float> DistanceToViewerSq;

	// Distances to each valid viewers frustums (Required when FLODLogic::bDoVisibilityLogic and FLODLogic::bStoreInfoPerViewer are enabled)
	TArray<float> DistanceToFrustum;
};

*/

/**
 * TMassLODCalculator outputs
 *
 struct FMassLODFragment
{
	// LOD information
	TEnumAsByte<EMassLOD::Type> LOD;
	TEnumAsByte<EMassLOD::Type> PrevLOD;

	// Visibility information (Required when FLODLogic::bDoVisibilityLogic is enabled)
	EMassVisibility Visibility;
	EMassVisibility PrevVisibility

	// Floating point LOD value, scaling from 0 to 3, 0 highest LOD and 3 being completely off LOD 
	// (Required only when FLODLogic::bCalculateLODSignificance is enabled)
	float LODSignificance = 0.0f; // 

	// Per viewer LOD information (Required when FLODLogic::bCalculateLODPerViewer is enabled)
	TArray<EMassLOD::Type> LODPerViewer;
	TArray<EMassLOD::Type> PrevLODPerViewer;

	// Visibility information per viewer (Required when FLODLogic::bDoVisibilityLogic and FLODLogicbStoreInfoPerViewer are enabled)
	TArray<EMassVisibility> VisibilityPerViewer;
	TArray<EMassVisibility> PrevVisibilityPerViewer;
}
*/

/**
 * TMassLODTickRateController outputs
 *
 struct FMassVariableTickFragment
{
	// Accumulated DeltaTime
	float DeltaTime = 0.0f;
	float LastTickedTime = 0.0f;
};
*/

struct FViewerLODInfo
{
	/* Boolean indicating the viewer is local or not */
	bool bLocal = false;

	/* Boolean indicating the viewer data needs to be cleared */
	bool bClearData = false;

	/** The handle to the viewer */
	FMassViewerHandle Handle;

	/** Viewer location and looking direction */
	FVector Location;
	FVector Direction;

	/** Viewer frustum (will not include near and far planes) */
	FConvexVolume Frustum;
};

/**
 * Base struct for the LOD calculation helpers
 */
struct FMassLODBaseLogic
{
	FMassLODBaseLogic(bool bShouldBuildFrustumData)
		: bBuildFrustumData(bShouldBuildFrustumData)
	{}

protected:
	UE_API void CacheViewerInformation(TConstArrayView<FViewerInfo> ViewerInfos);

	/** Per viewer LOD information conditional fragment accessors */
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, EMassLOD::Type, LODPerViewer);
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, EMassLOD::Type, PrevLODPerViewer);

	/** LOD Significance conditional fragment accessors */
	DECLARE_CONDITIONAL_MEMBER_ACCESSORS(Condition, float, LODSignificance);

	/** Visibility conditional fragment accessors */
	DECLARE_CONDITIONAL_MEMBER_ACCESSORS(Condition, float, ClosestDistanceToFrustum);
	DECLARE_CONDITIONAL_MEMBER_ACCESSORS(Condition, EMassVisibility, Visibility);
	DECLARE_CONDITIONAL_MEMBER_ACCESSORS(Condition, EMassVisibility, PrevVisibility);

	/** Per viewer distance conditional fragment accessors */
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, float, DistanceToViewerSq);

	/** Per viewer visibility conditional fragment accessors */
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, float, DistanceToFrustum);
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, EMassVisibility, VisibilityPerViewer);
	DECLARE_CONDITIONAL_MEMBER_ARRAY_ACCESSORS(Condition, EMassVisibility, PrevVisibilityPerViewer);

	TArray<FViewerLODInfo> Viewers;

private:
	/** 
	 * Setting to false will prevent costly FViewerLODInfo.Frustum creation. Makes sense only if that data is not required.
	 * Note that this property is not expected to be changed at runtime.
	 */
	bool bBuildFrustumData = true;
};

#undef UE_API
