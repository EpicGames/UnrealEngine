// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "IndexedHandle.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLODTypes.h"
#include "MassProcessor.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLODSubsystem.generated.h"

class UMassLODSubsystem;
class AActor;
class APlayerController;

/*
 * Handle that lets you reference the concept of a viewer
 */
USTRUCT()
struct FMassViewerHandle : public FIndexedHandleBase
{
	GENERATED_BODY()

	friend class UMassLODSubsystem;
};

USTRUCT()
struct FViewerInfo
{
	GENERATED_BODY()

	FViewerInfo() = default;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewerInfo(const FViewerInfo& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(transient)
	TObjectPtr<AActor> ActorViewer = nullptr;
	
	FName StreamingSourceName;

#if WITH_EDITOR
	int8 EditorViewportClientIndex = INDEX_NONE;
#endif // WITH_EDITOR

	FMassViewerHandle Handle;
	uint32 HashValue = 0;

	FVector Location;
	FRotator Rotation;

	/** Effective horizontal field of view in degrees at the viewer's actual viewport. */
	float FOV = 90.0f;

	/** Effective viewport aspect ratio (W/H) used when constructing the cull frustum. */
	float AspectRatio = 16.0f / 9.0f;

	bool bEnabled = true;

	void Reset();

	bool IsLocal() const;

	MASSLOD_API APlayerController* GetPlayerController() const;
private:
	UE_DEPRECATED_FORGAME(5.4, "PlayerController member variable has been deprecated in favor of more generic ActorViewer. Use that instead.")
	TObjectPtr<APlayerController> PlayerController = nullptr;
};

/*
 * Manager responsible to manage and synchronized available viewers
 */
UCLASS(MinimalAPI, config = Mass, defaultconfig)
class UMassLODSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewerAdded, const FViewerInfo& ViewerInfo);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewerRemoved, const FViewerInfo& ViewerInfo);

public:
	/** Checks the validity of a viewer handle */
	bool IsValidViewer(const FMassViewerHandle& ViewerHandle) const { return GetValidViewerIdx(ViewerHandle) != INDEX_NONE; }

	/** Returns the index of the viewer if valid, otherwise INDEX_NONE is return */
	MASSLOD_API int32 GetValidViewerIdx(const FMassViewerHandle& ViewerHandle) const;

	/** Returns the array of viewers */
	const TArray<FViewerInfo>& GetViewers() const { return Viewers; }

	/** Synchronize the viewers if not done this frame and returns the updated array */
	MASSLOD_API const TArray<FViewerInfo>& GetSynchronizedViewers();

	/** Returns viewer handle from the PlayerController pointer */
	MASSLOD_API FMassViewerHandle GetViewerHandleFromActor(const AActor& Actor) const;

	/** Returns viewer handle from the streaming source name */
	MASSLOD_API FMassViewerHandle GetViewerHandleFromStreamingSource(const FName StreamingSourceName) const;

	/** Returns PlayerController pointer from the viewer handle */
	MASSLOD_API APlayerController* GetPlayerControllerFromViewerHandle(const FMassViewerHandle& ViewerHandle) const;

	/** Returns the delegate called when new viewer are added to the list */
	FOnViewerAdded& GetOnViewerAddedDelegate() { return OnViewerAddedDelegate;  }

	/** Returns the delegate called when viewer are removed from the list */
	FOnViewerRemoved& GetOnViewerRemovedDelegate() { return OnViewerRemovedDelegate; }

	MASSLOD_API void RegisterActorViewer(AActor& ActorViewer);
	MASSLOD_API void UnregisterActorViewer(AActor& ActorViewer);

	bool IsUsingPlayerPawnLocationInsteadOfCamera() const { return bUsePlayerPawnLocationInsteadOfCamera; }

#if WITH_MASSGAMEPLAY_DEBUG
	void DebugSetGatherPlayers(const bool bInValue) { bGatherPlayerControllers = bInValue; }
	void DebugSetUsePlayerPawnLocationInsteadOfCamera(const bool bInValue) { bUsePlayerPawnLocationInsteadOfCamera = bInValue; }
	MASSLOD_API void DebugUnregisterActorViewer();
#endif

protected:
	// USubsystem BEGIN
	MASSLOD_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSLOD_API virtual void Deinitialize() override;
	// USubsystem END

	/** Called at the start of the PrePhysics mass processing phase and calls SynchronizeViewers */ 
	MASSLOD_API void OnPrePhysicsPhaseStarted(float DeltaTime);

	/** Synchronizes the viewers from the engine PlayerController list */
	MASSLOD_API void SynchronizeViewers();

	/** Adds the given player as a viewer to the list and sends notification about addition */
	MASSLOD_API void AddPlayerViewer(APlayerController& PlayerController);

	/** Adds the given streaming source as a viewer to the list and sends notification about addition */
	MASSLOD_API void AddStreamingSourceViewer(const FName StreamingSourceName);

	MASSLOD_API void AddActorViewer(AActor& ActorViewer);

#if WITH_EDITOR
	/** Adds the editor viewport client (identified via an index) as a viewer to the list and sends notification about addition */
	MASSLOD_API void AddEditorViewer(const int32 HashValue, const int32 ClientIndex);
#endif // WITH_EDITOR

	/** Removes a viewer to the list and send notification about removal */
	MASSLOD_API void RemoveViewer(const FMassViewerHandle& ViewerHandle);

	/** Returns the next new viewer serial number */
	uint32 GetNextViewerSerialNumber() { return ViewerSerialNumberCounter++; }

	/** Player controller EndPlay callback, removing viewers from the list */
	UFUNCTION()
	MASSLOD_API void OnPlayerControllerEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason);

protected:
	/** If true, all PlayerControllers will be gathered as viewers for LOD calculations. */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|LOD", config)
	uint8 bGatherPlayerControllers : 1 = true;

	/** If true, all streaming sources will be gathered as viewers for LOD calculations. */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|LOD", config)
	uint8 bGatherStreamingSources : 1 = true;

	/** Whether using non-player actors as LOD Viewers is supported. */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|LOD", config)
	uint8 bAllowNonPlayerViwerActors : 1 = true;

	/** 
	 * If set to true will prefer to use Player-owned Pawn's location and rotation over Player's camera as the viewer's 
	 * location and rotation.
	 * Note that this works best with distance-only LOD and can introduce subtle inaccuracies if Frustum-based LOD is being used. 
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|LOD", config)
	uint8 bUsePlayerPawnLocationInsteadOfCamera : 1 = false;

private:
	/** Removes a viewer to the list and send notification about removal */
	MASSLOD_API void RemoveViewerInternal(const FMassViewerHandle& ViewerHandle);

	/** The actual array of viewer's information*/
	UPROPERTY(Transient)
	TArray<FViewerInfo> Viewers;

	/** The map that do reverse look up to get ViewerHandle */
	UPROPERTY(Transient)
	TMap<uint32, FMassViewerHandle> ViewerMap;

	UPROPERTY(Transient)
	TArray<TObjectPtr<AActor>> RegisteredActorViewers;

	uint64 LastSynchronizedFrame = 0;

	/** Viewer serial number counter */
	uint32 ViewerSerialNumberCounter = 0;

#if WITH_EDITOR
	bool bUseEditorLevelViewports = false;
	bool bIgnorePlayerControllersDueToSimulation = false;
#endif // WITH_EDITOR

	/** Free list of indices in the sparse viewer array */
	TArray<int32> ViewerFreeIndices;

	/** Delegates to notify anyone who needs to know about viewer changes */
	FOnViewerAdded OnViewerAddedDelegate;
	FOnViewerRemoved OnViewerRemovedDelegate;
};
