// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODLogic.h"
#include "MassLODUtils.h"
#include "MassCommonUtils.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommandBuffer.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

/**
 * Helper struct to control LOD tick rate for each agent, 
 * It will add a fragment tag to group the agent of the same LOD together, that way the user can do tick rate logic per chunk.
 */
template <typename TVariableTickChunkFragment, typename FLODLogic = FLODDefaultLogic >
struct TMassLODTickRateController : public FMassLODBaseLogic
{
public:
	TMassLODTickRateController()
		: FMassLODBaseLogic(/*bShouldBuildFrustumData=*/false)
	{}

	/**
	 * Initializes the LOD trick rate controller, needed to be called once at initialization time (Only when FLODLogic::bDoVariableTickRate is enabled)
	 * @Param InTickRate the rate at which entities should be ticked per 
	 * @Param bInShouldSpreadFirstUpdate over the period specified in InTickRate parameter
	 */
	void Initialize(const float InTickRate[EMassLOD::Max], const bool bInShouldSpreadFirstUpdate = false);

	/**
	 * Retrieve if it is needed to calculate the LOD for this chunk
	 *
	 * @return if the LOD needs to be calculated
	 */
	bool ShouldCalculateLODForChunk(const FMassExecutionContext& Context) const;

	/**
	 * Retrieve if it is needed to adjust LOD from the newly calculated count for this chunk
	 * 
	 * @return if the LOD needs to be adjusted
	 */
	bool ShouldAdjustLODFromCountForChunk(const FMassExecutionContext& Context) const;

	/**
	 * Updates tick rate for this chunk and its entities
	 * @param Context of the chunk execution 
	 * @param LODList is the fragment where calculation are stored 
	 * @param Time of the simulation to use for this update
	 * @return bool return if the chunk should be tick this frame
	 */
	template <typename TLODFragment, typename TVariableTickRateFragment>
	bool UpdateTickRateFromLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TArrayView<TVariableTickRateFragment> TickRateList, const double Time);

	friend inline uint32 GetTypeHash(const TMassLODTickRateController<TVariableTickChunkFragment, FLODLogic>& SharedFragmentInstance)
	{
		return HashCombineFast(GetTypeHash(SharedFragmentInstance.TickRates), uint32(SharedFragmentInstance.bShouldSpreadFirstUpdate));
	}

protected:

	/** Tick rate for each LOD */
	TStaticArray<float, EMassLOD::Max> TickRates;

	/* Whether or not to spread the first update over the period specified in tick rate member for its LOD */
	bool bShouldSpreadFirstUpdate = false;
};

template <typename TVariableTickChunkFragment, typename FLODLogic>
void TMassLODTickRateController<TVariableTickChunkFragment, FLODLogic>::Initialize(const float InTickRates[EMassLOD::Max], const bool bInShouldSpreadFirstUpdate/* = false*/)
{
	checkf(InTickRates, TEXT("You need to provide tick rate values to use this class."));

	// Make a copy of all the settings
	for (int x = 0; x < EMassLOD::Max; x++)
	{
		TickRates[x] = InTickRates[x];
	}

	bShouldSpreadFirstUpdate = bInShouldSpreadFirstUpdate;
}

template <typename TVariableTickChunkFragment, typename FLODLogic>
bool TMassLODTickRateController<TVariableTickChunkFragment, FLODLogic>::ShouldCalculateLODForChunk(const FMassExecutionContext& Context) const
{
	// EMassLOD::Off does not need to handle max count, so we can use ticking rate for them if available
	const FMassVariableTickChunkFragment& ChunkData = Context.GetChunkFragment<TVariableTickChunkFragment>();
	return ChunkData.GetLOD() != EMassLOD::Off || ChunkData.ShouldTickThisFrame();
}

template <typename TVariableTickChunkFragment, typename FLODLogic>
bool TMassLODTickRateController<TVariableTickChunkFragment, FLODLogic>::ShouldAdjustLODFromCountForChunk(const FMassExecutionContext& Context) const
{
	// EMassLOD::Off does not need to handle max count, so we can skip it
	const FMassVariableTickChunkFragment& ChunkData = Context.GetChunkFragment<TVariableTickChunkFragment>();
	return ChunkData.GetLOD() != EMassLOD::Off;
}

template <typename TVariableTickChunkFragment, typename FLODLogic>
template <typename TLODFragment, typename TVariableTickRateFragment>
bool TMassLODTickRateController<TVariableTickChunkFragment, FLODLogic>::UpdateTickRateFromLOD(FMassExecutionContext& Context, TConstArrayView<TLODFragment> LODList, TArrayView<TVariableTickRateFragment> TickRateList, const double Time)
{
	bool bShouldTickThisFrame = true;
	bool bWasChunkTicked = true;
	const float DeltaTime = Context.GetDeltaTimeSeconds();
	bool bFirstUpdate = false;

	FMassVariableTickChunkFragment& ChunkData = Context.GetMutableChunkFragment<TVariableTickChunkFragment>();
	EMassLOD::Type ChunkLOD = ChunkData.GetLOD();
	if (ChunkLOD == EMassLOD::Max)
	{
		// The LOD on the chunk fragment data isn't set yet, let see if the Archetype has an LOD tag and set it on the ChunkData
		ChunkLOD = UE::MassLOD::GetLODFromArchetype(Context);
		ChunkData.SetLOD(ChunkLOD);
		bFirstUpdate = bShouldSpreadFirstUpdate;
	}
	else
	{
		checkfSlow(UE::MassLOD::IsLODTagSet(Context, ChunkLOD), TEXT("Expecting the same LOD as what we saved in the chunk data, maybe external code is modifying the tags"))
	}

	if (ChunkLOD != EMassLOD::Max)
	{
		float TimeUntilNextTick = ChunkData.GetTimeUntilNextTick();
		bWasChunkTicked = ChunkData.ShouldTickThisFrame();

		const int32 LastChunkSerialModificationNumber = ChunkData.GetLastChunkSerialModificationNumber();
		const int32 ChunkSerialModificationNumber = Context.GetChunkSerialModificationNumber();

		// Prevent the chunk modification tracking logic to trigger a tick until we actually tick from the first update tick calculation
		int32 NewChunkSerialModificationNumber = (LastChunkSerialModificationNumber == INDEX_NONE) ? INDEX_NONE : ChunkSerialModificationNumber;

		const float TickRate = TickRates[ChunkLOD];
		if (bFirstUpdate)
		{
			// @todo: Add some randomization for deterministic runs too. The randomization is used to distribute the infrequent ticks evenly on different frames.
			TimeUntilNextTick = UE::Mass::Utils::IsDeterministic() ? TickRate * 0.5f : FMath::RandRange(0.0f, TickRate);
		}
		else if(bWasChunkTicked)
		{
			// Reset DeltaTime if we ticked last frame and start tracking chunk modifications
			// @todo: Add some randomization for deterministic runs too. The randomization is used to distribute the infrequent ticks evenly on different frames.
			TimeUntilNextTick = UE::Mass::Utils::IsDeterministic() ? TickRate : (TickRate * (1.0f + FMath::RandRange(-0.1f, 0.1f)));
			NewChunkSerialModificationNumber = ChunkSerialModificationNumber;
		}
		else
		{
			// Decrement delta time
			TimeUntilNextTick -= DeltaTime;
		}

		// Should we tick this frame?
		bShouldTickThisFrame = TimeUntilNextTick <= 0.0f || LastChunkSerialModificationNumber != NewChunkSerialModificationNumber;
		ChunkData.Update(bShouldTickThisFrame, TimeUntilNextTick, NewChunkSerialModificationNumber);
	}

	if (bWasChunkTicked)
	{
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const TLODFragment& EntityLOD = LODList[EntityIt];
			TVariableTickRateFragment& TickRate = TickRateList[EntityIt];
			TickRate.DeltaTime = TickRate.LastTickedTime != 0.0 ? static_cast<float>(Time - TickRate.LastTickedTime) : DeltaTime;
			TickRate.LastTickedTime = Time;
			if (EntityLOD.LOD != ChunkLOD)
			{
				const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
				UE::MassLOD::PushSwapTagsCommand(Context.Defer(), Entity, ChunkLOD, EntityLOD.LOD);
			}
		}
	}

	return bShouldTickThisFrame;
}
