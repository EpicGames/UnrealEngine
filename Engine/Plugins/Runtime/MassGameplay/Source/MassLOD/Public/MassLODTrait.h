// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassSimulationLOD.h"
#include "MassLODTrait.generated.h"

#define UE_API MASSLOD_API

UCLASS(MinimalAPI, meta = (DisplayName = "LODCollector"))
class UMassLODCollectorTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

public:
	/** Whether we should verify that the LOD collector processor associated with this trait is enabled by default*/
	UPROPERTY(Category="LOD", EditAnywhere, meta=(AdvancedDisplay), config)
	bool bTestCollectorProcessor = true;

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	UE_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const override;
};

// Simplest version of UMassLODCollectorTrait that will ensure collection strictly based on Distance from Viewer
UCLASS(MinimalAPI, meta = (DisplayName = "DistanceLODCollector"))
class UMassDistanceLODCollectorTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

public:
	/** Whether we should verify that the LOD collector processor associated with this trait is enabled by default*/
	UPROPERTY(Category="LOD", EditAnywhere, meta=(AdvancedDisplay), config)
	bool bTestCollectorProcessor = true;

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	UE_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const override;
};

UCLASS(MinimalAPI, meta = (DisplayName = "SimulationLOD"))
class UMassSimulationLODTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UPROPERTY(Category = "Config", EditAnywhere)
	FMassSimulationLODParameters Params;

	UPROPERTY(Category = "Config", EditAnywhere)
	bool bEnableVariableTicking = false;

	UPROPERTY(Category = "Config", EditAnywhere, meta = (EditCondition = "bEnableVariableTicking", EditConditionHides))
	FMassSimulationVariableTickParameters VariableTickParams;

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
