// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_OR
#include "MassLODTypes.generated.h"

/** Debug option to allow multiple viewers per controller. Useful for testing and profiling purposes */
#define UE_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER 0

#define UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER (UE_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER && !UE_BUILD_SHIPPING)

namespace UE::MassLOD
{
#if UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER
	constexpr int32 DebugNumberViewersPerController = 50;
#endif // UE_ALLOW_DEBUG_REPLICATION_DUPLICATE_VIEWERS_PER_CONTROLLER

	constexpr int32 MaxBucketsPerLOD = 250;

	extern MASSLOD_API FColor LODColors[];
} // UE::MassLOD

namespace UE::Mass::ProcessorGroupNames
{
	// Defined in MassLODSubsystem.cpp
	extern MASSLOD_API const FName LODCollector;
	extern MASSLOD_API const FName LOD;
}

// We are not using enum class here because we are doing so many arithmetic operation and comparison on them 
// that it is not worth polluting int32 casts everywhere in the code.
UENUM()
namespace EMassLOD
{
	enum Type : int
	{
		High,
		Medium,
		Low,
		Off,
		Max
	};
}


UENUM()
enum class EMassVisibility : uint8
{
	CanBeSeen, // Not too far and within camera frustum
	CulledByFrustum, // Not in camera frustum but within visibility distance
	CulledByDistance, // Too far whether in or out of frustum
	Max
};
