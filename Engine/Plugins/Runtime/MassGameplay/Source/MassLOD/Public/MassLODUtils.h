// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_OR
#include "MassCommonTypes.h"
#include "MassLODFragments.h"
#include "Logging/LogMacros.h"

MASSLOD_API DECLARE_LOG_CATEGORY_EXTERN(LogMassLOD, Log, All);

struct FMassCommandBuffer;

namespace UE::MassLOD 
{

inline EMassLOD::Type GetLODFromArchetype(const FMassExecutionContext& Context)
{
	if (Context.DoesArchetypeHaveTag<FMassOffLODTag>())
	{
		return EMassLOD::Off;
	}
	if (Context.DoesArchetypeHaveTag<FMassLowLODTag>())
	{
		return EMassLOD::Low;
	}
	if (Context.DoesArchetypeHaveTag<FMassMediumLODTag>())
	{
		return EMassLOD::Medium;
	}
	if (Context.DoesArchetypeHaveTag<FMassHighLODTag>())
	{
		return EMassLOD::High;
	}
	return EMassLOD::Max;
}

template <EMassLOD::Type Level>
struct TMassLODTagForLevel
{
	typedef FMassOffLODTag FTag;
};

template<>
struct TMassLODTagForLevel<EMassLOD::High>
{
	typedef FMassHighLODTag FTag;
};

template<>
struct TMassLODTagForLevel<EMassLOD::Medium>
{
	typedef FMassMediumLODTag FTag;
};

template<>
struct TMassLODTagForLevel<EMassLOD::Low>
{
	typedef FMassLowLODTag FTag;
};


inline const UScriptStruct* GetLODTagFromLOD(EMassLOD::Type LOD)
{
	switch (LOD)
	{
		case EMassLOD::High:
			return TMassLODTagForLevel<EMassLOD::High>::FTag::StaticStruct();
		case EMassLOD::Medium:
			return TMassLODTagForLevel<EMassLOD::Medium>::FTag::StaticStruct();
		case EMassLOD::Low:
			return TMassLODTagForLevel<EMassLOD::Low>::FTag::StaticStruct();
		case EMassLOD::Off:
			return TMassLODTagForLevel<EMassLOD::Off>::FTag::StaticStruct();
		default:
			checkf(false, TEXT("Unsupported LOD Type"));
		case EMassLOD::Max:
			return nullptr;
	}
}

inline bool IsLODTagSet(const FMassExecutionContext& Context, EMassLOD::Type LOD)
{
	switch (LOD)
	{
		case EMassLOD::High:
			return Context.DoesArchetypeHaveTag<FMassHighLODTag>();
		case EMassLOD::Medium:
			return Context.DoesArchetypeHaveTag<FMassMediumLODTag>();
		case EMassLOD::Low:
			return Context.DoesArchetypeHaveTag<FMassLowLODTag>();
		case EMassLOD::Off:
			return Context.DoesArchetypeHaveTag<FMassOffLODTag>();
		default:
			checkf(false, TEXT("Unsupported LOD Type"));
		case EMassLOD::Max:
			return false;
	}
}

void MASSLOD_API PushSwapTagsCommand(FMassCommandBuffer& CommandBuffer, const FMassEntityHandle Entity, const EMassLOD::Type PrevLOD, const EMassLOD::Type NewLOD);

#if WITH_MASSGAMEPLAY_DEBUG
namespace Debug 
{
	MASSLOD_API extern bool bLODCalculationsPaused;
} // UE::MassLOD::Debug
#endif // WITH_MASSGAMEPLAY_DEBUG
} // UE::MassLOD