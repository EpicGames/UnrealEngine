// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassLODCollector.h"
#include "MassLODCalculator.h"
#include "MassLODTickRateController.h"
#include "MassLODLogic.h"
#include "MassEntityTypes.h"
#include "MassLODFragments.h"
#include "MassSimulationLOD.generated.h"

#define UE_API MASSLOD_API

USTRUCT()
struct FMassSimulationLODFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Saved closest ViewerDistance */
	float ClosestViewerDistanceSq = FLT_MAX;

	/**LOD information */
	TEnumAsByte<EMassLOD::Type> LOD = EMassLOD::Max;

	/** Previous LOD information*/
	TEnumAsByte<EMassLOD::Type> PrevLOD = EMassLOD::Max;
};

USTRUCT()
struct FMassSimulationVariableTickFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Accumulated delta time to use upon next tick */
	double LastTickedTime = 0.;
	float DeltaTime = 0.0f;
};

USTRUCT()
struct FMassSimulationVariableTickChunkFragment : public FMassVariableTickChunkFragment
{
	GENERATED_BODY();

	/**
	 * IsChunkHandledThisFrame
	 * 
	 * This function is used by LOD collector query chunk filters to check if the Simulation LOD will be updated this frame.
	 * It defaults to false (no LOD update), if simulation variable tick chunk fragment is NOT present.
	 * 
	 * @return true if the simulation LOD will be updated this frame
	 */
	static bool IsChunkHandledThisFrame(const FMassExecutionContext& Context)
	{
		const FMassSimulationVariableTickChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassSimulationVariableTickChunkFragment>();
		return ChunkFragment != nullptr && ChunkFragment->ShouldTickThisFrame();
	}

	/**
	 * ShouldTickChunkThisFrame
	 * 
	 * This function is used by query chunk filters in processors that require variable rate ticking based on LOD.
	 * It defaults to true (always ticking) if simulation variable tick chunk fragment is NOT present.
	 * 
	 * @return if the chunk should be ticked this frame
	 */
	static bool ShouldTickChunkThisFrame(const FMassExecutionContext& Context)
	{
		const FMassSimulationVariableTickChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassSimulationVariableTickChunkFragment>();
		return ChunkFragment == nullptr || ChunkFragment->ShouldTickThisFrame();
	}

	static EMassLOD::Type GetChunkLOD(const FMassExecutionContext& Context)
	{
		const FMassSimulationVariableTickChunkFragment* ChunkFragment = Context.GetChunkFragmentPtr<FMassSimulationVariableTickChunkFragment>();
		return ChunkFragment ? ChunkFragment->GetLOD() : EMassLOD::High;
	}

};

USTRUCT()
struct FMassSimulationLODParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	UE_API FMassSimulationLODParameters();

	/** Distance where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "LOD", config)
	float LODDistance[EMassLOD::Max];

	/** Hysteresis percentage on delta between the LOD distances */
	UPROPERTY(EditAnywhere, Category = "LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float BufferHysteresisOnDistancePercentage = 10.0f;

	/** Maximum limit of entity per LOD */
	UPROPERTY(EditAnywhere, Category = "LOD", config)
	int32 LODMaxCount[EMassLOD::Max];

	/** If true, will set the associated LOD tag on the entity */
	UPROPERTY(EditAnywhere, Category = "LOD", config)
	bool bSetLODTags = false;
};

USTRUCT()
struct FMassSimulationVariableTickParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	UE_API FMassSimulationVariableTickParameters();

	/** Rate in seconds at which entities should update when in this LOD */
	UPROPERTY(EditAnywhere, Category = "VariableTick", config)
	float TickRates[EMassLOD::Max];

	/** If true, will spread the first simulation update over TickRate period */
	UPROPERTY(EditAnywhere, Category = "VariableTick", config)
	bool bSpreadFirstSimulationUpdate = false;
};

USTRUCT()
struct FMassSimulationLODSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	FMassSimulationLODSharedFragment() = default;
	UE_API FMassSimulationLODSharedFragment(const FMassSimulationLODParameters& LODParams);

	/** Runtime data for matching the simulation LOD parameters */
	TMassLODCalculator<FMassSimulationLODLogic> LODCalculator;
	bool bHasAdjustedDistancesFromCount = false;
};

USTRUCT()
struct FMassSimulationVariableTickSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	FMassSimulationVariableTickSharedFragment() = default;
	UE_API FMassSimulationVariableTickSharedFragment(const FMassSimulationVariableTickParameters& TickRateParams);

	/** Runtime data for matching the simulation tick rate parameters */
	TMassLODTickRateController<FMassSimulationVariableTickChunkFragment, FMassSimulationLODLogic> LODTickRateController;

	static bool ShouldCalculateLODForChunk(const FMassExecutionContext& Context)
	{
		if (const FMassSimulationVariableTickSharedFragment* TickRateSharedFragment = Context.GetSharedFragmentPtr<FMassSimulationVariableTickSharedFragment>())
		{
			return TickRateSharedFragment->LODTickRateController.ShouldCalculateLODForChunk(Context);
		}
		return true;
	}

	static bool ShouldAdjustLODFromCountForChunk(const FMassExecutionContext& Context)
	{
		if (const FMassSimulationVariableTickSharedFragment* TickRateSharedFragment = Context.GetSharedFragmentPtr<FMassSimulationVariableTickSharedFragment>())
		{
			return TickRateSharedFragment->LODTickRateController.ShouldAdjustLODFromCountForChunk(Context);
		}
		return true;
	}
};

UCLASS(MinimalAPI, meta = (DisplayName = "Simulation LOD"))
class UMassSimulationLODProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassSimulationLODProcessor();

protected:

	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	UE_API void CalculateLODForConfig(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSimulationLODParameters& LODParams);

	FMassEntityQuery EntityQuery;
	FMassEntityQuery EntityQueryCalculateLOD;
	FMassEntityQuery EntityQueryAdjustDistances;
	FMassEntityQuery EntityQueryVariableTick;
	FMassEntityQuery EntityQuerySetLODTag;

	UPROPERTY(config, EditDefaultsOnly, Category="Mass")
	bool bDoAdjustmentFromCount = true;
};

#undef UE_API
