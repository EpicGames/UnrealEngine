// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassMovement : ModuleRules
	{
		public MassMovement(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AIModule",
					"MassCore",
					"MassEntity",
					"Core",
					"CoreUObject",
					"Engine",
					"MassCommon",
					"MassLOD",
					"MassSignals",
					"MassSpawner",
					"NavigationSystem",
					"ZoneGraph",
					"ZoneGraphAnnotations",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
