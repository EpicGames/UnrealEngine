// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovementDelegates.h"

namespace UE::MassMovement::Delegates
{
#if WITH_EDITOR
	FOnMassMovementNamesChanged MASSMOVEMENT_API OnMassMovementNamesChanged;
#endif
} // UEUE::MassMovement::Delegates
