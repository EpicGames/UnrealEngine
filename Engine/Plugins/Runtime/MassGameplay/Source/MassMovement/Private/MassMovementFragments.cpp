// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovementFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovementFragments)

void FMassMovementParameters::Update()
{
	for (FMassMovementStyleParameters& Style : MovementStyles)
	{
		if (Style.DesiredSpeeds.IsEmpty())
		{
			continue;
		}

		// Calculate probability threshold for the speeds, so that a speed can be looked up based in a float in range [0...1]. 
		float Total = 0.0f;
		for (const FMassMovementStyleSpeedParameters& Speed : Style.DesiredSpeeds)
		{
			Total += Speed.Probability;
		}

		if (Total > KINDA_SMALL_NUMBER)
		{
			const float Scale = 1.0f / Total;
			float Sum = 0.0f;
			for (FMassMovementStyleSpeedParameters& Speed : Style.DesiredSpeeds)
			{
				Sum += Speed.Probability;
				Speed.ProbabilityThreshold = FMath::Min(Sum * Scale, 1.0f);
			}
		}
		else
		{
			const float Scale = 1.0f / static_cast<float>(Style.DesiredSpeeds.Num());
			float Sum = 0.0f;
			for (FMassMovementStyleSpeedParameters& Speed : Style.DesiredSpeeds)
			{
				Sum += 1.0f;
				Speed.ProbabilityThreshold = FMath::Min(Sum * Scale, 1.0f);
			}
		}
	}
}
