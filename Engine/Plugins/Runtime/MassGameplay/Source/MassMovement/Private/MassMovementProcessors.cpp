// Copyright Epic Games, Inc. All Rights Reserved.

#include "Movement/MassMovementProcessors.h"
#include "MassCommonUtils.h"
#include "MassCommandBuffer.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "Math/UnrealMathUtility.h"
#include "MassSimulationLOD.h"


//----------------------------------------------------------------------//
//  UMassApplyForceProcessor
//----------------------------------------------------------------------//

#include "MassSpringMovementFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovementProcessors)

UMassApplyForceProcessor::UMassApplyForceProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::ApplyForces;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassApplyForceProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassForceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);

	EntityQuery.AddConstSharedRequirement<FMassMovementParameters>(EMassFragmentPresence::All);
}

void UMassApplyForceProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	// Clamp max delta time to avoid force explosion on large time steps (i.e. during initialization).
	const float DeltaTime = FMath::Min(0.1f, Context.GetDeltaTimeSeconds());

	EntityQuery.ForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
	{
		const TArrayView<FMassForceFragment> ForceList = Context.GetMutableFragmentView<FMassForceFragment>();
		const TArrayView<FMassDesiredMovementFragment> MovementList = Context.GetMutableFragmentView<FMassDesiredMovementFragment>();

		const FMassMovementParameters& MovementParams = Context.GetConstSharedFragment<FMassMovementParameters>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassForceFragment& Force = ForceList[EntityIt];
			FMassDesiredMovementFragment& DesiredMovement = MovementList[EntityIt];

			// Update desired velocity from steering forces.
			const FVector DesiredVelocity  = DesiredMovement.DesiredVelocity + (Force.Value * DeltaTime);
			DesiredMovement.DesiredVelocity = DesiredVelocity.GetClampedToMaxSize(MovementParams.MaxSpeed);

			// Reset to zero after force is applied. Processors accumulate forces into the force fragment
			Force.Value = FVector::ZeroVector;
		}
	});
}

//----------------------------------------------------------------------//
//  UMassApplyMovementProcessor
//----------------------------------------------------------------------//

UMassApplyMovementProcessor::UMassApplyMovementProcessor()
	: EntityQuery(*this)
#if WITH_MASSGAMEPLAY_DEBUG
	, DebugEntityQuery(*this)
#endif
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Movement;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::ApplyForces);
}

void UMassApplyMovementProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FSpringMovementSettings>(EMassFragmentPresence::None);	// Do not apply simple movement processor if spring is being used
	EntityQuery.AddTagRequirement<FMassCustomMovementTag>(EMassFragmentPresence::None);	// Do not apply if a custom movement processor handles this entity
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassCodeDrivenMovementTag>(EMassFragmentPresence::All);

#if WITH_MASSGAMEPLAY_DEBUG
	// We have a separate processor because we want the debug entity query to run always, regardless of the FMassCodeDrivenMovementTag 
	DebugEntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
#endif
}

void UMassApplyMovementProcessor::Execute(FMassEntityManager& EntityManager,
													FMassExecutionContext& Context)
{
	// Clamp max delta time to avoid force explosion on large time steps (i.e. during initialization).
	const float DeltaTime = FMath::Min(0.1f, Context.GetDeltaTimeSeconds());

	EntityQuery.ForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
	{
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();
		const TConstArrayView<FMassDesiredMovementFragment> MovementList = Context.GetFragmentView<FMassDesiredMovementFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassVelocityFragment& Velocity = VelocityList[EntityIt];

			// For code driven, we just apply desired velocity as velocity
			// This is basically the equivalent of the character movement component
			// Smoothing / acceleration could be done here if we wanted
			Velocity.Value = MovementList[EntityIt].DesiredVelocity;
			
			FTransform& CurrentTransform = LocationList[EntityIt].GetMutableTransform();

#if WITH_MASSGAMEPLAY_DEBUG
			if (UE::MassMovement::bFreezeMovement)
			{
				Velocity.Value = FVector::ZeroVector;
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			FVector CurrentLocation = CurrentTransform.GetLocation();
			CurrentLocation += Velocity.Value * DeltaTime;
			CurrentTransform.SetTranslation(CurrentLocation);
		}
	});

#if WITH_MASSGAMEPLAY_DEBUG
	DebugEntityQuery.ForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
	{
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassVelocityFragment& Velocity = VelocityList[EntityIt];

			// Keep as "expected value" for next frame.
			Velocity.DebugPreviousValue = Velocity.Value;
		}
	});
#endif // WITH_MASSGAMEPLAY_DEBUG
}
