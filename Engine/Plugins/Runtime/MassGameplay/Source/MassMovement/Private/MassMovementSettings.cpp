// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovementSettings.h"
#include "MassMovementDelegates.h"
#include "MassMovementTypes.h"

//----------------------------------------------------------------------//
// UMassMovementSettings
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovementSettings)

UMassMovementSettings::UMassMovementSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Create default style
	FMassMovementStyle& Style = MovementStyles.AddDefaulted_GetRef();
	Style.Name = FName(TEXT("Default"));
	Style.ID = FGuid::NewGuid();
}


#if WITH_EDITOR
void UMassMovementSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMassMovementSettings, MovementStyles))
		{
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(MemberProperty->GetFName().ToString());

			// Ensure unique ID on duplicated items.
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
			{
				if (MovementStyles.IsValidIndex(ArrayIndex))
				{
					MovementStyles[ArrayIndex].ID = FGuid::NewGuid();
					MovementStyles[ArrayIndex].Name = FName(TEXT("Movement Style"));
				}
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				if (MovementStyles.IsValidIndex(ArrayIndex))
				{
					MovementStyles[ArrayIndex].ID = FGuid::NewGuid();
					MovementStyles[ArrayIndex].Name = FName(MovementStyles[ArrayIndex].Name.ToString() + TEXT(" Duplicate"));
				}
			}

			UE::MassMovement::Delegates::OnMassMovementNamesChanged.Broadcast();
		}
	}
}
#endif // WITH_EDITOR

const FMassMovementStyle* UMassMovementSettings::GetMovementStyleByID(const FGuid ID) const
{
	return MovementStyles.FindByPredicate([ID](const FMassMovementStyle& Style) { return Style.ID == ID; });
}

