// Copyright Epic Games, Inc. All Rights Reserved.
#include "Movement/MassMovementTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "MassMovementFragments.h"
#include "MassMovementTypes.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovementTrait)


void UMassMovementTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.RequireFragment<FAgentRadiusFragment>();
	BuildContext.RequireFragment<FTransformFragment>();

	BuildContext.AddFragment<FMassVelocityFragment>();
	BuildContext.AddFragment<FMassForceFragment>();
	BuildContext.AddFragment<FMassDesiredMovementFragment>();

	const FMassMovementParameters MovementValidated = Movement.GetValidated();
	const FConstSharedStruct MovementFragment = EntityManager.GetOrCreateConstSharedFragment(MovementValidated);
	BuildContext.AddConstSharedFragment(MovementFragment);

	if (Movement.bIsCodeDrivenMovement)
	{
		BuildContext.AddTag<FMassCodeDrivenMovementTag>();
	}
}
