// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovementTypes.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovementTypes)

namespace UE::Mass::ProcessorGroupNames
{
	const FName SpringMovement = TEXT("SpringMovement");
}

namespace UE::MassMovement
{
	int32 bFreezeMovement = 0;
	FAutoConsoleVariableRef CVarFreezeMovement(TEXT("mass.debug.FreezeMovement"), bFreezeMovement, TEXT("Freeze any movement by common movement processors."));

} // UE::MassMovement
