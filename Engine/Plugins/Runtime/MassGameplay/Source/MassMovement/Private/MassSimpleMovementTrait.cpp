// Copyright Epic Games, Inc. All Rights Reserved.

#include "Example/MassSimpleMovementTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "Engine/World.h"
#include "MassMovementFragments.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassSimulationLOD.h"


//----------------------------------------------------------------------//
//  UMassSimpleMovementTrait
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSimpleMovementTrait)
void UMassSimpleMovementTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FTransformFragment>();
	BuildContext.AddFragment<FMassVelocityFragment>();
	BuildContext.AddTag<FMassSimpleMovementTag>();	
}

//----------------------------------------------------------------------//
//  UMassSimpleMovementProcessor
//----------------------------------------------------------------------//
UMassSimpleMovementProcessor::UMassSimpleMovementProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Avoidance;
}

void UMassSimpleMovementProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassSimpleMovementTag>(EMassFragmentPresence::All);

	EntityQuery.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.SetChunkFilter(&FMassSimulationVariableTickChunkFragment::ShouldTickChunkThisFrame);
}

void UMassSimpleMovementProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	EntityQuery.ForEachEntityChunk(ExecutionContext, ([this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassVelocityFragment> VelocitiesList = Context.GetFragmentView<FMassVelocityFragment>();
			const TArrayView<FTransformFragment> TransformsList = Context.GetMutableFragmentView<FTransformFragment>();
			const TConstArrayView<FMassSimulationVariableTickFragment> SimVariableTickList = Context.GetFragmentView<FMassSimulationVariableTickFragment>();
			const bool bHasVariableTick = (SimVariableTickList.Num() > 0);
			const float WorldDeltaTime = Context.GetDeltaTimeSeconds();
		
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassVelocityFragment& Velocity = VelocitiesList[EntityIt];
				FTransform& Transform = TransformsList[EntityIt].GetMutableTransform();
				const float DeltaTime = bHasVariableTick ? SimVariableTickList[EntityIt].DeltaTime : WorldDeltaTime;
				Transform.SetTranslation(Transform.GetTranslation() + Velocity.Value * DeltaTime);
			}
		}));
}
