// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpringMovementProcessors.h"

#include "MassMovementFragments.h"
#include "MassExecutionContext.h"
#include "Math/SpringMath.h"
#include "MassCommonTypes.h"
#include "MassMovementTypes.h"
#include "MassSpringMovementFragments.h"
#include "MassLODFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpringMovementProcessors)

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// USpringUpdateProcessor — runs the spring damper, does not move the character

USpringUpdateProcessor::USpringUpdateProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::ApplyForces);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Avoidance);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SpringMovement;
}

void USpringUpdateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// Inputs
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FSpringMovementSettings>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);

	// Outputs
	EntityQuery.AddRequirement<FSpringMovementRuntime>(EMassFragmentAccess::ReadWrite);
}

void USpringUpdateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	const float DeltaTime = Context.GetDeltaTimeSeconds();

	EntityQuery.ForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const FSpringMovementSettings& SpringSettings = Context.GetConstSharedFragment<FSpringMovementSettings>();
			const TConstArrayView<FMassDesiredMovementFragment> InputList = Context.GetFragmentView<FMassDesiredMovementFragment>();
			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FSpringMovementRuntime> RuntimeList = Context.GetMutableFragmentView<FSpringMovementRuntime>();

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				const FTransformFragment& TransformFragment = TransformList[EntityIndex];
				FSpringMovementRuntime& RuntimeFragment = RuntimeList[EntityIndex];
				const FMassDesiredMovementFragment& InputFragment = InputList[EntityIndex];

				// Update Position + Velocity
				RuntimeFragment.CurrentPosition = TransformFragment.GetTransform().GetLocation();
				SpringMath::SpringCharacterUpdate(RuntimeFragment.CurrentPosition, RuntimeFragment.CurrentVelocity, RuntimeFragment.CurrentAccel, InputFragment.DesiredVelocity, SpringSettings.VelocitySmoothingTime, DeltaTime, SpringSettings.VelocityDeadzoneThreshold);

				// Update Rotation
				RuntimeFragment.CurrentFacing = TransformFragment.GetTransform().GetRotation();
				SpringMath::CriticalSpringDamperQuat(RuntimeFragment.CurrentFacing, RuntimeFragment.CurrentAngularVelocity, InputFragment.DesiredFacing, SpringSettings.FacingSmoothingTime, DeltaTime);
			}
		});
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// USpringApplyMovementProcessor — applies spring state to the character transform

USpringApplyMovementProcessor::USpringApplyMovementProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::UpdateWorldFromMass);
}

void USpringApplyMovementProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// Inputs
	EntityQuery.AddRequirement<FSpringMovementRuntime>(EMassFragmentAccess::ReadOnly);

	// Outputs
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);

	// Skip entities using custom movement (e.g. trajectory movement)
	EntityQuery.AddTagRequirement<FMassCustomMovementTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
}

void USpringApplyMovementProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const TConstArrayView<FSpringMovementRuntime> RuntimeList = Context.GetFragmentView<FSpringMovementRuntime>();
			const TArrayView<FTransformFragment> TransformList = Context.GetMutableFragmentView<FTransformFragment>();
			const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				const FSpringMovementRuntime& RuntimeFragment = RuntimeList[EntityIndex];
				FTransformFragment& TransformFragment = TransformList[EntityIndex];
				FMassVelocityFragment& VelocityFragment = VelocityList[EntityIndex];

				TransformFragment.GetMutableTransform().SetTranslation(RuntimeFragment.CurrentPosition);
				TransformFragment.GetMutableTransform().SetRotation(RuntimeFragment.CurrentFacing);

#if WITH_MASSGAMEPLAY_DEBUG
				VelocityFragment.DebugPreviousValue = VelocityFragment.Value;
#endif
				VelocityFragment.Value = RuntimeFragment.CurrentVelocity;
			}
		});
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// USpringMovementLODResyncObserver — resyncs spring state when entity transitions back on-LOD

USpringMovementLODResyncObserver::USpringMovementLODResyncObserver()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassOffLODTag::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::RemoveElement;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void USpringMovementLODResyncObserver::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FSpringMovementRuntime>(EMassFragmentAccess::ReadWrite);
}

void USpringMovementLODResyncObserver::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			const int32 NumEntities = Context.GetNumEntities();
			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
			const TArrayView<FSpringMovementRuntime> RuntimeList = Context.GetMutableFragmentView<FSpringMovementRuntime>();

			for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
			{
				const FTransformFragment& TransformFragment = TransformList[EntityIndex];
				const FMassVelocityFragment& VelocityFragment = VelocityList[EntityIndex];
				FSpringMovementRuntime& RuntimeFragment = RuntimeList[EntityIndex];

				RuntimeFragment.CurrentPosition = TransformFragment.GetTransform().GetLocation();
				RuntimeFragment.CurrentVelocity = VelocityFragment.Value;
				RuntimeFragment.CurrentAccel = FVector::ZeroVector;
				RuntimeFragment.CurrentFacing = TransformFragment.GetTransform().GetRotation();
				RuntimeFragment.CurrentAngularVelocity = FVector::ZeroVector;
			}
		});
}
