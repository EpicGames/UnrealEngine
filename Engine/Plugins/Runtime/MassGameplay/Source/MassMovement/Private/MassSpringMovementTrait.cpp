// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpringMovementTrait.h"

#include "MassEntityTemplateRegistry.h"
#include "MassMovementFragments.h"
#include "MassSpringMovementFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpringMovementTrait)

void USpringMovementTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FSpringMovementRuntime>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);
	const FConstSharedStruct SpringSettingsFragment = EntityManager.GetOrCreateConstSharedFragment(SpringSettings);
	BuildContext.AddConstSharedFragment(SpringSettingsFragment);
}

