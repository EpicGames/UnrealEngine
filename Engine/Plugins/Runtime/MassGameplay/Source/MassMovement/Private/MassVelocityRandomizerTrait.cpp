// Copyright Epic Games, Inc. All Rights Reserved.

#include "Example/MassVelocityRandomizerTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"
#include "MassMovementFragments.h"
#include "MassCommonUtils.h"

//----------------------------------------------------------------------//
//  UMassVelocityRandomizerTrait
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassVelocityRandomizerTrait)
void UMassVelocityRandomizerTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassVelocityFragment& VelocityTemplate = BuildContext.AddFragment_GetRef<FMassVelocityFragment>();
	// This is a small @hack to support sending parameters to initializer.
	// A proper solution will allow users to specify a "lambda" initializer that will be used during creation
	VelocityTemplate.Value.X = MinSpeed;
	VelocityTemplate.Value.Y = MaxSpeed;
	VelocityTemplate.Value.Z = bSetZComponent ? 1.f : 0.f;
}

//----------------------------------------------------------------------//
//  UMassRandomVelocityInitializer
//----------------------------------------------------------------------//
UMassRandomVelocityInitializer::UMassRandomVelocityInitializer()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassVelocityFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
	RandomStream.Initialize(UE::Mass::Utils::GenerateRandomSeed());
}

void UMassRandomVelocityInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassRandomVelocityInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	// note: the author is aware that the vectors produced below are not distributed uniformly, but it's good enough
	EntityQuery.ForEachEntityChunk(Context, ([this](FMassExecutionContext& Context)
		{
			const TArrayView<FMassVelocityFragment> VelocitiesList = Context.GetMutableFragmentView<FMassVelocityFragment>();
			for (FMassVelocityFragment& VelocityFragment : VelocitiesList)
			{
				// the given VelocityFragment's value is encoding the initialization parameters, as per comment in
				// UMassVelocityRandomizerTrait::BuildTemplate
				const FVector RandomVector = VelocityFragment.Value.Z != 0.f 
					? FVector(RandomStream.FRandRange(-1.f, 1.f), RandomStream.FRandRange(-1.f, 1.f), RandomStream.FRandRange(-1.f, 1.f)).GetSafeNormal()
					: FVector(RandomStream.FRandRange(-1.f, 1.f), RandomStream.FRandRange(-1.f, 1.f), 0).GetSafeNormal();

				VelocityFragment.Value = RandomVector * RandomStream.FRandRange(VelocityFragment.Value.X, VelocityFragment.Value.Y);
			}
		}));
}
