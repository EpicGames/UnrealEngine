// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassEntityTypes.h"
#include "MassEntityTraitBase.h"
#include "MassProcessor.h"
#include "MassSimpleMovementTrait.generated.h"

#define UE_API MASSMOVEMENT_API


USTRUCT()
struct FMassSimpleMovementTag : public FMassTag
{
	GENERATED_BODY()
};


UCLASS(MinimalAPI, meta = (DisplayName = "Simple Movement"))
class UMassSimpleMovementTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};


UCLASS(MinimalAPI)
class UMassSimpleMovementProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassSimpleMovementProcessor();
		
protected:	
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
