// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassEntityTraitBase.h"
#include "MassObserverProcessor.h"
#include "MassVelocityRandomizerTrait.generated.h"

#define UE_API MASSMOVEMENT_API


class UMassRandomVelocityInitializer;

UCLASS(MinimalAPI, meta = (DisplayName = "Velocity Randomizer"))
class UMassVelocityRandomizerTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	/** The speed is expressed in UnrealUnits per second, which usually translates to 0.01m/s */
	UPROPERTY(Category = "Velocity", EditAnywhere, meta = (UIMin = 0.0, ClampMin = 0.0))
	float MinSpeed = 0.f;

	/** The speed is expressed in UnrealUnits per second, which usually translates to 0.01m/s */
	UPROPERTY(Category = "Velocity", EditAnywhere, meta = (UIMin = 1.0, ClampMin = 1.0))
	float MaxSpeed = 200.f;

	UPROPERTY(Category = "Velocity", EditAnywhere)
	bool bSetZComponent = false;
};


UCLASS(MinimalAPI)
class UMassRandomVelocityInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassRandomVelocityInitializer();

	UE_API void SetParameters(const float InMinSpeed, const float InMaxSpeed, const bool bInSetZComponent);

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;

	UPROPERTY()
	float MinSpeed = 0.f;

	/** The default max is set to 0 to enforce explicit configuration via SetParameters call. */
	UPROPERTY()
	float MaxSpeed = 0.f;

	UPROPERTY()
	bool bSetZComponent = false;
	FRandomStream RandomStream;
};

#undef UE_API
