// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

namespace UE::MassMovement::Delegates
{
#if WITH_EDITOR
	/** Called when movement names have changed (UI update). */
	DECLARE_MULTICAST_DELEGATE(FOnMassMovementNamesChanged);
	extern MASSMOVEMENT_API FOnMassMovementNamesChanged OnMassMovementNamesChanged;
#endif // WITH_EDITOR
} // UE::MassMovement::Delegates
