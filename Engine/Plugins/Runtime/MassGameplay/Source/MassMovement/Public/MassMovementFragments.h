// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#include "MassEntityTypes.h"
#include "MassMovementTypes.h"
#include "RandomSequence.h"
#include "MassMovementFragments.generated.h"

/**
 * This represents the actual physical velocity of the mass entity in the world
 * For agents with an actor representation, this is the velocity of the movement component
 */
USTRUCT()
struct FMassVelocityFragment : public FMassFragment
{
	GENERATED_BODY()

	FVector Value = FVector::ZeroVector;

#if WITH_MASSGAMEPLAY_DEBUG
	FVector DebugPreviousValue = FVector::ZeroVector;
#endif	
};

/**
 * This is the output of all processors that intend to affect movement
 * It is the input to the movement system (e.g. mover, animation etc.)
 */
USTRUCT()
struct FMassDesiredMovementFragment : public FMassFragment
{
	GENERATED_BODY()
	
	FVector DesiredVelocity = FVector::ZeroVector;
	FQuat DesiredFacing = FQuat::Identity;

	float DesiredMaxSpeedOverride = FLT_MAX;
};

/** Accumulator for steering / avoidance forces to apply to the desired velocity */ 
USTRUCT()
struct FMassForceFragment : public FMassFragment
{
	GENERATED_BODY()

	FVector Value = FVector::ZeroVector;
};

/**
 * The presence of this tag indicates that this mass agent's velocity should be
 * controlled by the FMassDesiredMovementFragment.
 *
 * For code driven displacement, we want the desired velocity to affect the velocity
 * directly, which is then applied to the character mover.
 * For e.g. Root Motion driven displacement, we just need to pipe the DesiredVelocity to the
 * animation system and let it do the test.
 */
USTRUCT()
struct FMassCodeDrivenMovementTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * The presence of this tag indicates that a custom movement processor handles
 * this entity's position updates. Default movement processors (UMassApplyMovementProcessor,
 * USpringApplyMovementProcessor) will skip entities with this tag.
 */
USTRUCT()
struct FMassCustomMovementTag : public FMassTag
{
	GENERATED_BODY()
};

/** Parameters describing how this mass agent should move */
USTRUCT()
struct FMassMovementParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassMovementParameters GetValidated() const
	{
		FMassMovementParameters Copy = *this;
		Copy.Update();
		return Copy;
	}

	/** Updates internal values for faster desired speed generation. */
	MASSMOVEMENT_API void Update();

	/** Generates desired speed based on style and unique id. The id is used deterministically assign a specific speed range. */
	float GenerateDesiredSpeed(const FMassMovementStyleRef& Style, const int32 UniqueId) const
	{
		float DesiredSpeed = DefaultDesiredSpeed;
		float DesiredSpeedVariance = DefaultDesiredSpeedVariance;
		
		const FMassMovementStyleParameters* StyleParams = MovementStyles.FindByPredicate([&Style](const FMassMovementStyleParameters& Config)
			{
				return Config.Style.ID == Style.ID;
			});
		
		if (StyleParams != nullptr)
		{
			const float Prob = UE::RandomSequence::FRand(UniqueId);
			for (const FMassMovementStyleSpeedParameters& Speed : StyleParams->DesiredSpeeds)
			{
				if (Prob < Speed.ProbabilityThreshold)
				{
					DesiredSpeed = Speed.Speed;
					DesiredSpeedVariance = Speed.Variance;
					break;
				}
			}
		}
		
		return DesiredSpeed * UE::RandomSequence::RandRange(UniqueId, 1.0f - DesiredSpeedVariance, 1.0f + DesiredSpeedVariance);;
	}
	
	/** Maximum speed (cm/s). */
	UPROPERTY(EditAnywhere, Category = "Movement", meta = (ClampMin = "0", ForceUnits="cm/s"))
	float MaxSpeed = 200.f;

	/** 200..600 Smaller steering maximum acceleration makes the steering more \"calm\" but less opportunistic, may not find solution, or gets stuck. */
	UPROPERTY(config, EditAnywhere, Category = "Movement", meta = (UIMin = 0.0, ClampMin = 0.0, ForceUnits="cm/s^2"))
	float MaxAcceleration = 250.f;

	/** Default desired speed (cm/s). */
	UPROPERTY(EditAnywhere, Category = "Movement", meta = (ClampMin = "0", ForceUnits="cm/s"))
	float DefaultDesiredSpeed = 140.f;

	/** How much default desired speed is varied randomly. */
	UPROPERTY(EditAnywhere, Category = "Movement", meta = (ClampMin = "0", ClampMax = "1"))
	float DefaultDesiredSpeedVariance = 0.1f;

	/** The time it takes the entity position to catchup to the requested height. */
	UPROPERTY(EditAnywhere, Category = "Movement", meta = (ClampMin = "0.0", ForceUnits="s"))
	float HeightSmoothingTime = 0.2f;

	/** List of supported movement styles for this configuration. */
	UPROPERTY(EditAnywhere, Category = "Movement")
	TArray<FMassMovementStyleParameters> MovementStyles;

	/** Indicate whether mass AI has direct control over the mass agent's velocity. If true, desired velocity will be written directly to velocity every frame */
	UPROPERTY(EditAnywhere, Category = "Movement")
	bool bIsCodeDrivenMovement = true;
};
