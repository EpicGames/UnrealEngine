// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSettings.h"
#include "MassMovementTypes.h"
#include "MassMovementSettings.generated.h"

UCLASS(MinimalAPI, config = Mass, defaultconfig, meta = (DisplayName = "Mass Movement"))
class UMassMovementSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	MASSMOVEMENT_API UMassMovementSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	static const UMassMovementSettings* Get()
	{
		return GetDefault<UMassMovementSettings>();
	}

	TConstArrayView<FMassMovementStyle> GetMovementStyles() const { return MovementStyles; }
	MASSMOVEMENT_API const FMassMovementStyle* GetMovementStyleByID(const FGuid ID) const; 
	
private:

#if WITH_EDITOR
	MASSMOVEMENT_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY(EditAnywhere, config, Category = Movement);
    TArray<FMassMovementStyle> MovementStyles;
};
