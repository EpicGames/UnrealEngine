// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "MassMovementTypes.generated.h"

namespace UE::Mass::ProcessorGroupNames
{
	extern MASSMOVEMENT_API const FName SpringMovement;
}

namespace UE::MassMovement
{
	extern MASSMOVEMENT_API int32 bFreezeMovement;
};

/** Reference to movement style in MassMovementSettings. */
USTRUCT()
struct FMassMovementStyleRef
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Movement")
	FName Name;

	UPROPERTY(EditAnywhere, Category = "Movement", meta = (IgnoreForMemberInitializationTest, EditCondition = "false", EditConditionHides))
	FGuid ID;
};

/** Describes movement style name. */
USTRUCT()
struct FMassMovementStyle
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Movement")
	FName Name;

	UPROPERTY(EditAnywhere, Category = "Movement", meta = (IgnoreForMemberInitializationTest, EditCondition = "false", EditConditionHides))
	FGuid ID;
};

/**
 * Movement style consists of multiple speeds which are assigned to agents based on agents unique ID.
 * Same speed is assigned consistently for the same ID.
 */
USTRUCT()
struct FMassMovementStyleSpeedParameters
{
	GENERATED_BODY()

	/** Desired speed */
	UPROPERTY(EditAnywhere, Category = "Movement Style", meta = (UIMin = 0.0, ClampMin = 0.0, ForceUnits="cm/s"))
	float Speed = 140.0f;

	/** How much default desired speed is varied randomly. */
	UPROPERTY(EditAnywhere, Category = "Movement Style", meta = (ClampMin = 0.0, ClampMax = 1.0))
	float Variance = 0.1f;

	/** Probability to assign this speed. */
	UPROPERTY(EditAnywhere, Category = "Movement Style", meta = (UIMin = 1, ClampMin = 1, UIMax = 100, ClampMax = 100))
	float Probability = 1.0f;

	/** Probability threshold for this style, used to faster lookup. Update via FMassMovementConfig::Update(). */
	float ProbabilityThreshold = 0.0f;
};

/** Movement style parameters. A movement style abstracts movement properties for specific style. Behaviors can refer to specific styles when handling speeds. */
USTRUCT()
struct FMassMovementStyleParameters
{
	GENERATED_BODY()

	/** Style of the movement */
	UPROPERTY(EditAnywhere, Category = "Movement Style")
	FMassMovementStyleRef Style;

	/** Array of desired speeds (cm/s) assigned to agents based on probability. */
	UPROPERTY(EditAnywhere, Category = "Movement Style")
	TArray<FMassMovementStyleSpeedParameters> DesiredSpeeds;
};
