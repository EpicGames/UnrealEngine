// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonFragments.h"
#include "MassSpringMovementFragments.generated.h"

USTRUCT()
struct FSpringMovementSettings : public FMassConstSharedFragment
{
	GENERATED_BODY()

	// How long to smooth to target velocity
	UPROPERTY(EditAnywhere, Category=Smoothing)
	float VelocitySmoothingTime = 0.1f;

	// How long to smooth to target facing
	UPROPERTY(EditAnywhere, Category=Smoothing)
	float FacingSmoothingTime = 0.1f;

	// Below this speed we set velocity to 0
	UPROPERTY(EditAnywhere, Category=Smoothing)
	float VelocityDeadzoneThreshold = 0.1f;
};

USTRUCT()
struct FSpringMovementRuntime : public FMassFragment
{
	GENERATED_BODY();

	// These are INTERNAL variables used for the spring damper
	// Do not take for granted that next frame Position = Velocity * Dt
	// Since this is used in a exponential damped spring with analytical solution (not Euler integrated)
	FVector CurrentPosition = FVector::ZeroVector;
	FVector CurrentAccel = FVector::ZeroVector;
	FVector CurrentVelocity = FVector::ZeroVector;
	FQuat CurrentFacing = FQuat::Identity;
	FVector CurrentAngularVelocity = FVector::ZeroVector;
};
