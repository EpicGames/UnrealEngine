// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassObserverProcessor.h"

#include "MassSpringMovementProcessors.generated.h"

struct FMassExecutionContext;

// This processor runs the spring damper to update FSpringMovementRuntime from desired movement input.
// It does NOT move the character — that is handled by USpringApplyMovementProcessor or a custom movement processor.
// Note:	applying this spring to the character's desired movement will introduce some latency between desired input
//			and the resulting movement that can degrade the accuracy of the navigation systems 
UCLASS(MinimalAPI)
class USpringUpdateProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSMOVEMENT_API USpringUpdateProcessor();

protected:
	MASSMOVEMENT_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSMOVEMENT_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

// This processor applies the spring runtime state (position, facing, velocity) to the entity transform.
// Skipped when FMassCustomMovementTag is present (a custom movement processor handles movement instead).
UCLASS(MinimalAPI)
class USpringApplyMovementProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSMOVEMENT_API USpringApplyMovementProcessor();

protected:
	MASSMOVEMENT_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSMOVEMENT_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

// Observes removal of FMassOffLODTag and resyncs spring internal state
// to the entity's current transform/velocity, preventing visual pops
// when entities transition back on-LOD.
UCLASS(MinimalAPI)
class USpringMovementLODResyncObserver : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	MASSMOVEMENT_API USpringMovementLODResyncObserver();

protected:
	MASSMOVEMENT_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSMOVEMENT_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};
