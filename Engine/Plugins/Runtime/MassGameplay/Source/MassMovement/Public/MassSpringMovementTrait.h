// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MassSpringMovementFragments.h"

#include "MassSpringMovementTrait.generated.h"

/**
 * Adds spring-damped movement to this entity. The spring smoothly interpolates
 * position and facing toward the desired movement input using a critically damped spring.
 * Note: applying this spring to the character's desired movement will introduce some latency between desired input
 * and the resulting movement that can degrade the accuracy of the navigation systems. Use care when tweaking the spring settings
 * to ensure that avoidance and navigation perform as expected.
 *
 * Processor execution is split into two steps:
 *   - USpringUpdateProcessor: runs the spring damper to update FSpringMovementRuntime
 *     (runs for ALL spring entities regardless of other traits).
 *   - USpringApplyMovementProcessor: writes the spring state to the entity transform and
 *     velocity. Skipped when FMassCustomMovementTag is present (e.g. when UCharacterTrajectoryMovementTrait
 *     is active, trajectory movement takes over instead).
 *
 * Can be combined with:
 *   - UCharacterTrajectoryTrait: spring state is used to generate a smoothed trajectory for animation.
 *   - UCharacterTrajectoryTrait + UCharacterTrajectoryMovementTrait: spring update runs, trajectory is generated
 *     from spring state, and trajectory movement drives the entity (spring apply is skipped).
 *
 * The smoothing parameters (VelocitySmoothingTime, FacingSmoothingTime) are also used by
 * USpringMovementToCharacterTrajectoryProcessor for trajectory prediction.
 */
UCLASS(MinimalAPI, EditInlineNew, CollapseCategories, meta = (DisplayName = "Spring Movement"))
class USpringMovementTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	MASSMOVEMENT_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category = "Steering", EditAnywhere, meta = (EditInline))
	FSpringMovementSettings SpringSettings;
};
