// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassCommonTypes.h" // For WITH_MASSGAMEPLAY_DEBUG
#include "MassEntityQuery.h"
#include "MassMovementProcessors.generated.h"

#define UE_API MASSMOVEMENT_API

/**
 * Calculate desired movement based on input forces
 */
UCLASS(MinimalAPI)
class UMassApplyForceProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassApplyForceProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};


/**
 * Updates entities position based on desired velocity.
 * Only required for agents that have code driven displacement
 * Not applied on Off-LOD entities.
 */
UCLASS(MinimalAPI)
class UMassApplyMovementProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassApplyMovementProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;

#if WITH_MASSGAMEPLAY_DEBUG
	FMassEntityQuery DebugEntityQuery;
#endif
};

#undef UE_API
