// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassMovementFragments.h"
#include "MassMovementTrait.generated.h"

#define UE_API MASSMOVEMENT_API

UCLASS(MinimalAPI, meta = (DisplayName = "Movement"))
class UMassMovementTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category="Movement", EditAnywhere)
	FMassMovementParameters Movement;
};

#undef UE_API
