// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassMovementEditor : ModuleRules
	{
		public MassMovementEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"AssetTools",
				"EditorFramework",
				"UnrealEd",
				"RHI",
				"Slate",
				"SlateCore",
				
				"PropertyEditor",
				"DetailCustomizations",
				"MassCore",
				"MassEntity",
				"MassCommon",
				"MassMovement",
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RenderCore",
				"KismetWidgets",
				"ToolMenus",
				"AppFramework",
				"Projects",
			}
			);
		}

	}
}
