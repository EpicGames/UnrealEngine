// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovementEditorModule.h"
#include "AssetToolsModule.h"
#include "MassMovementStyleRefDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "MassMovementEditor"

IMPLEMENT_MODULE(FMassMovementEditorModule, MassMovementEditor)

void FMassMovementEditorModule::StartupModule()
{
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("MassMovementStyleRef"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMassMovementStyleRefDetails::MakeInstance));
}

void FMassMovementEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MassMovementStyleRef"));
	}
}

#undef LOCTEXT_NAMESPACE
