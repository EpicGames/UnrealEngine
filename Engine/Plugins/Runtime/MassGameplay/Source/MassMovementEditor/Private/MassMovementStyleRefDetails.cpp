// Copyright Epic Games, Inc. All Rights Reserved.
#include "MassMovementStyleRefDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "MassSettings.h"
#include "MassMovementSettings.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "MassMovementPropertyUtils.h"

#define LOCTEXT_NAMESPACE "MassMovementEditor"

TSharedRef<IPropertyTypeCustomization> FMassMovementStyleRefDetails::MakeInstance()
{
	return MakeShareable(new FMassMovementStyleRefDetails);
}

void FMassMovementStyleRefDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();
	
	NameProperty = StructProperty->GetChildHandle(TEXT("Name"));
	IDProperty = StructProperty->GetChildHandle(TEXT("ID"));

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FMassMovementStyleRefDetails::OnGetProfileContent)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FMassMovementStyleRefDetails::GetCurrentProfileDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FMassMovementStyleRefDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FMassMovementStyleRefDetails::OnProfileComboChange(int32 Idx)
{
	if (Idx == -1)
	{
		const UMassSettings* MassSettings = GetDefault<UMassSettings>();
		check(MassSettings);
		
		// Goto settings to create new Profile
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(MassSettings->GetContainerName(), MassSettings->GetCategoryName(), MassSettings->GetSectionName());
		return;
	}

	const UMassMovementSettings* MovementSettings = GetDefault<UMassMovementSettings>();
	check(MovementSettings);

	TConstArrayView<FMassMovementStyle> MovementStyles = MovementSettings->GetMovementStyles();
	if (MovementStyles.IsValidIndex(Idx))
	{
		const FMassMovementStyle& Style = MovementStyles[Idx];

		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		if (NameProperty)
		{
			NameProperty->SetValue(Style.Name, EPropertyValueSetFlags::NotTransactable);
		}

		if (IDProperty)
		{
			UE::MassMovement::PropertyUtils::SetValue<FGuid>(IDProperty, Style.ID, EPropertyValueSetFlags::NotTransactable);
		}

		if (PropUtils)
		{
			PropUtils->ForceRefresh();
		}
	}
}

TSharedRef<SWidget> FMassMovementStyleRefDetails::OnGetProfileContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);
	const UMassMovementSettings* Settings = GetDefault<UMassMovementSettings>();
	check(Settings);

	FUIAction NewItemAction(FExecuteAction::CreateSP(const_cast<FMassMovementStyleRefDetails*>(this), &FMassMovementStyleRefDetails::OnProfileComboChange, -1));
	MenuBuilder.AddMenuEntry(LOCTEXT("CreateOrEditStyles", "Create or Edit Movement Style..."), TAttribute<FText>(), FSlateIcon(), NewItemAction);
	MenuBuilder.AddMenuSeparator();

	TConstArrayView<FMassMovementStyle> MovementStyles = Settings->GetMovementStyles();
	for (int32 Index = 0; Index < MovementStyles.Num(); Index++)
	{
		const FMassMovementStyle& Style = MovementStyles[Index];
		FUIAction ItemAction(FExecuteAction::CreateSP(const_cast<FMassMovementStyleRefDetails*>(this), &FMassMovementStyleRefDetails::OnProfileComboChange, Index));
		MenuBuilder.AddMenuEntry(FText::FromName(Style.Name), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}
	return MenuBuilder.MakeWidget();
}

FText FMassMovementStyleRefDetails::GetCurrentProfileDesc() const
{
	TOptional<FGuid> IDOpt = UE::MassMovement::PropertyUtils::GetValue<FGuid>(IDProperty);
	if (IDOpt.IsSet())
	{
		const FGuid ID = IDOpt.GetValue();
		if (ID.IsValid())
		{
			const UMassMovementSettings* Settings = GetDefault<UMassMovementSettings>();
			check(Settings);
			
			const FMassMovementStyle* Style = Settings->GetMovementStyleByID(ID);
			if (Style)
			{
				return FText::FromName(Style->Name);
			}
			else
			{
				FName OldProfileName;
				if (NameProperty && NameProperty->GetValue(OldProfileName) == FPropertyAccess::Success)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("Identifier"), FText::FromName(OldProfileName));
					return FText::Format(LOCTEXT("InvalidStyle", "Invalid Style {Identifier}"), Args);
				}
			}
		}
		else
		{
			return LOCTEXT("Invalid", "Invalid");
		}
	}
	// TODO: handle multiple values
	return FText();
}

#undef LOCTEXT_NAMESPACE