// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

/** MassMovement Editor public interface */
class IMassMovementEditor : public FAssetEditorToolkit
{
public:

};


