// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "AssetTypeCategories.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"


class IMassMovementEditor;

/**
* The public interface to this module
*/
class FMassMovementEditorModule : public IModuleInterface//, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	// Begin IModuleInterface
	MASSMOVEMENTEDITOR_API virtual void StartupModule() override;
	MASSMOVEMENTEDITOR_API virtual void ShutdownModule() override;
};
