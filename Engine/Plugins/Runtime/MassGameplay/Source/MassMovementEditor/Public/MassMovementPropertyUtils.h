// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Editor.h"
#include "PropertyHandle.h"


#define LOCTEXT_NAMESPACE "MassMovementEditor"

namespace UE::MassMovement::PropertyUtils
{

// Expects T is struct.
template<typename T>
TOptional<T> GetValue(const TSharedPtr<IPropertyHandle>& ValueProperty)
{
	if (!ValueProperty)
	{
		return TOptional<T>();
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(ValueProperty->GetProperty());
	if (!StructProperty)
	{
		return TOptional<T>();
	}
	if (StructProperty->Struct != TBaseStructure<T>::Get())
	{
		return TOptional<T>();
	}

	T Value = T();
	bool bValueSet = false;

	TArray<void*> RawData;
	ValueProperty->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		if (Data)
		{
			T CurValue = *reinterpret_cast<T*>(Data);
			if (!bValueSet)
			{
				bValueSet = true;
				Value = CurValue;
			}
			else if (CurValue != Value)
			{
				// Multiple values
				return TOptional<T>();
			}
		}
	}

	return TOptional<T>(Value);
}

// Expects T is struct.
template<typename T>
void SetValue(TSharedPtr<IPropertyHandle>& ValueProperty, T NewValue, EPropertyValueSetFlags::Type Flags = 0)
{
	if (!ValueProperty)
	{
		return;
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(ValueProperty->GetProperty());
	if (!StructProperty)
	{
		return;
	}
	if (StructProperty->Struct != TBaseStructure<T>::Get())
	{
		return;
	}

	const bool bTransactable = (Flags & EPropertyValueSetFlags::NotTransactable) == 0;
	bool bNotifiedPreChange = false;
	TArray<void*> RawData;
	ValueProperty->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		if (Data)
		{
			if (!bNotifiedPreChange)
			{
				if (bTransactable && GEditor)
				{
					GEditor->BeginTransaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), ValueProperty->GetPropertyDisplayName()));
				}
				ValueProperty->NotifyPreChange();
				bNotifiedPreChange = true;
			}

			T* Value = reinterpret_cast<T*>(Data);
			*Value = NewValue;
		}
	}

	if (bNotifiedPreChange)
	{
		ValueProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		if (bTransactable && GEditor)
		{
			GEditor->EndTransaction();
		}
	}

	ValueProperty->NotifyFinishedChangingProperties();
}

} // UE::MassMovement::PropertyUtils

#undef LOCTEXT_NAMESPACE
