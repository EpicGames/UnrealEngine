// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"


class IPropertyHandle;
class SWidget;

/**
 * Type customization for FMassMovementStyleRef.
 */
class FMassMovementStyleRefDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void OnProfileComboChange(int32 Idx);
	TSharedRef<SWidget> OnGetProfileContent() const;
	FText GetCurrentProfileDesc() const;

	TSharedPtr<IPropertyHandle> NameProperty;
	TSharedPtr<IPropertyHandle> IDProperty;
	TSharedPtr<IPropertyHandle> LanesProperty;

	class IPropertyUtilities* PropUtils = nullptr;
	TSharedPtr<IPropertyHandle> StructProperty;
};
