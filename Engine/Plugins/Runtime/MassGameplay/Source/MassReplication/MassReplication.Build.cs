// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassReplication : ModuleRules
	{
		public MassReplication(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassSimulation",
					"MassSpawner",
					"MassLOD",
					"NetCore",
					"MassMovement",
					"ZoneGraph"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
