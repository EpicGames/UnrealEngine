// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassClientBubbleInfoBase.h"
#include "MassClientBubbleSerializerBase.h"
#include "MassClientBubbleHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassClientBubbleInfoBase)

AMassClientBubbleInfoBase::AMassClientBubbleInfoBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReplicates = true;
	bOnlyRelevantToOwner = true;
	bNetUseOwnerRelevancy = true;

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_LastDemotable;
	PrimaryActorTick.bStartWithTickEnabled = false;
}

void AMassClientBubbleInfoBase::SetClientHandle(FMassClientHandle InClientHandle)
{
	for (const FMassClientBubbleSerializerBase* Serializer : Serializers)
	{
		check(Serializer->GetClientHandler());

		Serializer->GetClientHandler()->SetClientHandle(InClientHandle);
	}
}

void AMassClientBubbleInfoBase::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UWorld* World = GetWorld();
		// This logic can't be done in BeginPlay() as the FMassClientBubbleSerializer::PostReplicatedAdd / PostReplicatedChange will already have been called
		// but it does need to be done after the world has been initialized.
		if (World != nullptr && World->bIsWorldInitialized)
		{
			InitializeForWorld(*World);
		}
		else
		{
			OnPostWorldInitDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &AMassClientBubbleInfoBase::OnPostWorldInit);
		}
	}
}

void AMassClientBubbleInfoBase::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World == GetWorld())
	{
		FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitDelegateHandle);
		InitializeForWorld(*World);
	}
}

void AMassClientBubbleInfoBase::InitializeForWorld(UWorld& World)
{
	for (const FMassClientBubbleSerializerBase* Serializer : Serializers)
	{
		IClientBubbleHandlerInterface* Handler = Serializer->GetClientHandler();

		checkf(Handler, TEXT("Handler not set up. Call TClientBubbleHandlerBase::Initialize() before InitializeForWorld gets called"));
		Handler->InitializeForWorld(World);
	}
}

void AMassClientBubbleInfoBase::BeginPlay()
{
	Super::BeginPlay();

	UWorld* World = GetWorld();
	check(World);

	// Allow tick function WITH_MASSGAMEPLAY_DEBUG as we have debug functionality there, otherwise only on the clients.
#if WITH_MASSGAMEPLAY_DEBUG
	PrimaryActorTick.SetTickFunctionEnable(true);
#else
	if (World && World->GetNetMode() == NM_Client)
	{
		PrimaryActorTick.SetTickFunctionEnable(true);
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}

void AMassClientBubbleInfoBase::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	for (FMassClientBubbleSerializerBase* Serializer : Serializers)
	{
		if (ensure(Serializer))
		{
			Serializer->ShutDown();

			check(Serializer->GetClientHandler());
			Serializer->GetClientHandler()->Reset();
		}
	}

	Super::EndPlay(EndPlayReason);
}

void AMassClientBubbleInfoBase::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	for (const FMassClientBubbleSerializerBase* Serializer : Serializers)
	{
		check(Serializer->GetClientHandler());

		Serializer->GetClientHandler()->Tick(DeltaTime);
	}
}
