// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassClientBubbleSerializerBase.h"
#include "MassClientBubbleHandler.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassClientBubbleSerializerBase)


void FMassClientBubbleSerializerBase::ShutDown()
{
	EntityManager = nullptr;
	World = nullptr;
	SpawnerSubsystem = nullptr;
	ReplicationSubsystem = nullptr;
}

void FMassClientBubbleSerializerBase::InitializeForWorld(UWorld& InWorld)
{
	World = &InWorld;

	SpawnerSubsystem = InWorld.GetSubsystem<UMassSpawnerSubsystem>();
	check(SpawnerSubsystem);

	ReplicationSubsystem = InWorld.GetSubsystem<UMassReplicationSubsystem>();

	EntityManager = UE::Mass::Utils::GetEntityManagerChecked(InWorld).AsShared();

	checkf(ClientHandler, TEXT("ClientHandler must be setup! See SetClientHandler()"));
}

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void FMassClientBubbleSerializerBase::PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize) const
{
	check(ClientHandler);

	ClientHandler->PreReplicatedRemove(RemovedIndices, FinalSize);
}

void FMassClientBubbleSerializerBase::PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize) const
{
	check(ClientHandler);

	ClientHandler->PostReplicatedAdd(AddedIndices, FinalSize);
}

void FMassClientBubbleSerializerBase::PostReplicatedChange(const TArrayView<int32> ChangedIndices, int32 FinalSize) const
{
	check(ClientHandler);

	ClientHandler->PostReplicatedChange(ChangedIndices, FinalSize);
}
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE
