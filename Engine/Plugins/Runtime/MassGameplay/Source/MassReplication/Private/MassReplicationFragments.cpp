// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationFragments.h"
#include "MassReplicationSubsystem.h"
#include "MassReplicationProcessor.h"
#include "MassExecutionContext.h"
#include "MassClientBubbleInfoBase.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
// UMassNetworkIDFragmentInitializer 
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationFragments)
UMassNetworkIDFragmentInitializer::UMassNetworkIDFragmentInitializer()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedTypes.Add(FMassNetworkIDFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
}

void UMassNetworkIDFragmentInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassNetworkIDFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassReplicationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassNetworkIDFragmentInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(MassProcessor_InitNetworkID_Run);

	const UWorld* World = EntityManager.GetWorld();
	const ENetMode NetMode = World->GetNetMode();

	if (NetMode != NM_Client)
	{
#if UE_REPLICATION_COMPILE_SERVER_CODE
		EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
			{
				UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();

				const TArrayView<FMassNetworkIDFragment> NetworkIDList = Context.GetMutableFragmentView<FMassNetworkIDFragment>();
				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					NetworkIDList[EntityIt].NetID = ReplicationSubsystem.GetNextAvailableMassNetID();
				}
			});
#endif //UE_REPLICATION_COMPILE_SERVER_CODE
	}
}

//----------------------------------------------------------------------//
//  FMassReplicationParameters
//----------------------------------------------------------------------//

FMassReplicationParameters::FMassReplicationParameters()
{
	LODDistance[EMassLOD::High] = 0.f;
	LODDistance[EMassLOD::Medium] = 1000.f;
	LODDistance[EMassLOD::Low] = 2500.f;
	LODDistance[EMassLOD::Off] = 5000.f;

	LODMaxCount[EMassLOD::High] = 1600;
	LODMaxCount[EMassLOD::Medium] = 3200;
	LODMaxCount[EMassLOD::Low] = 48000;
	LODMaxCount[EMassLOD::Off] = 0;

	LODMaxCountPerViewer[EMassLOD::High] = 100;
	LODMaxCountPerViewer[EMassLOD::Medium] = 200;
	LODMaxCountPerViewer[EMassLOD::Low] = 300;
	LODMaxCountPerViewer[EMassLOD::Off] = 0;

	UpdateInterval[EMassLOD::High] = 0.1f;
	UpdateInterval[EMassLOD::Medium] = 0.2f;
	UpdateInterval[EMassLOD::Low] = 0.3f;
	UpdateInterval[EMassLOD::Off] = 0.5f;
}

//----------------------------------------------------------------------//
//  FMassReplicationSharedFragment
//----------------------------------------------------------------------//
FMassReplicationSharedFragment::FMassReplicationSharedFragment(UMassReplicationSubsystem& ReplicationSubsystem, const FMassReplicationParameters& Params)
{
	// Setting the LODMaxCoundPerViewer as the MAXCount as the processor will be called for each client and will recalculate it per client
	LODCalculator.Initialize(Params.LODDistance, Params.BufferHysteresisOnDistancePercentage / 100.0f, Params.LODMaxCountPerViewer); 
	BubbleInfoClassHandle = ReplicationSubsystem.GetBubbleInfoClassHandle(Params.BubbleInfoClass);
	checkf(BubbleInfoClassHandle.IsValid(), TEXT("%s has not been registered as a bubble info class (yet)"), *GetNameSafe(Params.BubbleInfoClass.Get()));

	CachedReplicator = Params.ReplicatorClass.GetDefaultObject();
	checkf(CachedReplicator, TEXT("Expecting a valid replicator class"))
}
