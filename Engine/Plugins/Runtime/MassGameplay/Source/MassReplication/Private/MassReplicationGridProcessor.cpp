// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationGridProcessor.h"
#include "MassReplicationSubsystem.h"
#include "MassReplicationTypes.h"
#include "MassReplicationFragments.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
//  UMassReplicationGridProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationGridProcessor)
UMassReplicationGridProcessor::UMassReplicationGridProcessor()
	: AddToGridEntityQuery(*this)
	, UpdateGridEntityQuery(*this)
	, RemoveFromGridEntityQuery(*this)
{
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
	ExecutionFlags = int32(EProcessorExecutionFlags::Server);
#else
	ExecutionFlags = int32(EProcessorExecutionFlags::AllNetModes);
#endif // UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE

	ProcessingPhase = EMassProcessingPhase::PostPhysics;
}

void UMassReplicationGridProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddToGridEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	AddToGridEntityQuery.AddRequirement<FMassReplicationGridCellLocationFragment>(EMassFragmentAccess::ReadWrite);
	AddToGridEntityQuery.AddSubsystemRequirement<UMassReplicationSubsystem>(EMassFragmentAccess::ReadWrite);

	// copying AddToGridEntityQuery to RemoveFromGridEntityQuery now because RemoveFromGridEntityQuery doesn't utilize 
	// the other fragments AddToGridEntityQuery relies on
	RemoveFromGridEntityQuery = AddToGridEntityQuery;

	// FAgentRadiusFragment is optional since it's not strictly required for the provided functionality 
	AddToGridEntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	// we don't care about "off-lod" entities
	AddToGridEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);

	// storing the state in UpdateGridEntityQuery, after that both queries diverge in terms of requirements
	UpdateGridEntityQuery = AddToGridEntityQuery;
		
	AddToGridEntityQuery.AddTagRequirement<FMassInReplicationGridTag>(EMassFragmentPresence::None);
		
	UpdateGridEntityQuery.AddTagRequirement<FMassInReplicationGridTag>(EMassFragmentPresence::All);

	RemoveFromGridEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	RemoveFromGridEntityQuery.AddTagRequirement<FMassInReplicationGridTag>(EMassFragmentPresence::All);
}

void UMassReplicationGridProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	AddToGridEntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();
		FReplicationHashGrid2D& ReplicationGrid = ReplicationSubsystem.GetGridMutable();

		TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
		TArrayView<FMassReplicationGridCellLocationFragment> ReplicationCellLocationList = Context.GetMutableFragmentView<FMassReplicationGridCellLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// Add to the grid
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			// note that 0-radius is fine, the underlying THierarchicalHashGrid2D supports that just fine
			const float Radius = RadiusList.IsEmpty() ? 0.f : RadiusList[EntityIt].Radius;

			const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			ReplicationCellLocationList[EntityIt].CellLoc = ReplicationGrid.Add(EntityHandle, NewBounds);

			Context.Defer().AddTag<FMassInReplicationGridTag>(EntityHandle);
		}
	});

	UpdateGridEntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();
		FReplicationHashGrid2D& ReplicationGrid = ReplicationSubsystem.GetGridMutable();
		TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
		TArrayView<FMassReplicationGridCellLocationFragment> ReplicationCellLocationList = Context.GetMutableFragmentView<FMassReplicationGridCellLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// Update position in grid
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			// note that 0-radius is fine, the underlying THierarchicalHashGrid2D supports that just fine
			const float Radius = RadiusList.IsEmpty() ? 0.f : RadiusList[EntityIt].Radius;
			const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			ReplicationCellLocationList[EntityIt].CellLoc = ReplicationGrid.Move(EntityHandle, ReplicationCellLocationList[EntityIt].CellLoc, NewBounds);

#if WITH_MASSGAMEPLAY_DEBUG && 0
			const FDebugContext BaseDebugContext(this, LogMassReplication, nullptr, EntityHandle);
			if (DebugIsSelected(EntityHandle))
			{
				FBox Box = ReplicationGrid.CalcCellBounds(ReplicationCellLocationList[EntityIt].CellLoc);
				Box.Max.Z += 200.f;
				DebugDrawBox(BaseDebugContext, Box, FColor::Yellow);
			}
#endif // WITH_MASSGAMEPLAY_DEBUG
		}
	});

	RemoveFromGridEntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();
		FReplicationHashGrid2D& ReplicationGrid = ReplicationSubsystem.GetGridMutable();

		TArrayView<FMassReplicationGridCellLocationFragment> ReplicationCellLocationList = Context.GetMutableFragmentView<FMassReplicationGridCellLocationFragment>();
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
			ReplicationGrid.Remove(EntityHandle, ReplicationCellLocationList[EntityIt].CellLoc);
			ReplicationCellLocationList[EntityIt].CellLoc = FReplicationHashGrid2D::FCellLocation();

			Context.Defer().RemoveTag<FMassInReplicationGridTag>(EntityHandle);
		}
	});
}

//----------------------------------------------------------------------//
//  UMassReplicationGridRemoverProcessor
//----------------------------------------------------------------------//
UMassReplicationGridRemoverProcessor::UMassReplicationGridRemoverProcessor()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassReplicationGridCellLocationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void UMassReplicationGridRemoverProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassReplicationGridCellLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassReplicationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassReplicationGridRemoverProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();
		FReplicationHashGrid2D& ReplicationGrid = ReplicationSubsystem.GetGridMutable();
		const TArrayView<FMassReplicationGridCellLocationFragment> ReplicationCellLocationList = Context.GetMutableFragmentView<FMassReplicationGridCellLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
			ReplicationGrid.Remove(EntityHandle, ReplicationCellLocationList[EntityIt].CellLoc);
			ReplicationCellLocationList[EntityIt].CellLoc = FReplicationHashGrid2D::FCellLocation();
		}
	});
}
