// Copyright Epic Games, Inc. All Rights Reserved.UMassSimulationSettings

#include "MassReplicationProcessor.h"
#include "MassClientBubbleHandler.h"
#include "MassLODSubsystem.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationProcessor)

namespace UE::Mass::Replication
{
	int32 DebugClientReplicationLOD = -1;
	FAutoConsoleVariableRef CVarDebugReplicationViewerLOD(TEXT("mass.debug.ClientReplicationLOD"), DebugClientReplicationLOD, TEXT("Debug Replication LOD of the specified client index"), ECVF_Cheat);
} // UE::Mass::Crowd

//----------------------------------------------------------------------//
//  UMassReplicationProcessor
//----------------------------------------------------------------------//
UMassReplicationProcessor::UMassReplicationProcessor()
	: SyncClientData(*this)
	, CollectViewerInfoQuery(*this)
	, CalculateLODQuery(*this)
	, AdjustLODDistancesQuery(*this)
	, EntityQuery(*this)
{
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
	ExecutionFlags = int32(EProcessorExecutionFlags::Server);
#else
	ExecutionFlags = int32(EProcessorExecutionFlags::AllNetModes);
#endif // UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE

	ProcessingPhase = EMassProcessingPhase::PostPhysics;

	// Processor might need to create UObjects when synchronizing clients and viewers
	// (e.g. SpawnActor from UMassReplicationSubsystem::SynchronizeClientsAndViewers())
	bRequiresGameThreadExecution = true;
}

void UMassReplicationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	SyncClientData.AddRequirement<FMassReplicationLODFragment>(EMassFragmentAccess::ReadWrite);
	SyncClientData.AddRequirement<FMassReplicatedAgentFragment>(EMassFragmentAccess::ReadWrite);

	CollectViewerInfoQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	CollectViewerInfoQuery.AddRequirement<FMassReplicationViewerInfoFragment>(EMassFragmentAccess::ReadWrite);
	CollectViewerInfoQuery.AddSharedRequirement<FMassReplicationSharedFragment>(EMassFragmentAccess::ReadWrite);

	CalculateLODQuery.AddRequirement<FMassReplicationViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	CalculateLODQuery.AddRequirement<FMassReplicationLODFragment>(EMassFragmentAccess::ReadWrite);
	CalculateLODQuery.AddConstSharedRequirement<FMassReplicationParameters>();
	CalculateLODQuery.AddSharedRequirement<FMassReplicationSharedFragment>(EMassFragmentAccess::ReadWrite);

	AdjustLODDistancesQuery.AddRequirement<FMassReplicationViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	AdjustLODDistancesQuery.AddRequirement<FMassReplicationLODFragment>(EMassFragmentAccess::ReadWrite);
	AdjustLODDistancesQuery.AddSharedRequirement<FMassReplicationSharedFragment>(EMassFragmentAccess::ReadWrite);
	AdjustLODDistancesQuery.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		const FMassReplicationSharedFragment& LODSharedFragment = Context.GetSharedFragment<FMassReplicationSharedFragment>();
		return LODSharedFragment.bHasAdjustedDistancesFromCount;
	});

	EntityQuery.AddRequirement<FMassNetworkIDFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FReplicationTemplateIDFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassReplicationLODFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassReplicatedAgentFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassReplicationParameters>();
	EntityQuery.AddSharedRequirement<FMassReplicationSharedFragment>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
	ProcessorRequirements.AddSubsystemRequirement<UMassReplicationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassReplicationProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

#if UE_REPLICATION_COMPILE_SERVER_CODE
	UWorld* World = Owner.GetWorld();
	ReplicationSubsystem = UWorld::GetSubsystem<UMassReplicationSubsystem>(World);

	check(ReplicationSubsystem);
#endif //UE_REPLICATION_COMPILE_SERVER_CODE
}

void UMassReplicationProcessor::PrepareExecution(FMassEntityManager& EntityManager)
{
#if UE_REPLICATION_COMPILE_SERVER_CODE

	check(ReplicationSubsystem);

	//first synchronize clients and viewers
	ReplicationSubsystem->SynchronizeClientsAndViewers();

	EntityManager.ForEachSharedFragment<FMassReplicationSharedFragment>([this](FMassReplicationSharedFragment& RepSharedFragment)
	{
		if (!ensureMsgf(RepSharedFragment.BubbleInfoClassHandle.IsValid()
			, TEXT("BubbleInfoClassHandle is not valid which means no class has been indicated or the class used has not been registered pre creation of the handle.")))
		{
			return;
		}

		if (!RepSharedFragment.bEntityQueryInitialized)
		{
			RepSharedFragment.EntityQuery = EntityQuery;
			RepSharedFragment.EntityQuery.SetChunkFilter([&RepSharedFragment](const FMassExecutionContext& Context)
			{
				const FMassReplicationSharedFragment& CurRepSharedFragment = Context.GetSharedFragment<FMassReplicationSharedFragment>();
				return &CurRepSharedFragment == &RepSharedFragment;
			});
			RepSharedFragment.CachedReplicator->AddRequirements(RepSharedFragment.EntityQuery);
			RepSharedFragment.bEntityQueryInitialized = true;
		}

		const TArray<FMassClientHandle>& CurrentClientHandles = ReplicationSubsystem->GetClientReplicationHandles();
		const int32 MinNumHandles = FMath::Min(RepSharedFragment.CachedClientHandles.Num(), CurrentClientHandles.Num()); // Why is this the min not the max?

		//check to see if we don't have enough cached client handles
		if (RepSharedFragment.CachedClientHandles.Num() < CurrentClientHandles.Num())
		{
			RepSharedFragment.CachedClientHandles.Reserve(CurrentClientHandles.Num());
			RepSharedFragment.BubbleInfos.Reserve(CurrentClientHandles.Num());

			for (int32 Idx = RepSharedFragment.CachedClientHandles.Num(); Idx < CurrentClientHandles.Num(); ++Idx)
			{
				const FMassClientHandle& CurrentClientHandle = CurrentClientHandles[Idx];

				RepSharedFragment.CachedClientHandles.Add(CurrentClientHandle);
				AMassClientBubbleInfoBase* Info = CurrentClientHandle.IsValid() ?
					ReplicationSubsystem->GetClientBubbleChecked(RepSharedFragment.BubbleInfoClassHandle, CurrentClientHandle) :
					nullptr;

				check(Info);

				RepSharedFragment.BubbleInfos.Add(Info);
			}
		}
		//check to see if we have too many cached client handles
		else if (RepSharedFragment.CachedClientHandles.Num() > CurrentClientHandles.Num())
		{
			const int32 NumRemove = RepSharedFragment.CachedClientHandles.Num() - CurrentClientHandles.Num();

			RepSharedFragment.CachedClientHandles.RemoveAt(CurrentClientHandles.Num(), NumRemove, EAllowShrinking::No);
			RepSharedFragment.BubbleInfos.RemoveAt(CurrentClientHandles.Num(), NumRemove, EAllowShrinking::No);
		}

		//check to see if any cached client handles have changed, if they have set the BubbleInfo[] appropriately
		for (int32 Idx = 0; Idx < MinNumHandles; ++Idx)
		{
			const FMassClientHandle& CurrentClientHandle = CurrentClientHandles[Idx];
			FMassClientHandle& CachedClientHandle = RepSharedFragment.CachedClientHandles[Idx];

			const bool bChanged = (CurrentClientHandle != CachedClientHandle);
			if (bChanged)
			{
				AMassClientBubbleInfoBase* Info = CurrentClientHandle.IsValid() ?
					ReplicationSubsystem->GetClientBubbleChecked(RepSharedFragment.BubbleInfoClassHandle, CurrentClientHandle) :
					nullptr;

				RepSharedFragment.BubbleInfos[Idx] = Info;
				CachedClientHandle = CurrentClientHandle;
			}
		}
	});

#endif //UE_REPLICATION_COMPILE_SERVER_CODE
}

void UMassReplicationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
#if UE_REPLICATION_COMPILE_SERVER_CODE
	UWorld* World = EntityManager.GetWorld();
	check(World);
	check(ReplicationSubsystem);

	{
		QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_Preperation);
		PrepareExecution(EntityManager);
	}

	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	const TArray<FViewerInfo>& AllViewersInfo = LODSubsystem.GetViewers();
	const TArray<FMassClientHandle>& ClientHandles = ReplicationSubsystem->GetClientReplicationHandles();
	for (const FMassClientHandle ClientHandle : ClientHandles)
	{
		if (ReplicationSubsystem->IsValidClientHandle(ClientHandle) == false)
		{
			continue;
		}

		FMassClientReplicationInfo& ClientReplicationInfo = ReplicationSubsystem->GetMutableClientReplicationInfoChecked(ClientHandle);

		// Figure out all viewer of this client
		TArray<FViewerInfo> Viewers;
		for (const FMassViewerHandle ClientViewerHandle : ClientReplicationInfo.Handles)
		{
			const FViewerInfo* ViewerInfo = AllViewersInfo.FindByPredicate([ClientViewerHandle](const FViewerInfo& ViewerInfo) { return ClientViewerHandle == ViewerInfo.Handle; });
			if (ensureMsgf(ViewerInfo, TEXT("Expecting to find the client viewer handle in the all viewers info list")))
			{
				Viewers.Add(*ViewerInfo);
			}
		}

		// Prepare LOD collector and calculator
		// Remember the max LOD distance from each
		float MaxLODDistance = 0.0f;
		EntityManager.ForEachSharedFragment<FMassReplicationSharedFragment>([&Viewers,&MaxLODDistance](FMassReplicationSharedFragment& RepSharedFragment)
		{
			RepSharedFragment.LODCollector.PrepareExecution(Viewers);
			RepSharedFragment.LODCalculator.PrepareExecution(Viewers);
			MaxLODDistance = FMath::Max(MaxLODDistance, RepSharedFragment.LODCalculator.GetMaxLODDistance());
		});

		// Fetch all entities to process
		const FVector HalfExtent(MaxLODDistance, MaxLODDistance, 0.0f);
		TArray<FMassEntityHandle> EntitiesInRange;
		EntitiesInRange.Reset();
		for (const FViewerInfo& Viewer : Viewers)
		{
			FBox Bounds(Viewer.Location - HalfExtent, Viewer.Location + HalfExtent);
			ReplicationSubsystem->GetGrid().Query(Bounds, EntitiesInRange);
		}

		EntitiesInRange.Append(ClientReplicationInfo.HandledEntities);
		TArray<FMassArchetypeEntityCollection> EntitySets;
		UE::Mass::Utils::CreateEntityCollections(Context.GetEntityManagerChecked(), EntitiesInRange
				, FMassArchetypeEntityCollection::FoldDuplicates
				, EntitySets);

		TArray<FMassEntityHandle> EntitiesHandledThisFrame;

		for (FMassArchetypeEntityCollection& Set : EntitySets)
		{
			checkf(Set.IsEmpty() == false, TEXT("We don't expect to get empty collections from UE::Mass::Utils::CreateEntityCollections"));

			Context.SetEntityCollection(Set);
			
			// Make sure we save a deduplicated list of entities that were handled to avoid
			// accumulating instances in ClientReplicationInfo.HandledEntities over several frames
			Set.ExportEntityHandles(EntitiesHandledThisFrame);

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_SyncToMass);
				SyncClientData.ForEachEntityChunk(Context, [&ClientReplicationInfo](FMassExecutionContext& Context)
				{
					const TArrayView<FMassReplicationLODFragment> ViewerLODList = Context.GetMutableFragmentView<FMassReplicationLODFragment>();
					TArrayView<FMassReplicatedAgentFragment> ReplicatedAgentList = Context.GetMutableFragmentView<FMassReplicatedAgentFragment>();

					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
						FMassReplicatedAgentFragment& AgentFragment = ReplicatedAgentList[EntityIt];
						FMassReplicationLODFragment& LODFragment = ViewerLODList[EntityIt];

						if (FMassReplicatedAgentData* AgentData = ClientReplicationInfo.AgentsData.Find(EntityHandle))
						{
							LODFragment.LOD = AgentData->LOD;
							AgentFragment.AgentData = *AgentData;
						}
						else
						{
							LODFragment.LOD = EMassLOD::Off;
							AgentFragment.AgentData.Invalidate();
						}
					}
				});
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_LODCollection);
				CollectViewerInfoQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
				{
					const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
					const TArrayView<FMassReplicationViewerInfoFragment> ViewersInfoList = Context.GetMutableFragmentView<FMassReplicationViewerInfoFragment>();
					FMassReplicationSharedFragment& RepSharedFragment = Context.GetMutableSharedFragment<FMassReplicationSharedFragment>();
					RepSharedFragment.LODCollector.CollectLODInfo(Context, LocationList, ViewersInfoList, ViewersInfoList);
				});
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_LODCaculation);
				CalculateLODQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
				{
					const TConstArrayView<FMassReplicationViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassReplicationViewerInfoFragment>();
					const TArrayView<FMassReplicationLODFragment> ViewerLODList = Context.GetMutableFragmentView<FMassReplicationLODFragment>();
					FMassReplicationSharedFragment& RepSharedFragment = Context.GetMutableSharedFragment<FMassReplicationSharedFragment>();
					RepSharedFragment.LODCalculator.CalculateLOD(Context, ViewersInfoList, ViewerLODList, ViewersInfoList);
				});
			}
			Context.ClearEntityCollection();
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_LODAdjustDistance);
			EntityManager.ForEachSharedFragment<FMassReplicationSharedFragment>([](FMassReplicationSharedFragment& RepSharedFragment)
			{
				RepSharedFragment.bHasAdjustedDistancesFromCount = RepSharedFragment.LODCalculator.AdjustDistancesFromCount();
			});
		}

		for (FMassArchetypeEntityCollection& Set : EntitySets)
		{
			Context.SetEntityCollection(Set);

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_LODAdjustLODFromCount);
				AdjustLODDistancesQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
				{
					const TConstArrayView<FMassReplicationViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassReplicationViewerInfoFragment>();
					const TArrayView<FMassReplicationLODFragment> ViewerLODList = Context.GetMutableFragmentView<FMassReplicationLODFragment>();
					FMassReplicationSharedFragment& RepSharedFragment = Context.GetMutableSharedFragment<FMassReplicationSharedFragment>();
					RepSharedFragment.LODCalculator.AdjustLODFromCount(Context, ViewersInfoList, ViewerLODList, ViewersInfoList);
				});
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_ProcessClientReplication);
				FMassReplicationContext ReplicationContext(*World, LODSubsystem, *ReplicationSubsystem);
				EntityManager.ForEachSharedFragment<FMassReplicationSharedFragment>([&EntityManager, &Context, &ReplicationContext, &ClientHandle](FMassReplicationSharedFragment& RepSharedFragment)
				{
					RepSharedFragment.CurrentClientHandle = ClientHandle;

					RepSharedFragment.EntityQuery.ForEachEntityChunk(Context, [&ReplicationContext, &RepSharedFragment](FMassExecutionContext& Context)
					{
						RepSharedFragment.CachedReplicator->ProcessClientReplication(Context, ReplicationContext);
					});
				});
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(UMassReplicationProcessor_SyncFromMass);
				SyncClientData.ForEachEntityChunk(Context, [&ClientReplicationInfo](FMassExecutionContext& Context)
				{
					TArrayView<FMassReplicatedAgentFragment> ReplicatedAgentList = Context.GetMutableFragmentView<FMassReplicatedAgentFragment>();

					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
						FMassReplicatedAgentFragment& AgentFragment = ReplicatedAgentList[EntityIt];
						ClientReplicationInfo.AgentsData.Add(EntityHandle, AgentFragment.AgentData);
					}
				});
			}

#if WITH_MASSGAMEPLAY_DEBUG
			// Optional debug display
			if (UE::Mass::Replication::DebugClientReplicationLOD == ClientHandle.GetIndex())
			{
				EntityManager.ForEachSharedFragment<FMassReplicationSharedFragment>([World, &EntityManager, &Context](FMassReplicationSharedFragment& RepSharedFragment)
				{
					RepSharedFragment.EntityQuery.ForEachEntityChunk(Context, [World, &RepSharedFragment](FMassExecutionContext& Context)
					{
						const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
						const TConstArrayView<FMassReplicationLODFragment> ViewerLODList = Context.GetFragmentView<FMassReplicationLODFragment>();
						RepSharedFragment.LODCalculator.DebugDisplayLOD(Context, ViewerLODList, TransformList, World);
					});
				});
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			Context.ClearEntityCollection();
		}

		// note that moving EntitiesHandledThisFrame will effectively reset it, so not point in calling Reset explicitly. 
		ClientReplicationInfo.HandledEntities = MoveTemp(EntitiesHandledThisFrame);

		// Cleanup any AgentData that isn't relevant anymore (that is EMassLOD::OFF or pending destruction)
		for (FMassReplicationAgentDataMap::TIterator It = ClientReplicationInfo.AgentsData.CreateIterator(); It; ++It)
		{
			FMassReplicatedAgentData& AgentData = It.Value();
			if (AgentData.bPendingDestruction || AgentData.LOD == EMassLOD::Off)
			{
				checkf(!AgentData.Handle.IsValid(), TEXT("This replicated agent should have been removed from this client and was not"));
				It.RemoveCurrent();
			}
		}
	}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE
}

UMassReplicationEntityDestructionObserver::UMassReplicationEntityDestructionObserver()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<uint8>(EProcessorExecutionFlags::Server);
	ObservedTypes.Add(FMassReplicatedAgentFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::DestroyEntity;

	// Make sure we can only run in Game Thread to sync properly with UMassReplicationProcessor
	bRequiresGameThreadExecution = true;
}

void UMassReplicationEntityDestructionObserver::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddSubsystemRequirement<UMassReplicationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassReplicationEntityDestructionObserver::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
#if UE_REPLICATION_COMPILE_SERVER_CODE
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassReplicationSubsystem& ReplicationSubsystem = Context.GetMutableSubsystemChecked<UMassReplicationSubsystem>();
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			ReplicationSubsystem.NotifyEntityDestroyed(Context.GetEntity(EntityIt));
		}
	});
#endif
}