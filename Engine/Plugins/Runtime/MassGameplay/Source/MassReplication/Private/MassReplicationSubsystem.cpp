// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationSubsystem.h"
#include "Engine/World.h"
#include "Engine/ChildConnection.h"
#include "GameFramework/GameModeBase.h"
#include "MassCommonTypes.h"
#include "MassEntityManager.h"
#include "MassClientBubbleHandler.h"
#include "MassClientBubbleInfoBase.h"
#include "MassReplicationSettings.h"
#include "MassEntityUtils.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationSubsystem)


uint32 UMassReplicationSubsystem::CurrentNetMassCounter = 0;

void UMassReplicationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	World = GetWorld();
	check(World);

	MassLODSubsystem = Collection.InitializeDependency<UMassLODSubsystem>();
	check(MassLODSubsystem);

	EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World).AsShared();

	OverrideSubsystemTraits<UMassReplicationSubsystem>(Collection);
}

void UMassReplicationSubsystem::Deinitialize()
{
	// remove all Clients
	for (const FMassClientHandle& Handle : ClientHandleManager.GetHandles())
	{
		if (Handle.IsValid())
		{
			RemoveClient(Handle);
		}
	}

	// make sure all the other data is reset
	ClientHandleManager.Reset();
	BubbleInfoArray.Reset();
	ClientsReplicationInfo.Reset();
	ClientToViewerHandleArray.Reset();
	ViewerToClientHandleArray.Reset();

	World = nullptr;
	MassLODSubsystem = nullptr;
	EntityManager.Reset();

	Super::Deinitialize();
}

UMassReplicationSubsystem::UMassReplicationSubsystem()
 : ReplicationGrid(GetDefault<UMassReplicationSettings>()->GetReplicationGridCellSize())
{
}

FMassNetworkID UMassReplicationSubsystem::GetNetIDFromHandle(const FMassEntityHandle Handle) const
{
	check(EntityManager);
	const FMassNetworkIDFragment& Data = EntityManager->GetFragmentDataChecked<FMassNetworkIDFragment>(Handle);
	return Data.NetID;
}

namespace UE { namespace Mass { namespace Replication {
// Checks the parent net connection has APlayerController OwningActor, unfortunately GetParentConnection() prevents UChildConnection being const here.
APlayerController* GetValidParentNetConnection(UChildConnection& ChildConnection)
{
	return (ChildConnection.GetParentConnection() != nullptr) ? Cast<APlayerController>(ChildConnection.GetParentConnection()->OwningActor) : nullptr;
}

enum class ENetConnectionType : uint8
{
	None,
	Parent,
	Child,
};

// Checks Controllers net connection is a parent net connection
bool HasParentNetConnection(const APlayerController* Controller)
{
	return (Controller != nullptr) && Controller->NetConnection && (Cast<UChildConnection>(Controller->NetConnection) == nullptr);
}

// If Controller has a UChildConnection then this gets the parent net connection's APlayerController OwningActor otherwise nullptr
APlayerController* GetParentControllerFromChildNetConnection(const APlayerController* Controller)
{
	//ChildConnection can't be const
	UChildConnection* ChildConnection = Controller ? Cast<UChildConnection>(Controller->NetConnection) : nullptr;

	return ChildConnection ? GetValidParentNetConnection(*ChildConnection) : nullptr;
}

// Gets the ENetConnectionType for Controller. Tests if Controller's net connection is a parent or if Controller's net connection is a child and its parent net connection has a APlayerController OwningActor, 
ENetConnectionType HasParentOrChildWithValidParentNetConnection(const APlayerController* Controller)
{
	ENetConnectionType ConnectionType = ENetConnectionType::None;

	if (Controller && Controller->NetConnection)
	{
		UChildConnection* ChildConnection = Cast<UChildConnection>(Controller->NetConnection);

		// if the NetConnection is either a parent connection or a child connection with APlayerController owning actor its valid 
		if (ChildConnection == nullptr)
		{
			ConnectionType = ENetConnectionType::Parent;
		}
		else
		{
			ConnectionType = (GetValidParentNetConnection(*ChildConnection) != nullptr) ? ENetConnectionType::Child : ENetConnectionType::None;
		}		
	}

	return ConnectionType;
}
}}};

bool UMassReplicationSubsystem::SynchronizeClients(const TArray<FViewerInfo>& Viewers)
{
	typedef TMap<FMassViewerHandle, FMassClientHandle> FControllerMap;
	struct FClientAddData
	{
		FClientAddData(FMassViewerHandle InHandle, APlayerController* InController)
			: Handle(InHandle)
			, Controller(InController)
		{}

		FMassViewerHandle Handle;
		APlayerController* Controller = nullptr;
	};

	bool bNeedShrinking = false;

	//we can only replicate if we have some BubbleInfos to use for replication
	if (BubbleInfoArray.Num() == 0)
	{
		return bNeedShrinking;
	}

	// Arbitrarily use index 0, if there are more items the corresponding InfoData, Bubbles will be the same length and corresponding indices between the Bubbles
	// will have the same player controller owner.
	const FMassClientBubbleInfoData& InfoData = BubbleInfoArray[0];

	check(MassLODSubsystem);

	FControllerMap ClientConnectionMap;

	TArray<FClientAddData> ClientsToAdd;

	{
		// Go through the stored Clients, add valid ones to the ClientConnectionMap and remove invalids.
		// Note this is only valid until we next add a client handle.
		const TArray<FMassClientHandle>& ClientHandles = ClientHandleManager.GetHandles();

		for (int32 Idx = 0; Idx < ClientHandles.Num(); ++Idx)
		{
			const FMassClientHandle& ClientHandle = ClientHandles[Idx];

			if (ClientHandle.IsValid())
			{
				check(ClientToViewerHandleArray.IsValidIndex(ClientHandle.GetIndex()));

				const FViewerClientPair ViewerClientPair = ClientToViewerHandleArray[ClientHandle.GetIndex()];

				check(ViewerClientPair.ViewerHandle.IsValid());
				check(ViewerClientPair.ClientHandle == ClientHandle);
				check(InfoData.Bubbles.IsValidIndex(ClientHandle.GetIndex()));

				const AMassClientBubbleInfoBase* ClientBubble = InfoData.Bubbles[ClientHandle.GetIndex()];
				const APlayerController* Controller = ClientBubble ? Cast<APlayerController>(ClientBubble->GetOwner()) : nullptr;

//no need to check netconnection if UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
				// must have a valid non child UNetConnection
				if (UE::Mass::Replication::HasParentNetConnection(Controller))
#endif
				{
					ClientConnectionMap.Add(ViewerClientPair.ViewerHandle, ClientHandle);
				}
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
				else
				{
					// this is safe as Clients and their handles are free list arrays
					RemoveClient(ClientHandle);
					bNeedShrinking |= Idx == ClientHandles.Num() - 1;
				}
#endif
			}
		}

		// now go through all current Viewers and add if they do not exist
		for (const FViewerInfo& Viewer : Viewers)
		{
			// must have valid Viewer and parent net connection
			if (Viewer.Handle.IsValid())
			{
//no need to check netconnection if UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
				APlayerController* ViewerAsPlayerController = Viewer.GetPlayerController();
				if (UE::Mass::Replication::HasParentNetConnection(ViewerAsPlayerController))
#endif
				{
					// check if the controller already exists by trying to remove it from the map which was filled up with controllers we were tracking
					if (ClientConnectionMap.Remove(Viewer.Handle) == 0)
					{
						// If not add it to ClientsToAdd. Its important AddClient isn't called until necessary Clients are removed, as we may reuse 
						// array indices that are already in use.
						ClientsToAdd.Emplace(Viewer.Handle, ViewerAsPlayerController);
					}
				}
			}
		}

		// anything left in the map needs to be removed from the list
		for (FControllerMap::TIterator Itr = ClientConnectionMap.CreateIterator(); Itr; ++Itr)
		{
			const int32 ViewerIdx = Itr->Value.GetIndex();

			RemoveClient(Itr->Value);

			//InfoData must be valid here as ClientConnectionMap can only have items if ClientHandleManager.GetHandles() was greater than 0 
			//when we entered the function.
			bNeedShrinking |= (ViewerIdx == (InfoData.Bubbles.Num() - 1));
		}
	}

	// resize the ViewerToClientHandleArray array to match the viewer handles array (for consistency)
	if (ViewerToClientHandleArray.Num() > Viewers.Num())
	{
		ViewerToClientHandleArray.RemoveAt(Viewers.Num(), ViewerToClientHandleArray.Num() - Viewers.Num(), EAllowShrinking::No);
	}

	for (FClientAddData& ClientAdd : ClientsToAdd)
	{
		//TODO make AddClients (ie plural) functionality
		AddClient(ClientAdd.Handle, *ClientAdd.Controller);
	}

	return bNeedShrinking;
}

// synchronize the ClientViewers
void UMassReplicationSubsystem::SynchronizeClientViewers(const TArray<FViewerInfo>& Viewers)
{
	check(MassLODSubsystem);

	struct FMassClientViewerHandle
	{
		FMassClientViewerHandle(FMassClientHandle InClientHandle, FMassViewerHandle InViewerHandle)
			: ClientHandle(InClientHandle)
			, ViewerHandle(InViewerHandle)
		{}

		FMassClientHandle ClientHandle;
		FMassViewerHandle ViewerHandle;
	};

	// go through the ClientReplicationInfo and check validity and store the valid ones into a map
	typedef TMap<APlayerController*, FMassClientViewerHandle> FViewerMap;

	FViewerMap ClientViewerMap;
	{
		const TArray<FMassClientHandle> ClientHandles = ClientHandleManager.GetHandles();

		for (const FMassClientHandle& ClientHandle : ClientHandles)
		{
			//as this is a fresh handle then we only need to check against !IsInvalid
			if (ClientHandle.IsValid())
			{
				FMassClientReplicationInfo& ClientReplicationInfo = ClientsReplicationInfo[ClientHandle.GetIndex()];

				int32 ViewerIdx = 0;

				while (ViewerIdx < ClientReplicationInfo.Handles.Num())
				{
					const FMassViewerHandle& ClientViewer = ClientReplicationInfo.Handles[ViewerIdx];

					APlayerController* Controller = MassLODSubsystem->GetPlayerControllerFromViewerHandle(ClientViewer);

					const UE::Mass::Replication::ENetConnectionType ConnectionType = UE::Mass::Replication::HasParentOrChildWithValidParentNetConnection(Controller);

					if (ConnectionType != UE::Mass::Replication::ENetConnectionType::None)
					{
						++ViewerIdx;

						// we don't verify or remove Parent Connections here as that was done when we synchronized the client bubbles
						if (ConnectionType == UE::Mass::Replication::ENetConnectionType::Child)
						{
							ClientViewerMap.Add(Controller, FMassClientViewerHandle(ClientHandle, ClientViewer));
						}
					}
					else //remove invalid ClientViewer, but dont increment the ViewerIdx
					{
						ClientReplicationInfo.Handles.RemoveAt(ViewerIdx, EAllowShrinking::No);
					}
				}
			}
		}

		// now go through all current Viewers and add if they are a valid ClientViewer with a child UNetconnection if they do not exist
		for (const FViewerInfo& Viewer : Viewers)
		{
			// we are only interested in valid Viewers
			if (Viewer.Handle.IsValid())
			{
				// we are only processing child UNetConnections that have a valid APlayerController OwningActor
				APlayerController* ViewerAsPlayerController = Viewer.GetPlayerController();
				const APlayerController* ParentController = UE::Mass::Replication::GetParentControllerFromChildNetConnection(ViewerAsPlayerController);

				// check if the parent controller is valid and already exists
				if (ParentController && (ClientViewerMap.Find(ViewerAsPlayerController) == nullptr))
				{
					FMassViewerHandle ParentViewerHandle = MassLODSubsystem->GetViewerHandleFromActor(*ParentController);

					if (ensureMsgf(ParentViewerHandle.IsValid(), TEXT("MassLODSubsystem handles are out of sync with PlayerController NetConnections!")))
					{
						// note all clients (parent NetConnections) should already be set up so we would expect valid handles here
						check(ViewerToClientHandleArray.IsValidIndex(ParentViewerHandle.GetIndex()));

						const FViewerClientPair& ParentViewerClientPair = ViewerToClientHandleArray[ParentViewerHandle.GetIndex()];

						check(MassLODSubsystem->IsValidViewer(ParentViewerClientPair.ViewerHandle));
						check(ClientHandleManager.IsValidHandle(ParentViewerClientPair.ClientHandle));

						// remove APlayerController from the ClientViewerMap and Add the viewer to the ClientsReplicationInfo
						ClientViewerMap.Remove(ViewerAsPlayerController);

						FMassClientReplicationInfo& ClientReplicationInfo = ClientsReplicationInfo[ParentViewerClientPair.ClientHandle.GetIndex()];

						ClientReplicationInfo.Handles.Add(Viewer.Handle);
					}
				}
			}
		}
	}

	{
		// anything left in the map needs to be removed from the list
		for (FViewerMap::TIterator Itr = ClientViewerMap.CreateIterator(); Itr; ++Itr)
		{
			const FMassClientViewerHandle& HandleData = Itr->Value;

			FMassClientReplicationInfo& ClientReplicationInfo = ClientsReplicationInfo[HandleData.ClientHandle.GetIndex()];

			ClientReplicationInfo.Handles.RemoveSingle(HandleData.ViewerHandle);
		}
	}
}

void UMassReplicationSubsystem::SynchronizeClientsAndViewers()
{
	// only execute this code at most once per frame
	if (LastSynchronizedFrame == GFrameCounter)
	{
		return;
	}
	LastSynchronizedFrame = GFrameCounter;

	if (MassLODSubsystem == nullptr)
	{
		checkNoEntry();
		return;
	}

	// makes sure the LOD manager Viewers are synced before we process them
	const TArray<FViewerInfo>& Viewers = MassLODSubsystem->GetSynchronizedViewers();

	const bool bNeedShrinking = SynchronizeClients(Viewers);

	//only synchronize the client viewers outside of the debug functionality for now
#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
	SynchronizeClientViewers(Viewers);
#endif //UE_ALLOW_REPLICATION_DEBUG_BUBBLES_IN_STANDALONE

	if (bNeedShrinking)
	{
		const int32 NumItems = ClientHandleManager.ShrinkHandles();
		 
		ClientsReplicationInfo.RemoveAt(NumItems, ClientsReplicationInfo.Num() - NumItems, EAllowShrinking::No);
		ClientToViewerHandleArray.RemoveAt(NumItems, ClientToViewerHandleArray.Num() - NumItems, EAllowShrinking::No);

		for (FMassClientBubbleInfoData& InfoData : BubbleInfoArray)
		{
			InfoData.Bubbles.RemoveAt(NumItems, InfoData.Bubbles.Num() - NumItems, EAllowShrinking::No);
		}
	}
}

FMassBubbleInfoClassHandle UMassReplicationSubsystem::RegisterBubbleInfoClass(const TSubclassOf<AMassClientBubbleInfoBase>& BubbleInfoClass)
{
	checkf(BubbleInfoClass.Get() != nullptr, TEXT("BubbleInfoClass must have been set!"));

	if (ClientHandleManager.GetHandles().Num() > 0)
	{
		checkf(false, TEXT("RegisterBubbleInfoClass() must not be called after AddClient, BubbleInfoClass has not been registered"));
		return FMassBubbleInfoClassHandle();
	}

	const int32 IdxFound = BubbleInfoArray.IndexOfByPredicate([BubbleInfoClass](const FMassClientBubbleInfoData& Data)
		{
			return BubbleInfoClass == Data.BubbleClass;
		});

	if (IdxFound != INDEX_NONE)
	{
		UE_LOGF(LogMassReplication, Log, "UMassReplicationSubsystem: Trying to RegisterBubbleInfoClass() twice with the same BubbleInfoClass, Only one BubbleInfoClass will be registered for this type");
		return FMassBubbleInfoClassHandle(IdxFound);
	}

	const int32 Idx = BubbleInfoArray.Emplace(BubbleInfoClass);

	return FMassBubbleInfoClassHandle(Idx);
}

FMassBubbleInfoClassHandle UMassReplicationSubsystem::GetBubbleInfoClassHandle(const TSubclassOf<AMassClientBubbleInfoBase>& BubbleInfoClass) const
{
	FMassBubbleInfoClassHandle Handle;

	for (int32 Idx = 0; Idx < BubbleInfoArray.Num(); ++Idx)
	{
		const FMassClientBubbleInfoData& BubbleData = BubbleInfoArray[Idx];

		if (BubbleData.BubbleClass == BubbleInfoClass)
		{
			Handle.SetIndex(Idx);
			break;
		}
	}

	UE_CVLOG_UELOG(Handle.IsValid() == false, this, LogMassReplication, Error, TEXT("%hs failed to find required Bubble Info class"), __FUNCTION__);

	return Handle;
}

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void UMassReplicationSubsystem::SetEntity(const FMassNetworkID NetworkID, const FMassEntityHandle Entity)
{
	FMassReplicationEntityInfo* EntityInfo = FindMassEntityInfoMutable(NetworkID);
	check(EntityInfo && !EntityInfo->Entity.IsSet());
	EntityInfo->Entity = Entity;

	OnMassAgentAdded.Broadcast(NetworkID, Entity);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
FMassEntityHandle UMassReplicationSubsystem::ResetEntityIfValid(const FMassNetworkID NetworkID, int32 ReplicationID)
{
	FMassEntityHandle EntityReset;

	FMassReplicationEntityInfo* EntityInfo = FindMassEntityInfoMutable(NetworkID);

	if (ensureMsgf(EntityInfo, TEXT("EntityData must have been added to EntityInfoMap!")))
	{
		checkf(EntityInfo->ReplicationID >= ReplicationID, TEXT("We must not be removing an item we've never added!"));

		// Only reset the item if its currently Set / Valid and its the most recent ReplicationID. Stale removes after more recent adds are ignored
		// We do need to check the ReplicationID in this case
		if (EntityInfo->Entity.IsSet() && (EntityInfo->ReplicationID == ReplicationID))
		{
			EntityReset = EntityInfo->Entity;
			OnRemovingMassAgent.Broadcast(NetworkID, EntityReset);

			//Unset the Entity handle, this indicates that its currently removed from the bubble
			EntityInfo->Entity = FMassEntityHandle();
		}
	}

	return EntityReset;
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void UMassReplicationSubsystem::RemoveFromEntityInfoMap(const FMassNetworkID NetworkID)
{
	check(NetworkID.IsValid());
	FMassReplicationEntityInfo Info;
	ensureMsgf(EntityInfoMap.RemoveAndCopyValue(NetworkID, Info), TEXT("Removing a non-existant NetworkID(%s)!"), *NetworkID.Describe());
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
UMassReplicationSubsystem::EFindOrAddMassEntityInfo UMassReplicationSubsystem::FindAndUpdateOrAddMassEntityInfo(const FMassNetworkID NetworkID, int32 ReplicationID, const FMassReplicationEntityInfo*& OutMassEntityInfo)
{
	FMassReplicationEntityInfo* MassEntityInfo = FindMassEntityInfoMutable(NetworkID);
	EFindOrAddMassEntityInfo FindOrAddStatus = EFindOrAddMassEntityInfo::FoundOlderReplicationID;

	if (MassEntityInfo)
	{
		// Currently we don't think this should be needed, but are leaving it in for bomb proofing
		if (ensure(MassEntityInfo->ReplicationID < ReplicationID))
		{
			// Update the replication ID to the latest
			MassEntityInfo->ReplicationID = ReplicationID;
		}
		else
		{
			FindOrAddStatus = EFindOrAddMassEntityInfo::FoundNewerReplicationID;
		}
	}
	else
	{
		MassEntityInfo = &EntityInfoMap.Add(NetworkID, FMassReplicationEntityInfo(FMassEntityHandle(), ReplicationID));
		FindOrAddStatus = EFindOrAddMassEntityInfo::Added;
	}

	OutMassEntityInfo = MassEntityInfo;
	return FindOrAddStatus;
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
const FMassReplicationEntityInfo* UMassReplicationSubsystem::FindMassEntityInfo(const FMassNetworkID NetworkID) const
{
	check(NetworkID.IsValid());
	return EntityInfoMap.Find(NetworkID);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
FMassReplicationEntityInfo* UMassReplicationSubsystem::FindMassEntityInfoMutable(const FMassNetworkID NetworkID)
{
	check(NetworkID.IsValid());
	return EntityInfoMap.Find(NetworkID);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
FMassEntityHandle UMassReplicationSubsystem::FindEntity(const FMassNetworkID NetworkID) const
{
	check(NetworkID.IsValid());
	const FMassReplicationEntityInfo* Info = EntityInfoMap.Find(NetworkID);
	return Info ? Info->Entity : FMassEntityHandle();
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

void UMassReplicationSubsystem::DebugCheckArraysAreInSync()
{
#if UE_DEBUG_REPLICATION

	checkf((ClientToViewerHandleArray.Num() == ClientsReplicationInfo.Num()), TEXT("Client arrays out of sync with each other!"));

	const int32 NumEntries = (BubbleInfoArray.Num() > 0) ? BubbleInfoArray[0].Bubbles.Num() : 0;

	for (int32 IdxOuter = 0; IdxOuter < BubbleInfoArray.Num(); ++IdxOuter)
	{
		FMassClientBubbleInfoData& InfoDataOuter = BubbleInfoArray[IdxOuter];

		checkf((InfoDataOuter.Bubbles.Num() == ClientsReplicationInfo.Num()), TEXT("BubbleInfoArray arrays out of sync with ClientsReplicationInfo!"));
		checkf(InfoDataOuter.Bubbles.Num() == NumEntries, TEXT("Bubbles have different numbers of items!"));

		for (int32 IdxInner = IdxOuter + 1; IdxInner < BubbleInfoArray.Num(); ++IdxInner)
		{
			FMassClientBubbleInfoData& InfoDataInner = BubbleInfoArray[IdxInner];

			for (int32 IdxBubble = 0; IdxBubble < InfoDataOuter.Bubbles.Num(); ++IdxBubble)
			{
				AMassClientBubbleInfoBase* InfoOuter = InfoDataOuter.Bubbles[IdxBubble];
				AMassClientBubbleInfoBase* InfoInner = InfoDataInner.Bubbles[IdxBubble];

				const TArray<FMassClientHandle>& Handles = ClientHandleManager.GetHandles();

				if (ClientHandleManager.IsValidHandle(Handles[IdxBubble]))
				{				
					check(InfoOuter && InfoInner);

					APlayerController* OuterController = Cast<APlayerController>(InfoOuter->GetOwner());
					APlayerController* InnerController = Cast<APlayerController>(InfoInner->GetOwner());

					checkf((OuterController != nullptr) && (InnerController != nullptr), TEXT("Controller owners must be valid in BubbleInfoArray"));
					checkf(OuterController == InnerController, TEXT("Owner controllers at the same indices in different BubbleInfoArray items must be equal!"));
				}
				else
				{
					check(!InfoOuter && !InfoInner);
				}
			}
		}
	}

 #endif // UE_DEBUG_REPLICATION
}

#if UE_REPLICATION_COMPILE_SERVER_CODE
void UMassReplicationSubsystem::NotifyEntityDestroyed(FMassEntityHandle EntityHandle)
{
	for (FMassClientReplicationInfo& ClientReplicationInfo : ClientsReplicationInfo)
	{
		if (FMassReplicatedAgentData* AgentDataPtr = ClientReplicationInfo.AgentsData.Find(EntityHandle))
		{
			AgentDataPtr->bPendingDestruction = true;
		}
	}
}
#endif // UE_REPLICATION_COMPILE_SERVER_CODE

void UMassReplicationSubsystem::AddClient(FMassViewerHandle ViewerHandle, APlayerController& InController)
{
	check(World);
	check(MassLODSubsystem);
	checkf(MassLODSubsystem->IsValidViewer(ViewerHandle), TEXT("ViewerHandle must be valid"));

#if !UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE
	checkf(UE::Mass::Replication::HasParentNetConnection(&InController), TEXT("InController must have a parent net connection or replication will not occur!"));
#endif

	if (!ensureMsgf(BubbleInfoArray.Num() > 0, TEXT("BubbleInfoClass has not been set Client will not be added to UMassReplicationSubsystem!")))
	{
		return;
	}

	if (ViewerHandle.GetIndex() >= ViewerToClientHandleArray.Num())
	{
		// making no assumptions about the order that ViewerHandles are added, make sure we add enough space to accomodate the new ViewerHandle index
		ViewerToClientHandleArray.AddDefaulted(ViewerHandle.GetIndex() - ViewerToClientHandleArray.Num() + 1);
	}
	else 
	{
		checkf(ViewerToClientHandleArray[ViewerHandle.GetIndex()].ViewerHandle.IsValid() == false, TEXT("Adding a Client without removing the previous with the same Index!"));
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.Owner = &InController;

	const FMassClientHandle ClientHandle = ClientHandleManager.GetNextHandle();

	const FViewerClientPair ViewerClientPair(ViewerHandle, ClientHandle);

	FViewerClientPair& ViewerToClientHandleItem = ViewerToClientHandleArray[ViewerHandle.GetIndex()];

	checkf(ViewerToClientHandleItem.IsValid() == false, TEXT("Handles should have been reset when previous Viewer in this slot was removed"));

	ViewerToClientHandleItem = ViewerClientPair;

	checkf(ClientHandle.GetIndex() <= ClientToViewerHandleArray.Num(), TEXT("ClientHandle out of sync with ClientToViewerHandleArray"));

	// check if the handle is a new entry in the free list arrays or uses an existing entry
	if (ClientHandle.GetIndex() == ClientToViewerHandleArray.Num())
	{
		ClientsReplicationInfo.AddDefaulted();
		ClientToViewerHandleArray.Emplace(ViewerHandle, ClientHandle);
	}
	else
	{
		checkf(ClientsReplicationInfo[ClientHandle.GetIndex()].IsEmpty(), TEXT("ClientsReplicationInfo being replaced must have been reset prior to being reused!"));

		FViewerClientPair& ClientToViewerHandleItem = ClientToViewerHandleArray[ClientHandle.GetIndex()];

		checkf(ClientToViewerHandleItem.IsValid() == false, TEXT("Handles should have been reset when previous Client in this slot was removed"));

		ClientToViewerHandleItem = ViewerClientPair;
	}

	FMassClientReplicationInfo& ClientReplicationInfo = ClientsReplicationInfo[ClientHandle.GetIndex()];
	ClientReplicationInfo.Handles.Add(ViewerHandle);

	for (FMassClientBubbleInfoData& InfoData : BubbleInfoArray)
	{
		AMassClientBubbleInfoBase* ClientBubbleInfo = World->SpawnActor<AMassClientBubbleInfoBase>(InfoData.BubbleClass, SpawnParams);
		ClientBubbleInfo->SetClientHandle(ClientHandle);

		checkf(ClientHandle.GetIndex() <= InfoData.Bubbles.Num(), TEXT("ClientHandle out of sync with Bubbles"));

		// check if the handle is a new entry in the free list arrays or uses an existing entry
		if (ClientHandle.GetIndex() == InfoData.Bubbles.Num())
		{
			InfoData.Bubbles.Push(ClientBubbleInfo);
		}
		else
		{
			TObjectPtr<AMassClientBubbleInfoBase>& BubbleUpdate = InfoData.Bubbles[ClientHandle.GetIndex()];
			checkf(BubbleUpdate == nullptr, TEXT("ClientBubble being replaced must be nullptr it should have been removed first!"));

			BubbleUpdate = ClientBubbleInfo;
		}
	}
	DebugCheckArraysAreInSync();
}

void UMassReplicationSubsystem::RemoveClient(FMassClientHandle ClientHandle)
{
	check(World);

	checkf(ClientHandleManager.IsValidHandle(ClientHandle), TEXT("ClientHandle must be a valid non stale handle"));

	checkf(ClientToViewerHandleArray.IsValidIndex(ClientHandle.GetIndex()), TEXT("ClientHandle is out of sync with ClientToViewerHandleArray!"));
	FViewerClientPair& ClientToViewerHandleItem = ClientToViewerHandleArray[ClientHandle.GetIndex()];

	checkf(ClientToViewerHandleItem.ViewerHandle.IsValid(), TEXT("Invalid ViewerHandle! ClientHandle is out of sync with ClientToViewerHandleArray!"));
	checkf(ClientToViewerHandleItem.ClientHandle == ClientHandle, TEXT("ClientHandle is out of sync with ClientToViewerHandleArray!"));

	{
		FMassClientReplicationInfo& ClientReplicationInfo = ClientsReplicationInfo[ClientHandle.GetIndex()];

		checkf(ClientReplicationInfo.Handles.Num() > 0, TEXT("There should always be atleast one client viewer handle (the parent NetConnection)"));

		ClientReplicationInfo.Reset();
	}

	checkf(ViewerToClientHandleArray.IsValidIndex(ClientToViewerHandleItem.ViewerHandle.GetIndex()), TEXT("ViewerHandle is out of sync with ViewerToClientHandleArray!"));
	FViewerClientPair& ViewerToClientHandleItem = ViewerToClientHandleArray[ClientToViewerHandleItem.ViewerHandle.GetIndex()];

	checkf(ViewerToClientHandleItem == ClientToViewerHandleItem, TEXT("ClientToViewerHandleArray and ViewerToClientHandleArray out of sync!"));

	ViewerToClientHandleItem.Invalidate();
	ClientToViewerHandleItem.Invalidate();

	for (FMassClientBubbleInfoData& InfoData : BubbleInfoArray)
	{
		checkf(InfoData.Bubbles.IsValidIndex(ClientHandle.GetIndex()), TEXT("ClientHandle is out of sync with Bubbles!"));
		TObjectPtr<AMassClientBubbleInfoBase>& BubbleInfoItem = InfoData.Bubbles[ClientHandle.GetIndex()];

		if ((World != nullptr) && (BubbleInfoItem != nullptr))
		{
			World->DestroyActor(BubbleInfoItem);
		}

		BubbleInfoItem = nullptr;
	}

	ClientHandleManager.RemoveHandle(ClientHandle);

	DebugCheckArraysAreInSync();
}
