// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "Engine/World.h"
#include "MassSpawnerTypes.h"
#include "MassSimulationLOD.h"
#include "MassReplicationTypes.h"
#include "MassReplicationSubsystem.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationTrait)


void UMassReplicationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	if (World.IsNetMode(NM_Standalone) && !BuildContext.IsInspectingData())
	{
		return;
	}

	FReplicationTemplateIDFragment& TemplateIDFragment = BuildContext.AddFragment_GetRef<FReplicationTemplateIDFragment>();
	TemplateIDFragment.ID = BuildContext.GetTemplateID();

	BuildContext.AddFragment<FMassNetworkIDFragment>();
	BuildContext.AddFragment<FMassReplicatedAgentFragment>();
	BuildContext.AddFragment<FMassReplicationViewerInfoFragment>();
	BuildContext.AddFragment<FMassReplicationLODFragment>();
	BuildContext.AddFragment<FMassReplicationGridCellLocationFragment>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	UMassReplicationSubsystem* ReplicationSubsystem = UWorld::GetSubsystem<UMassReplicationSubsystem>(&World);

	FConstSharedStruct ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(Params);
	BuildContext.AddConstSharedFragment(ParamsFragment);

	if (LIKELY(!BuildContext.IsInspectingData()))
	{
		check(ReplicationSubsystem);

		FMassReplicationSharedFragment ReplicationFragment(*ReplicationSubsystem, Params);
		if (ensureMsgf(ReplicationFragment.IsValid(), TEXT("Created FMassReplicationSharedFragment instance is not valid")))
		{
			FSharedStruct SharedFragment = EntityManager.GetOrCreateSharedFragment(*FMassReplicationSharedFragment::StaticStruct(), reinterpret_cast<uint8*>(&ReplicationFragment));
			BuildContext.AddSharedFragment(SharedFragment);
		}
	}
	else
	{
		// in the investigation mode we only care about the fragment type
		FSharedStruct SharedFragment = EntityManager.GetOrCreateSharedFragment<FMassReplicationSharedFragment>();
		BuildContext.AddSharedFragment(SharedFragment);
	}
}
