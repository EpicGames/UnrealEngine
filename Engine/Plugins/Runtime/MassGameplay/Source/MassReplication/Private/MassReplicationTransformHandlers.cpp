// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationTransformHandlers.h"
#include "MassEntityQuery.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "AIHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationTransformHandlers)

void FMassReplicationProcessorTransformHandlerBase::AddRequirements(FMassEntityQuery& InQuery)
{
	InQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void FMassReplicationProcessorTransformHandlerBase::CacheFragmentViews(FMassExecutionContext& ExecContext)
{
	TransformList = ExecContext.GetMutableFragmentView<FTransformFragment>();
}

void FMassReplicationProcessorPositionYawHandler::AddEntity(const int32 EntityIdx, FReplicatedAgentPositionYawData& InOutReplicatedPositionYawData) const
{
	const FTransformFragment& TransformFragment = TransformList[EntityIdx];
	InOutReplicatedPositionYawData.SetPosition(TransformFragment.GetTransform().GetLocation());

	const FRotator::FReal Yaw = FMath::DegreesToRadians(TransformFragment.GetTransform().Rotator().Yaw);
	InOutReplicatedPositionYawData.SetYaw(static_cast<float>(Yaw));
}
