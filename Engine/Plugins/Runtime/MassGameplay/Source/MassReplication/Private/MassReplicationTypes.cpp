// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationTypes)

DEFINE_LOG_CATEGORY(LogMassReplication);
