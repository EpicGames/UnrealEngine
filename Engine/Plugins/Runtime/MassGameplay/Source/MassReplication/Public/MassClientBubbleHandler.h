// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassReplicationTypes.h"
#include "Containers/ArrayView.h"
#include "MassClientBubbleSerializerBase.h"
#include "MassSpawnerSubsystem.h"
#include "MassReplicationFragments.h"
#include "MassReplicationSubsystem.h"
#include "MassReplicationTemplates.h"
#include "Engine/World.h"
#include "MassEntityTemplate.h"
#include "MassEntityView.h"
#include "MassEntityQuery.h"
#include "MassClientBubbleHandlerInterface.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#include "MassSpawnerTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "MassClientBubbleHandler.generated.h"

class UWorld;
struct FMassEntityManager;

namespace UE::Mass::Replication
{
	constexpr float AgentRemoveInterval = 5.f;

	template<typename T>
	concept TIsCallable = std::is_class_v<typename TRemoveReference<T>::Type>
		&& requires(T t)
		{
			&TRemoveReference<T>::Type::operator();
		};
}; //namespace UE::Mass::Replication

/**
 *  Base for fast array items. For replication of new entity types this type should be inherited from. FReplicatedAgentBase should also be inherited from
 *  and made a member variable of the FMassFastArrayItemBase derived struct, with the member variable called Agent.
 */
USTRUCT()
struct FMassFastArrayItemBase : public FFastArraySerializerItem
{
	GENERATED_BODY()

	FMassFastArrayItemBase() = default;
	FMassFastArrayItemBase(FMassReplicatedAgentHandle InHandle)
		: Handle(InHandle)
	{};

	FMassReplicatedAgentHandle GetHandle() const { return Handle; }

protected:
	/** Only to be used on a server */
	UPROPERTY(NotReplicated)
	FMassReplicatedAgentHandle Handle;
};

/** Data that can be accessed from a FMassReplicatedAgentHandle on a server */
struct FMassAgentLookupData
{
	FMassAgentLookupData(FMassEntityHandle InEntity, FMassNetworkID InNetID, int32 InAgentsIdx)
		: Entity(InEntity)
		, NetID(InNetID)
		, AgentsIdx(InAgentsIdx)
	{}

	void Invalidate()
	{
		Entity = FMassEntityHandle();
		NetID.Invalidate();
		AgentsIdx = INDEX_NONE;
	}

	bool IsValid() const
	{
		return Entity.IsSet() && NetID.IsValid() && (AgentsIdx >= 0);
	}

	FMassEntityHandle Entity;
	FMassNetworkID NetID;
	int32 AgentsIdx  = INDEX_NONE;
};

/**
 *  Data that is stored when an agent is removed from the bubble, when it times out its safe enough to remove entries in EntityInfoMap.
 *  The idea is that any out of order adds and removes will happen after this time.
 */
struct FMassAgentRemoveData
{
	FMassAgentRemoveData() = default;
	FMassAgentRemoveData(double InTimeLastRemoved)
		: TimeLastRemoved(InTimeLastRemoved)
	{}

	double TimeLastRemoved = 0.;
};


/** 
 * Template client bubble functionality. Replication logic for specific agent types is provided by deriving from this class.
 * Interaction with the FMassClientBubbleSerializerBase and derived classes should be done from this class 
 */
template<typename AgentArrayItem>
class TClientBubbleHandlerBase : public IClientBubbleHandlerInterface
{
public:
	typedef TFunctionRef<void(FMassEntityQuery&)> FAddRequirementsForSpawnQueryFunction;
	typedef TFunctionRef<void(FMassExecutionContext&)> FCacheFragmentViewsForSpawnQueryFunction;
	typedef TFunctionRef<void(const FMassEntityView&, const typename AgentArrayItem::FReplicatedAgentType&, const int32)> FSetSpawnedEntityDataFunction;
	typedef TFunctionRef<void(const FMassEntityView&, const typename AgentArrayItem::FReplicatedAgentType&)> FSetModifiedEntityDataFunction;

	/** This must be called from outside before InitializeForWorld() is called. Its called from agent specific bubble implementations */
	virtual void Initialize(TArray<AgentArrayItem>& InAgents, FMassClientBubbleSerializerBase& InSerializer);

	virtual void InitializeForWorld(UWorld& InWorld) override;
	UWorld* GetWorld() const;

#if UE_REPLICATION_COMPILE_SERVER_CODE
	FMassReplicatedAgentHandle AddAgent(FMassEntityHandle Entity, typename AgentArrayItem::FReplicatedAgentType& Agent);

	bool RemoveAgent(FMassNetworkID NetID);
	bool RemoveAgent(FMassReplicatedAgentHandle AgentHandle);
	void RemoveAgentChecked(FMassReplicatedAgentHandle AgentHandle);
	void MarkItemDirty(FFastArraySerializerItem& Item) const;
	AgentArrayItem& GetAgentItem(const FMassReplicatedAgentHandle Handle) const;

	/** Gets an agent safely */
	const typename AgentArrayItem::FReplicatedAgentType* GetAgent(FMassReplicatedAgentHandle Handle) const;

	/** Faster version to get an agent that performs check()s for debugging */
	const typename AgentArrayItem::FReplicatedAgentType& GetAgentChecked(FMassReplicatedAgentHandle Handle) const;
	const TArray<AgentArrayItem>& GetAgents() const { return *Agents; }

	template<typename TReplicatedDataType>
	void SetBubbleData(const FMassReplicatedAgentHandle Handle, const TReplicatedDataType& NewData)
	{
		AgentArrayItem& Item = GetAgentItem(Handle);

		checkf(Item.Agent.GetNetID().IsValid(),
			TEXT("Data should not be updated on Items that have an Invalid ID! First Add the Agent!"));

		TReplicatedDataType& OldData = Item.Agent.template GetReplicatedData<TReplicatedDataType>();

		if (UE::Mass::Replication::CheckIsDirty(OldData, NewData))
		{
			OldData = NewData;
			MarkItemDirty(Item);
		}
	}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

protected:

#if UE_REPLICATION_COMPILE_CLIENT_CODE

	virtual void PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize) override;

	/**
	 * the TEventHandlers template parameters below require the following callable members (functions, lambda, functors):
	 * {
	 *		AddRequirementsForSpawnQuery;
	 *		CacheFragmentViewsForSpawnQuery;
	 *		SetSpawnedEntityData;
	 *		SetModifiedEntityData;
	 *		GetSpawnQuery -> FMassEntityQuery;
	 * }
	 * Note:
	 *	- PostReplicatedAddEntitiesHelper doesn't call SetModifiedEntityData
	 *	- PostReplicatedChangeHelper only calls SetModifiedEntityData
	 *	- it's recommended to supply GetSpawnQuery by deriving from UE::Mass::Replication::TCachedSpawnQuery
	 */

	/** Called from TClientBubbleHandlerBase derived classes in PostReplicatedAd() */
	template<typename TEventHandlers>
	void PostReplicatedAddHelper(const TArrayView<int32> AddedIndices, TEventHandlers& EventHandlers);

	void PostReplicatedAddHelper(const TArrayView<int32> AddedIndices, FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery
		, FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery, FSetSpawnedEntityDataFunction SetSpawnedEntityData, FSetModifiedEntityDataFunction SetModifiedEntityData);

	/** used by PostReplicatedAddHelper */
	template<typename TEventHandlers>
	void PostReplicatedAddEntitiesHelper(const TArrayView<int32> AddedIndices, TEventHandlers& EventHandlers);

	void PostReplicatedAddEntitiesHelper(const TArrayView<int32> AddedIndices, FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery
		, FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery, FSetSpawnedEntityDataFunction SetSpawnedEntityData);


	/** Called from TClientBubbleHandlerBase derived classes in PostReplicatedChange() */
	void PostReplicatedChangeHelper(const TArrayView<int32> ChangedIndices, FSetModifiedEntityDataFunction SetModifiedEntityData);

	template<typename TEventHandlers>
	requires (!UE::Mass::Replication::TIsCallable<TEventHandlers>)
	void PostReplicatedChangeHelper(const TArrayView<int32> ChangedIndices, TEventHandlers& HandlersPack);
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_SERVER_CODE
	void RemoveAgentImpl(FMassReplicatedAgentHandle Handle);
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

	virtual void SetClientHandle(FMassClientHandle InClientHandle) override;

	virtual void Reset() override;
	virtual void Tick(float DeltaTime) override;

	virtual void UpdateAgentsToRemove() override;

	virtual void DebugValidateBubbleOnClient() override;
	virtual void DebugValidateBubbleOnServer() override;

protected:
	/** Pointer to the Agents array in the associated Serializer class */
	TArray<AgentArrayItem>* Agents = nullptr;

	FMassClientHandle ClientHandle;

#if UE_REPLICATION_COMPILE_SERVER_CODE
	FMassReplicatedAgentHandleManager AgentHandleManager;

	/** Used to look up Agent data from a FMassReplicatedAgentHandle, the AgentsIdx member will be the Idx in to the Agents array */
	TArray<FMassAgentLookupData> AgentLookupArray;

	TMap<FMassNetworkID, FMassReplicatedAgentHandle> NetworkIDToAgentHandleMap;
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	/**
	 *  Data that is stored when an agent is removed from the bubble, when it times out its safe enough to remove entries in EntityInfoMap
	 *  The idea is that any out of order adds and subsequent removes for this NetID will normally happen before FAgentRemoveData::TimeLastRemoved,
	 *  Those that happen after will be on such a bad connection that it doest matter.
	 */
	TMap<FMassNetworkID, FMassAgentRemoveData> AgentsRemoveDataMap;
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

	/** Base class pointer to the associated Serializer class */
	FMassClientBubbleSerializerBase* Serializer = nullptr;
};

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::Initialize(TArray<AgentArrayItem>& InAgents, FMassClientBubbleSerializerBase& InSerializer)
{
	Agents = &InAgents;
	Serializer = &InSerializer;
	Serializer->SetClientHandler(*this);
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::InitializeForWorld(UWorld& InWorld)
{
	checkf(Agents, TEXT("Agents not set up. Call Initialize() before InitializeForWorld() gets called"));
	checkf(Serializer, TEXT("Serializer not set up. Call Initialize() before InitializeForWorld() gets called"));

	Serializer->InitializeForWorld(InWorld);
}

template <typename AgentArrayItem>
UWorld* TClientBubbleHandlerBase<AgentArrayItem>::GetWorld() const
{
	return Serializer->GetWorld();
}

#if UE_REPLICATION_COMPILE_SERVER_CODE
template<typename AgentArrayItem>
FMassReplicatedAgentHandle TClientBubbleHandlerBase<AgentArrayItem>::AddAgent(FMassEntityHandle Entity, typename AgentArrayItem::FReplicatedAgentType& Agent)
{
	checkf(Agent.GetNetID().IsValid(), TEXT("Agent.NetID must be valid!"));
	checkf(NetworkIDToAgentHandleMap.Find(Agent.GetNetID()) == nullptr, TEXT("Only add agents once"));

	FMassReplicatedAgentHandle AgentHandle = AgentHandleManager.GetNextHandle();

	checkf(AgentHandle.GetIndex() <= AgentLookupArray.Num(), TEXT("AgentHandle is out of sync with the AgentLookupArray Array!"));

	const int32 Idx = AgentHandle.GetIndex();

	if (Idx == AgentLookupArray.Num())
	{
		AgentLookupArray.Emplace(Entity, Agent.GetNetID(), (*Agents).Num());
	}
	else
	{
		checkf(AgentLookupArray[Idx].IsValid() == false, TEXT("Agent being replaced must be Invalid (should have been removed first)!"));

		AgentLookupArray[Idx] = FMassAgentLookupData(Entity, Agent.GetNetID(), (*Agents).Num());
	}

	AgentArrayItem& Item = (*Agents).Emplace_GetRef(Agent, AgentHandle);
	Serializer->MarkItemDirty(Item);

	NetworkIDToAgentHandleMap.Add(Agent.GetNetID(), AgentHandle);

	return AgentHandle;
}

template<typename AgentArrayItem>
bool TClientBubbleHandlerBase<AgentArrayItem>::RemoveAgent(FMassNetworkID NetID)
{
	const FMassReplicatedAgentHandle* const AgentHandle = NetworkIDToAgentHandleMap.Find(NetID);

	return (AgentHandle != nullptr) ? RemoveAgent(*AgentHandle) : false;
}

template<typename AgentArrayItem>
bool TClientBubbleHandlerBase<AgentArrayItem>::RemoveAgent(FMassReplicatedAgentHandle AgentHandle)
{
	bool bRemoved = false;

	if (ensureMsgf(AgentHandleManager.IsValidHandle(AgentHandle), TEXT("FMassReplicatedAgentHandle should be Valid")))
	{
		bRemoved = true;
		RemoveAgentImpl(AgentHandle);
	}

	return bRemoved;
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::RemoveAgentChecked(FMassReplicatedAgentHandle Handle)
{
	checkf(AgentHandleManager.IsValidHandle(Handle), TEXT("FMassReplicatedAgentHandle must be Valid"));
	RemoveAgentImpl(Handle);
}

template <typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::MarkItemDirty(FFastArraySerializerItem& Item) const
{
	Serializer->MarkItemDirty(Item);
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::RemoveAgentImpl(FMassReplicatedAgentHandle Handle)
{
	FMassAgentLookupData& LookUpData = AgentLookupArray[Handle.GetIndex()];
	const bool bDidSwap = LookUpData.AgentsIdx < ((*Agents).Num() - 1);

	(*Agents).RemoveAtSwap(LookUpData.AgentsIdx, EAllowShrinking::No);

	Serializer->MarkArrayDirty();

	verifyf(NetworkIDToAgentHandleMap.Remove(LookUpData.NetID) == 1, TEXT("Failed to find 1 matching NetID in NetworkIDToAgentHandleMap"));

	if (bDidSwap)
	{
		//we need to change the AgentsIdx of the Lookup data free list for the item that was swapped
		const FMassReplicatedAgentHandle HandleSwap = (*Agents)[LookUpData.AgentsIdx].GetHandle();
		checkf(AgentHandleManager.IsValidHandle(HandleSwap), TEXT("Handle of the Agent we RemoveAtSwap with must be valid as its in the Agents array"));

		FMassAgentLookupData& LookUpDataSwap = AgentLookupArray[HandleSwap.GetIndex()];
		LookUpDataSwap.AgentsIdx = LookUpData.AgentsIdx;
		checkf(LookUpDataSwap.NetID.IsValid(), TEXT("NetID of item we are swaping with must be valid as its in the Agents array"));
	}

	AgentHandleManager.RemoveHandle(Handle);

	LookUpData.Invalidate();
}

template <typename AgentArrayItem>
AgentArrayItem& TClientBubbleHandlerBase<AgentArrayItem>::GetAgentItem(const FMassReplicatedAgentHandle Handle) const
{
	check(AgentHandleManager.IsValidHandle(Handle));
        
	const int32 AgentsIdx = AgentLookupArray[Handle.GetIndex()].AgentsIdx;
	
	return (*Agents)[AgentsIdx];
}

template<typename AgentArrayItem>
const typename AgentArrayItem::FReplicatedAgentType* TClientBubbleHandlerBase<AgentArrayItem>::GetAgent(FMassReplicatedAgentHandle Handle) const
{
	if (AgentHandleManager.IsValidHandle(Handle))
	{
		const FMassAgentLookupData& LookUpData = AgentLookupArray[Handle.GetIndex()];

		return &((*Agents)[LookUpData.AgentsIdx].Agent);
	}
	return nullptr;
}

template<typename AgentArrayItem>
const typename AgentArrayItem::FReplicatedAgentType& TClientBubbleHandlerBase<AgentArrayItem>::GetAgentChecked(FMassReplicatedAgentHandle Handle) const
{
	check(AgentHandleManager.IsValidHandle(Handle));

	const FMassAgentLookupData& LookUpData = AgentLookupArray[Handle.GetIndex()];
	return (*Agents)[LookUpData.AgentsIdx].Agent;
}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::UpdateAgentsToRemove()
{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	QUICK_SCOPE_CYCLE_COUNTER(MassProcessor_Replication_CalculateClientReplication);

	check(Serializer->GetWorld());

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	const double TimeRemove = Serializer->GetWorld()->GetRealTimeSeconds() - UE::Mass::Replication::AgentRemoveInterval;

	// @todo do this in a more efficient way, we may potentially be able to use ACK's and FReplicatedAgentBase::bRemovedFromServeSim
	// Check to see if we should free any EntityInfoMap entries, this is to avoid gradually growing EntityInfoMap perpetually
	for (TMap<FMassNetworkID, FMassAgentRemoveData>::TIterator Iter = AgentsRemoveDataMap.CreateIterator(); Iter; ++Iter)
	{
		const FMassAgentRemoveData& RemoveData = (*Iter).Value;

		// The idea here is that AgentRemoveInterval represents a reasonable amount of time that if an out of order add and remove come in after this that we don't care, as the accuracy of the simulation
		// must already be pretty awful.
		if (RemoveData.TimeLastRemoved < TimeRemove)
		{
			const FMassNetworkID NetID = (*Iter).Key;

			ReplicationSubsystem->RemoveFromEntityInfoMap(NetID);
			Iter.RemoveCurrent();
		}
	}
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE
}

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize)
{
	TArray<FMassEntityHandle> EntitiesToDestroy;

	UWorld* World = Serializer->GetWorld();
	check(World);

	UMassSpawnerSubsystem* SpawnerSubsystem = Serializer->GetSpawnerSubsystem();
	check(SpawnerSubsystem);

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	for (int32 Idx : RemovedIndices)
	{
		const AgentArrayItem& RemovedItem = (*Agents)[Idx];

		FMassEntityHandle Entity = ReplicationSubsystem->ResetEntityIfValid(RemovedItem.Agent.GetNetID(), RemovedItem.ReplicationID);

		// Only remove the item if its currently Set / Valid and its the most recent ReplicationID. Stale removes after more recent adds are ignored
		// We do need to check the ReplicationID in this case
		if (Entity.IsSet())
		{
			EntitiesToDestroy.Add(Entity);
			
			check(AgentsRemoveDataMap.Find(RemovedItem.Agent.GetNetID()) == nullptr);
			AgentsRemoveDataMap.Add(RemovedItem.Agent.GetNetID(), FMassAgentRemoveData(World->GetRealTimeSeconds()));
		}
	}

	SpawnerSubsystem->DestroyEntities(EntitiesToDestroy);
}
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
template<typename TEventHandlers>
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedAddHelper(const TArrayView<int32> AddedIndices, TEventHandlers& EventHandlers)
{
	FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	TMap<FMassNetworkID, int32> AgentsToAddMap; //NetID to index in AgentsToAddArray
	AgentsToAddMap.Reserve(AddedIndices.Num());

	TArray<int32> AgentsToAddArray;
	AgentsToAddArray.Reserve(AddedIndices.Num());

	const FMassReplicationEntityInfo* EntityInfo;

	for (int32 Idx : AddedIndices)
	{
		const AgentArrayItem& AddedItem = (*Agents)[Idx];

		switch (ReplicationSubsystem->FindAndUpdateOrAddMassEntityInfo(AddedItem.Agent.GetNetID(), AddedItem.ReplicationID, EntityInfo))
		{
		case UMassReplicationSubsystem::EFindOrAddMassEntityInfo::FoundOlderReplicationID:
		{
			AgentsRemoveDataMap.Remove(AddedItem.Agent.GetNetID());

			// If EntityData IsSet() it means we have had multiple Adds without a remove and we treat this add as modifying an existing agent.
			if (EntityInfo->Entity.IsSet())
			{
				FMassEntityView EntityView(EntityManager, EntityInfo->Entity);

				EventHandlers.SetModifiedEntityData(EntityView, AddedItem.Agent);
			}
			else // This entity is not in the client simulation yet and needs adding.
			{
				const int32* IdxInAgentsToAddArray = AgentsToAddMap.Find(AddedItem.Agent.GetNetID());

				// If IdxInAgentsToAddArray then we've had multiple Adds
				if (IdxInAgentsToAddArray)
				{
					// Adjust the existing AgentsToAddArray index
					AgentsToAddArray[*IdxInAgentsToAddArray] = Idx;
				}
				else // First time we've tried to add an entity with this FMassNetworkID this update
				{	
					const int32 IdxAdd = AgentsToAddArray.Add(Idx);
					AgentsToAddMap.Add(AddedItem.Agent.GetNetID(), IdxAdd);
				}
			}
		}
		break;

		case UMassReplicationSubsystem::EFindOrAddMassEntityInfo::Added:
		{
			check(AgentsToAddMap.Find(AddedItem.Agent.GetNetID()) == nullptr);

			const int32 IdxAdd = AgentsToAddArray.Add(Idx);
			AgentsToAddMap.Add(AddedItem.Agent.GetNetID(), IdxAdd);
		}
		break;

		case UMassReplicationSubsystem::EFindOrAddMassEntityInfo::FoundNewerReplicationID:
			break;

		default:
			checkf(false, TEXT("Unhandled EFindOrAddMassEntityInfo type"));
			break;
		}
	}

	PostReplicatedAddEntitiesHelper(AgentsToAddArray, EventHandlers);
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedAddHelper(const TArrayView<int32> AddedIndices, FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery
	, FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery, FSetSpawnedEntityDataFunction SetSpawnedEntityData, FSetModifiedEntityDataFunction SetModifiedEntityData)
{
	struct FEventHandlers : UE::Mass::Replication::TCachedSpawnQuery<FEventHandlers>
	{
		using UE::Mass::Replication::TCachedSpawnQuery<FEventHandlers>::GetSpawnQuery;
		FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery;
		FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery;
		FSetSpawnedEntityDataFunction SetSpawnedEntityData;
		FSetModifiedEntityDataFunction SetModifiedEntityData;
	};

	FEventHandlers EventHandlers = {
		.AddRequirementsForSpawnQuery = AddRequirementsForSpawnQuery,
		.CacheFragmentViewsForSpawnQuery = CacheFragmentViewsForSpawnQuery,
		.SetSpawnedEntityData = SetSpawnedEntityData,
		.SetModifiedEntityData = SetModifiedEntityData
	};

	PostReplicatedAddHelper(AddedIndices, EventHandlers);
}

template<typename AgentArrayItem>
template<typename TEventHandlers>
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedAddEntitiesHelper(const TArrayView<int32> AddedIndices, TEventHandlers& EventHandlers)
{
	check(Serializer);

	FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	UMassSpawnerSubsystem* SpawnerSubsystem = Serializer->GetSpawnerSubsystem();
	check(SpawnerSubsystem);

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	TMap<FMassEntityTemplateID, TArray<typename AgentArrayItem::FReplicatedAgentType*>> AgentsSpawnMap;

	// Group together Agents per TemplateID
	for (int32 Idx : AddedIndices)
	{
		typename AgentArrayItem::FReplicatedAgentType& Agent = (*Agents)[Idx].Agent;

		TArray<typename AgentArrayItem::FReplicatedAgentType*>& AgentsArray = AgentsSpawnMap.FindOrAdd(Agent.GetTemplateID());
		AgentsArray.Add(&Agent);
	}

	FMassExecutionContext ExecContext(EntityManager);
	FMassEntityQuery& Query = EventHandlers.GetSpawnQuery(EntityManager.AsShared());

	// Batch spawn per FMassEntityTemplateID
	for (const TPair<FMassEntityTemplateID, TArray<typename AgentArrayItem::FReplicatedAgentType*>>& Item : AgentsSpawnMap)
	{
		const FMassEntityTemplateID& TemplateID = Item.Key;
		const TArray <typename AgentArrayItem::FReplicatedAgentType*>& AgentsSpawn = Item.Value;

		TArray<FMassEntityHandle> Entities;

		SpawnerSubsystem->SpawnEntities(TemplateID, AgentsSpawn.Num(), FStructView(), TSubclassOf<UMassProcessor>(), Entities);

		// if there were any initializer or observer processors it's possible that the spawned entities
		// are no longer in the archetype indicated by the template. We need to create the entity collections
		// the slower way (i.e. assuming we don't know the current archetype)
		// Note that we're not using CreationContext->GetEntityCollections (with CreationContext being
		// the return value of SpawnerSubsystem->SpawnEntities) because it might contain unrelated entities.
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);

		// Note that the way this functionality is implemented here it's not possible to parallelize the lambda's
		// execution, due to using AgentsSpawnIdx. If attempt is made at parallelizing the work this detail needs to be changed.
		int32 AgentsSpawnIdx = 0;

		Query.ForEachEntityChunkInCollections(EntityCollections, ExecContext
			, [&AgentsSpawn, &AgentsSpawnIdx, this, ReplicationSubsystem, &ExecContext, &EventHandlers](FMassExecutionContext& Context)
			{
				EventHandlers.CacheFragmentViewsForSpawnQuery(Context);

				const TArrayView<FMassNetworkIDFragment> NetworkIDList = Context.GetMutableFragmentView<FMassNetworkIDFragment>();

				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const typename AgentArrayItem::FReplicatedAgentType& AgentSpawn = *AgentsSpawn[AgentsSpawnIdx];
					const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

					FMassNetworkIDFragment& NetIDFragment = NetworkIDList[EntityIt];

					NetIDFragment.NetID = AgentSpawn.GetNetID();
					ReplicationSubsystem->SetEntity(NetIDFragment.NetID, Entity);

					FMassEntityView EntityView(Context.GetEntityManagerChecked(), Entity);
					EventHandlers.SetSpawnedEntityData(EntityView, AgentSpawn, EntityIt);

					++AgentsSpawnIdx;
				}
			});
	}
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedAddEntitiesHelper(const TArrayView<int32> AddedIndices, FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery
	, FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery, FSetSpawnedEntityDataFunction SetSpawnedEntityData)
{
	struct FEventHandlers : UE::Mass::Replication::TCachedSpawnQuery<FEventHandlers>
	{
		using UE::Mass::Replication::TCachedSpawnQuery<FEventHandlers>::GetSpawnQuery;
		FAddRequirementsForSpawnQueryFunction AddRequirementsForSpawnQuery;
		FCacheFragmentViewsForSpawnQueryFunction CacheFragmentViewsForSpawnQuery;
		FSetSpawnedEntityDataFunction SetSpawnedEntityData;
	};

	FEventHandlers EventHandlers = {
		.AddRequirementsForSpawnQuery = AddRequirementsForSpawnQuery,
		.CacheFragmentViewsForSpawnQuery = CacheFragmentViewsForSpawnQuery,
		.SetSpawnedEntityData = SetSpawnedEntityData
	};

	PostReplicatedAddEntitiesHelper(AddedIndices, EventHandlers);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedChangeHelper(const TArrayView<int32> ChangedIndices, FSetModifiedEntityDataFunction SetModifiedEntityData)
{
	FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	// Go through the changed Entities and update their Mass data
	for (int32 Idx : ChangedIndices)
	{
		const AgentArrayItem& ChangedItem = (*Agents)[Idx];

		const FMassReplicationEntityInfo* EntityInfo = ReplicationSubsystem->FindMassEntityInfo(ChangedItem.Agent.GetNetID());

		checkf(EntityInfo, TEXT("EntityInfo must be valid if the Agent has already been added (which it must have been to get PostReplicatedChange"));
		checkf(EntityInfo->ReplicationID >= ChangedItem.ReplicationID, TEXT("ReplicationID out of sync, this should never happen!"));

		// Currently we don't think this should be needed, but are leaving it in for bomb proofing.
		if (ensure(EntityInfo->ReplicationID == ChangedItem.ReplicationID))
		{
			FMassEntityView EntityView(EntityManager, EntityInfo->Entity);
			SetModifiedEntityData(EntityView, ChangedItem.Agent);
		}
	}
}

template<typename AgentArrayItem>
template<typename TEventHandlers>
requires (!UE::Mass::Replication::TIsCallable<TEventHandlers>)
void TClientBubbleHandlerBase<AgentArrayItem>::PostReplicatedChangeHelper(const TArrayView<int32> ChangedIndices, TEventHandlers& HandlersPack)
{
	FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	// Go through the changed Entities and update their Mass data
	for (int32 Idx : ChangedIndices)
	{
		const AgentArrayItem& ChangedItem = (*Agents)[Idx];

		const FMassReplicationEntityInfo* EntityInfo = ReplicationSubsystem->FindMassEntityInfo(ChangedItem.Agent.GetNetID());

		checkf(EntityInfo, TEXT("EntityInfo must be valid if the Agent has already been added (which it must have been to get PostReplicatedChange"));
		checkf(EntityInfo->ReplicationID >= ChangedItem.ReplicationID, TEXT("ReplicationID out of sync, this should never happen!"));

		// Currently we don't think this should be needed, but are leaving it in for bomb proofing.
		if (ensure(EntityInfo->ReplicationID == ChangedItem.ReplicationID))
		{
			FMassEntityView EntityView(EntityManager, EntityInfo->Entity);
			HandlersPack.SetModifiedEntityData(EntityView, ChangedItem.Agent);
		}
	}
}
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::SetClientHandle(FMassClientHandle InClientHandle)
{
	ClientHandle = InClientHandle;
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::Reset()
{
	(*Agents).Reset();

#if UE_REPLICATION_COMPILE_SERVER_CODE
	AgentHandleManager.Reset();
	NetworkIDToAgentHandleMap.Reset();
	AgentLookupArray.Reset();
#endif //UE_REPLICATION_COMPILE_SERVER_CODE
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::Tick(float DeltaTime)
{
	UWorld* World = Serializer->GetWorld();
	check(World);

	const ENetMode NetMode = World->GetNetMode();

	if (NetMode != NM_Client)
	{
		DebugValidateBubbleOnServer();
	}
	else
	{
		DebugValidateBubbleOnClient();

		UpdateAgentsToRemove();
	}
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::DebugValidateBubbleOnClient()
{
#if UE_ALLOW_DEBUG_REPLICATION
	const FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
	check(ReplicationSubsystem);

	for (int32 Idx = 0; Idx < (*Agents).Num(); ++Idx)
	{
		const AgentArrayItem& Item = (*Agents)[Idx];
		const typename AgentArrayItem::FReplicatedAgentType& Agent = Item.Agent;

		check(Agent.GetTemplateID().IsValid());
		check(Agent.GetNetID().IsValid());

		const FMassReplicationEntityInfo* EntityInfo = ReplicationSubsystem->FindMassEntityInfo(Agent.GetNetID());
		checkf(EntityInfo, TEXT("There should always be an EntityInfoMap entry for Agents that are in the Agents array!"));

		if (EntityInfo)
		{
			if (EntityInfo->ReplicationID == Item.ReplicationID)
			{
				const bool bIsEntityValid = EntityManager.IsEntityValid(EntityInfo->Entity);
				checkf(bIsEntityValid, TEXT("Must be valid entity if at latest ReplciationID"));

				if (bIsEntityValid)
				{
					const FMassNetworkIDFragment& FragmentNetID = EntityManager.GetFragmentDataChecked<FMassNetworkIDFragment>(EntityInfo->Entity);

					checkf(FragmentNetID.NetID == Agent.GetNetID(), TEXT("Fragment and Agent NetID do not match!"));
				}
			}
		}
	}

#if UE_ALLOW_DEBUG_SLOW_REPLICATION
	const TMap<FMassNetworkID, FMassReplicationEntityInfo>& EntityInfoMap = ReplicationSubsystem->GetEntityInfoMap();

	for (TMap<FMassNetworkID, FMassReplicationEntityInfo>::TConstIterator Iter = EntityInfoMap.CreateConstIterator(); Iter; ++Iter)
	{
		if (Iter->Value.Entity.IsSet())
		{
			checkf(AgentsRemoveDataMap.Find(Iter->Key) == nullptr, TEXT("If Entity.IsSet() we should not have an entry in AgentsRemoveDataMap!"));
		}
	}

	for (TMap<FMassNetworkID, FMassAgentRemoveData>::TIterator Iter = AgentsRemoveDataMap.CreateIterator(); Iter; ++Iter)
	{
		const FMassReplicationEntityInfo* EntityInfo = EntityInfoMap.Find(Iter->Key);
		check(EntityInfo);

		checkf(EntityInfo->Entity.IsSet() == false, TEXT("AgentsRemoveDataMap NetIDs in EntityInfoMap should be !Entity.IsSet()"));
	}
#endif // UE_ALLOW_DEBUG_SLOW_REPLICATION

#endif // UE_ALLOW_DEBUG_REPLICATION
}

template<typename AgentArrayItem>
void TClientBubbleHandlerBase<AgentArrayItem>::DebugValidateBubbleOnServer()
{
#if UE_ALLOW_DEBUG_REPLICATION
	using namespace UE::Mass::Replication;

	const FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

	for (int32 OuterIdx = 0; OuterIdx < (*Agents).Num(); ++OuterIdx)
	{
		const AgentArrayItem& OuterItem = (*Agents)[OuterIdx];

		//check there are no duplicate NetID's'
		if (OuterItem.Agent.GetNetID().IsValid())
		{
			for (int32 InnerIdx = OuterIdx + 1; InnerIdx < (*Agents).Num(); ++InnerIdx)
			{
				const AgentArrayItem& InnerItem = (*Agents)[InnerIdx];

				checkf(InnerItem.Agent.GetNetID() != OuterItem.Agent.GetNetID(), TEXT("Repeated NetIDs in the server Agents array"));
			}
		}
		checkf(OuterItem.Agent.GetTemplateID().IsValid(), TEXT("Server Agent with Invalid TemplateID"));
		checkf(AgentHandleManager.IsValidHandle(OuterItem.GetHandle()), TEXT("Server Agent with Invalid Handle"));
		checkf(OuterItem.Agent.GetNetID().IsValid(), TEXT("Server Agent with Invalid NetID"));

		const FMassAgentLookupData& LookupData = AgentLookupArray[OuterItem.GetHandle().GetIndex()];

		checkf(LookupData.AgentsIdx == OuterIdx, TEXT("Agent index must match lookup data!"));
		checkf(EntityManager.IsEntityValid(LookupData.Entity), TEXT("Must be valid entity"));

		const FMassNetworkIDFragment& FragmentNetID = EntityManager.GetFragmentDataChecked<FMassNetworkIDFragment>(LookupData.Entity);

		checkf(FragmentNetID.NetID == OuterItem.Agent.GetNetID(), TEXT("Fragment and Agent NetID do not match!"));
		checkf(LookupData.NetID == OuterItem.Agent.GetNetID(), TEXT("LookupData and Agent NetID do not match!"));

		const FReplicationTemplateIDFragment& FragmentTemplateID = EntityManager.GetFragmentDataChecked<FReplicationTemplateIDFragment>(LookupData.Entity);
		checkf(FragmentTemplateID.ID == OuterItem.Agent.GetTemplateID(), TEXT("Agent TemplateID different to Fragment!"));
	}

	checkf(AgentHandleManager.CalcNumUsedHandles() == (*Agents).Num(), TEXT("Num used Agent handles must be the same as the size of the agents array!"));
#endif // UE_ALLOW_DEBUG_REPLICATION
}
