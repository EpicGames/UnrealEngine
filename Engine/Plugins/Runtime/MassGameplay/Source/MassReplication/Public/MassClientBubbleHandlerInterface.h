// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationTypes.h"
#include "Containers/ArrayView.h"

class UWorld;

/**
 * Interface for the bubble handler classes. All the outside interaction with the FastArray logic should be done via the Handler interface
 * or derived classes where possible.
 * These virtual functions are either only called once each per frame on the client for a few struct instances
 * or called at startup / shutdown.
 */
class IClientBubbleHandlerInterface
{
public:
	virtual ~IClientBubbleHandlerInterface() = default;

	virtual void InitializeForWorld(UWorld& InWorld) = 0;

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	/** These functions are processed internally by TClientBubbleHandlerBase */
	virtual void PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize) = 0;
	virtual void PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize) = 0;
	virtual void PostReplicatedChange(const TArrayView<int32> ChangedIndices, int32 FinalSize) = 0;
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

	virtual void Reset() = 0;
	virtual void UpdateAgentsToRemove() = 0;

	virtual void Tick(float DeltaTime) = 0;
	virtual void SetClientHandle(FMassClientHandle InClientHandle) = 0;

	virtual void DebugValidateBubbleOnServer() = 0;
	virtual void DebugValidateBubbleOnClient() = 0;
};
