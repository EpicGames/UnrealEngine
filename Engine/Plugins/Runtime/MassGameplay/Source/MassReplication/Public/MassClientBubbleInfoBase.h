// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Info.h"
#include "MassReplicationTypes.h"
#include "Engine/World.h"
#include "MassClientBubbleInfoBase.generated.h"

#define UE_API MASSREPLICATION_API

struct FMassClientBubbleSerializerBase;

/** The info actor base class that provides the actual replication */
UCLASS(MinimalAPI)
class AMassClientBubbleInfoBase : public AInfo
{
	GENERATED_BODY()

public:
	UE_API AMassClientBubbleInfoBase(const FObjectInitializer& ObjectInitializer);

	UE_API void SetClientHandle(FMassClientHandle InClientHandle);

protected:
	UE_API virtual void PostInitProperties() override;

	// Called either on PostWorldInit() or PostInitProperties()
	UE_API virtual void InitializeForWorld(UWorld& World);

	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void Tick(float DeltaTime) override;

private:
	void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues);

protected:
	FDelegateHandle OnPostWorldInitDelegateHandle;
	TArray<FMassClientBubbleSerializerBase*> Serializers;
};

#undef UE_API
