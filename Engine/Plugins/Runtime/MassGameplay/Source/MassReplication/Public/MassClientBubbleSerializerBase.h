// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetSerialization.h"
#include "MassReplicationTypes.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "MassClientBubbleSerializerBase.generated.h"

#define UE_API MASSREPLICATION_API


class UWorld;
class UMassSpawnerSubsystem;
struct FMassEntityManager;
class IClientBubbleHandlerInterface;

/**
 * Classes derived from this will contain the IClientBubbleHandlerInterface derived class as well as the actual Fast Array.
 * This class mainly provides the base automation with the IClientBubbleHandlerInterface
 */
USTRUCT()
struct FMassClientBubbleSerializerBase : public FFastArraySerializer
{
	GENERATED_BODY()

public:

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UE_API void PreReplicatedRemove(const TArrayView<int32> RemovedIndices, int32 FinalSize) const;
	UE_API void PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize) const;
	UE_API void PostReplicatedChange(const TArrayView<int32> ChangedIndices, int32 FinalSize) const;
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

	UE_API void InitializeForWorld(UWorld& InWorld);

	UWorld* GetWorld() const { return World; }
	UMassSpawnerSubsystem* GetSpawnerSubsystem() const { return SpawnerSubsystem; }
	UMassReplicationSubsystem* GetReplicationSubsystem() const { return ReplicationSubsystem; }
	FMassEntityManager& GetEntityManagerChecked() const { check(EntityManager); return *EntityManager.Get(); }

	void SetClientHandler(IClientBubbleHandlerInterface& InClientHandler) { ClientHandler = &InClientHandler; }
	IClientBubbleHandlerInterface* GetClientHandler() const { return ClientHandler; }

	UE_API void ShutDown();

private:
	UPROPERTY(Transient, NotReplicated)
	TObjectPtr<UWorld> World = nullptr;

	UPROPERTY(Transient, NotReplicated)
	TObjectPtr<UMassSpawnerSubsystem> SpawnerSubsystem = nullptr;

	TSharedPtr<FMassEntityManager> EntityManager;

	UPROPERTY(Transient, NotReplicated)
	TObjectPtr<UMassReplicationSubsystem> ReplicationSubsystem = nullptr;

	/** Pointer to the IClientBubbleHandlerInterface derived class in the class derived from this one */
	IClientBubbleHandlerInterface* ClientHandler = nullptr;
};

#undef UE_API
