// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassReplicationTypes.h"
#include "MassLODCollector.h"
#include "MassLODCalculator.h"
#include "MassObserverProcessor.h"
#include "MassReplicationFragments.generated.h"

#define UE_API MASSREPLICATION_API

class AMassClientBubbleInfoBase;
class UMassReplicatorBase;

/**
 *  Fragment type for the mass network id of a mass entity
 */
USTRUCT()
struct FMassNetworkIDFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassNetworkID NetID;
};

/**
 * Agent handle per client, these will be at TArray indices of the Client handles indicies (used as a free list array)
 */
USTRUCT()
struct FMassReplicatedAgentFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassReplicatedAgentData AgentData;
};

/*
 * Data fragment to store the calculated distances to viewers
 */
USTRUCT()
struct FMassReplicationViewerInfoFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Closest viewer distance */
	float ClosestViewerDistanceSq;

	/** Distance between each viewer and entity */
	TArray<float> DistanceToViewerSq;
};

template<>
struct TMassFragmentTraits<FMassReplicationViewerInfoFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FMassReplicationLODFragment : public FMassFragment
{
	GENERATED_BODY()

	/**LOD information */
	TEnumAsByte<EMassLOD::Type> LOD = EMassLOD::Max;

	/** Previous LOD information*/
	TEnumAsByte<EMassLOD::Type> PrevLOD = EMassLOD::Max;
};

UCLASS(MinimalAPI)
class UMassNetworkIDFragmentInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassNetworkIDFragmentInitializer();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassReplicationParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()
public:
	FMassReplicationParameters();

	/** Distance where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float LODDistance[EMassLOD::Max];

	/** Hysteresis percentage on delta between the LOD distances */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float BufferHysteresisOnDistancePercentage = 10.0f;

	/** Maximum limit of entity per LOD */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	int32 LODMaxCount[EMassLOD::Max];

	/** Maximum limit of entity per LOD per viewer */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	int32 LODMaxCountPerViewer[EMassLOD::Max];

	/** Distance where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float UpdateInterval[EMassLOD::Max];

	UPROPERTY(EditAnywhere, Category = "Mass|Replication", config)
	TSubclassOf<AMassClientBubbleInfoBase> BubbleInfoClass;

	UPROPERTY(EditAnywhere, Category = "Mass|Replication", config)
	TSubclassOf<UMassReplicatorBase> ReplicatorClass;

};

USTRUCT()
struct FMassReplicationSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()
public:
	FMassReplicationSharedFragment() = default;
	FMassReplicationSharedFragment(UMassReplicationSubsystem& ReplicationSubsystem, const FMassReplicationParameters& Params);

	FMassBubbleInfoClassHandle BubbleInfoClassHandle;

	FMassClientHandle CurrentClientHandle;
	TArray<FMassClientHandle> CachedClientHandles;

	//TODO review if we need to have this as a UPROPERTY at all and also if we can make this use a TInlineAllocator
	//Can not use TInlineAllocator with UPROPERTY()
	UPROPERTY(Transient)
	TArray<TObjectPtr<AMassClientBubbleInfoBase>> BubbleInfos;

	TMassLODCollector<FReplicationLODLogic> LODCollector;
	TMassLODCalculator<FReplicationLODLogic> LODCalculator;
	bool bHasAdjustedDistancesFromCount = false;

	bool bEntityQueryInitialized = false;
	FMassEntityQuery EntityQuery;

	UPROPERTY(Transient)
	mutable TObjectPtr<UMassReplicatorBase> CachedReplicator = nullptr;

	bool IsValid() const
	{
		return BubbleInfoClassHandle.IsValid() && CachedReplicator;
	}

	template<typename T>
	T& GetTypedClientBubbleInfoChecked(FMassClientHandle Handle)
	{
		checkSlow(BubbleInfos.IsValidIndex(Handle.GetIndex()));

		AMassClientBubbleInfoBase* BubbleInfo = BubbleInfos[Handle.GetIndex()];

		checkSlow(BubbleInfo && Cast<T>(BubbleInfo) != nullptr);

		return *static_cast<T*>(BubbleInfo);
	}
};

/** Cell location for replicated mass agents, used to fetch quickly the agents around each clients */
USTRUCT()
struct FMassReplicationGridCellLocationFragment : public FMassFragment
{
	GENERATED_BODY()
	FReplicationHashGrid2D::FCellLocation CellLoc;
};

/** Component Tag to tell if the entity is in the replication grid */
USTRUCT()
struct FMassInReplicationGridTag : public FMassTag
{
	GENERATED_BODY()
};

#undef UE_API
