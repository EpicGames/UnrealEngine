// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MassReplicationGridProcessor.generated.h"

#define UE_API MASSREPLICATION_API

class UMassReplicationSubsystem;

/** Processor to update entity in the replication grid used to fetch entities close to clients */
UCLASS(MinimalAPI)
class UMassReplicationGridProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassReplicationGridProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery AddToGridEntityQuery;
	FMassEntityQuery UpdateGridEntityQuery;
	FMassEntityQuery RemoveFromGridEntityQuery;
};

/** Deinitializer processor to remove entity from the replication grid */
UCLASS(MinimalAPI)
class UMassReplicationGridRemoverProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassReplicationGridRemoverProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
