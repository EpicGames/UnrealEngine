// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "MassProcessor.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSimulationLOD.h"
#include "MassReplicationTypes.h"
#include "MassReplicationFragments.h"
#include "MassSpawnerTypes.h"
#include "MassReplicationSubsystem.h"
#include "MassReplicationProcessor.generated.h"


class UMassReplicationSubsystem;
class AMassClientBubbleInfoBase;
class UWorld;

/** 
 *  Base processor that handles replication and only runs on the server. You should derive from this per entity type (that require different replication processing). It and its derived classes 
 *  query Mass entity fragments and set those values for replication when appropriate, using the MassClientBubbleHandler.
 */
UCLASS(MinimalAPI)
class UMassReplicationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSREPLICATION_API UMassReplicationProcessor();

protected:
	MASSREPLICATION_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSREPLICATION_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSREPLICATION_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	MASSREPLICATION_API void PrepareExecution(FMassEntityManager& EntityManager);

protected:

	UPROPERTY(Transient)
	TObjectPtr<UMassReplicationSubsystem> ReplicationSubsystem = nullptr;

	FMassEntityQuery SyncClientData;
	FMassEntityQuery CollectViewerInfoQuery;
	FMassEntityQuery CalculateLODQuery;
	FMassEntityQuery AdjustLODDistancesQuery;
	FMassEntityQuery EntityQuery;
};

UCLASS(MinimalAPI)
class UMassReplicationEntityDestructionObserver : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	MASSREPLICATION_API UMassReplicationEntityDestructionObserver();

protected:
	MASSREPLICATION_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSREPLICATION_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

struct FMassReplicationContext
{
	FMassReplicationContext(UWorld& InWorld, const UMassLODSubsystem& InLODSubsystem, UMassReplicationSubsystem& InReplicationSubsystem)
		: World(InWorld)
		, LODSubsystem(InLODSubsystem)
		, ReplicationSubsystem(InReplicationSubsystem)
	{}

	UWorld& World;
	const UMassLODSubsystem& LODSubsystem;
	UMassReplicationSubsystem& ReplicationSubsystem;
};

UCLASS(MinimalAPI, Abstract)
class UMassReplicatorBase : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Must override to add specific entity query requirements for replication
	 * Usually we add replication processor handler requirements
	 */
	virtual void AddRequirements(FMassEntityQuery& EntityQuery) PURE_VIRTUAL(UMassReplicatorBase::ConfigureQueries, );

	/**
	 * Must override to process the client replication
	 * This methods should call CalculateClientReplication with the appropriate callback implementation
	 */
	virtual void ProcessClientReplication(FMassExecutionContext& Context, FMassReplicationContext& ReplicationContext) PURE_VIRTUAL(UMassReplicatorBase::ProcessClientReplication, );

protected:
	/**
	 *  Implemented as straight template callbacks as when profiled this was faster than TFunctionRef. Its probably easier to pass Lamdas in to these
	 *  but Functors can also be used as well as TFunctionRefs etc. Its also fairly straight forward to call member functions via some Lamda glue code
	 */
	template<typename AgentArrayItem, typename CacheViewsCallback, typename AddEntityCallback, typename ModifyEntityCallback, typename RemoveEntityCallback>
	static void CalculateClientReplication(FMassExecutionContext& Context, FMassReplicationContext& ReplicationContext, CacheViewsCallback&& CacheViews, AddEntityCallback&& AddEntity, ModifyEntityCallback&& ModifyEntity, RemoveEntityCallback&& RemoveEntity);
};

template<typename AgentArrayItem, typename CacheViewsCallback, typename AddEntityCallback, typename ModifyEntityCallback, typename RemoveEntityCallback>
void UMassReplicatorBase::CalculateClientReplication(FMassExecutionContext& Context, FMassReplicationContext& ReplicationContext, CacheViewsCallback&& CacheViews
	, AddEntityCallback&& AddEntity, ModifyEntityCallback&& ModifyEntity, RemoveEntityCallback&& RemoveEntity)
{
#if UE_REPLICATION_COMPILE_SERVER_CODE
	TConstArrayView<FMassNetworkIDFragment> NetworkIDList = Context.GetFragmentView<FMassNetworkIDFragment>();
	TArrayView<FMassReplicationLODFragment> ViewerLODList = Context.GetMutableFragmentView<FMassReplicationLODFragment>();
	TArrayView<FMassReplicatedAgentFragment> ReplicatedAgentList = Context.GetMutableFragmentView<FMassReplicatedAgentFragment>();
	TConstArrayView<FReplicationTemplateIDFragment> TemplateIDList = Context.GetFragmentView<FReplicationTemplateIDFragment>();
	FMassReplicationSharedFragment& RepSharedFragment = Context.GetMutableSharedFragment<FMassReplicationSharedFragment>();

	CacheViews(Context);

	const double Time = ReplicationContext.World.GetRealTimeSeconds();
	const FMassClientHandle& ClientHandle = RepSharedFragment.CurrentClientHandle;
	check(ClientHandle.IsValid());
	checkSlow(RepSharedFragment.BubbleInfos[ClientHandle.GetIndex()] != nullptr);
	
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		FMassReplicatedAgentFragment& AgentFragment = ReplicatedAgentList[EntityIt];
		FMassReplicatedAgentData& AgentData = AgentFragment.AgentData;

		const EMassLOD::Type LOD = ViewerLODList[EntityIt].LOD;
		if (LOD < EMassLOD::Off)
		{
			AgentData.LOD = LOD;

			//if the handle isn't valid we need to add the agent
			if (!AgentData.Handle.IsValid())
			{
				typename AgentArrayItem::FReplicatedAgentType ReplicatedAgent;

				const FMassNetworkIDFragment& NetIDFragment = NetworkIDList[EntityIt];
				const FReplicationTemplateIDFragment& TemplateIDFragment = TemplateIDList[EntityIt];

				ReplicatedAgent.SetNetID(NetIDFragment.NetID);
				ReplicatedAgent.SetTemplateID(TemplateIDFragment.ID);

				AgentData.Handle = AddEntity(Context, EntityIt, ReplicatedAgent, ClientHandle);

				AgentData.LastUpdateTime = Time;
			}
			else
			{
				ModifyEntity(Context, EntityIt, LOD, Time, AgentData.Handle, ClientHandle);
			}
		}
		else
		{
			// as this is a fresh handle, if its valid then we can use the unsafe remove function
			if (AgentData.Handle.IsValid())
			{
				if constexpr (std::is_invocable_v<RemoveEntityCallback, FMassExecutionContext&, FMassEntityHandle, FMassReplicatedAgentHandle, FMassClientHandle>)
				{
					RemoveEntity(Context, Context.GetEntity(EntityIt), AgentData.Handle, ClientHandle);
				}
				else
				{
					RemoveEntity(Context, AgentData.Handle, ClientHandle);
				}
				AgentData.Invalidate();
			}
		}
	}

	// Handle destroyed entities
	FMassClientReplicationInfo& ClientReplicationInfo = ReplicationContext.ReplicationSubsystem.GetMutableClientReplicationInfoChecked(ClientHandle);
	for (FMassReplicationAgentDataMap::TIterator It = ClientReplicationInfo.AgentsData.CreateIterator(); It; ++It)
	{
		FMassReplicatedAgentData& AgentData = It.Value();
		if (AgentData.Handle.IsValid() && AgentData.bPendingDestruction)
		{
			if constexpr (std::is_invocable_v<RemoveEntityCallback, FMassExecutionContext&, FMassEntityHandle, FMassReplicatedAgentHandle, FMassClientHandle>)
			{
				RemoveEntity(Context, It.Key(), AgentData.Handle, ClientHandle);
			}
			else
			{
				RemoveEntity(Context, AgentData.Handle, ClientHandle);
			}
			AgentData.Invalidate();
		}
	}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE
}
