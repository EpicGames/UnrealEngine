// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassReplicationSettings.generated.h"

UCLASS(MinimalAPI, config = Mass, defaultconfig, meta = (DisplayName = "Mass Replication"))
class UMassReplicationSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	static const UMassReplicationSettings* Get()
	{
		return GetDefault<UMassReplicationSettings>();
	}

	float GetReplicationGridCellSize() const
	{ 
		return ReplicationGridCellSize; 
	}
	
private:

	UPROPERTY(EditAnywhere, config, Category = Replication);
	float ReplicationGridCellSize = 5000.0f; // 50 meters;
};
