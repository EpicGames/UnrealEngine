// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassEntityTypes.h"
#include "MassReplicationTypes.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassReplicationSubsystem.generated.h"

#define UE_API MASSREPLICATION_API

struct FMassEntityManager;
class UMassLODSubsystem;
class AMassClientBubbleInfoBase;

typedef TMap<FMassEntityHandle, FMassReplicatedAgentData> FMassReplicationAgentDataMap;

struct FMassClientReplicationInfo
{
	void Reset()
	{
		Handles.Reset();
		HandledEntities.Reset();
		AgentsData.Reset();
	}

	/** Note this struct is constructed IsEmpty() == true */
	bool IsEmpty() const
	{
		return (Handles.Num() == 0) && (HandledEntities.Num() == 0) && (AgentsData.Num() == 0);
	}

	/** Array of all the viewer of this client */
	TArray<FMassViewerHandle> Handles;

	/** Array of all the entities handled by this client 
	  * This array might contains duplicates if there is more than one viewer per client as this concatenates the entities from all viewer of a client) */
	TArray<FMassEntityHandle> HandledEntities;

	/** The saved agent data of the entities handle by this client */
	FMassReplicationAgentDataMap AgentsData;
};

struct FViewerClientPair
{
	FViewerClientPair() = default;

	FViewerClientPair(FMassViewerHandle InViewerHandle, FMassClientHandle InClientHandle)
		: ViewerHandle(InViewerHandle)
		, ClientHandle(InClientHandle)
	{}

	bool operator== (FViewerClientPair Other) const { return ViewerHandle == Other.ViewerHandle && ClientHandle == Other.ClientHandle; }

	bool IsValid() const { return ViewerHandle.IsValid() && ClientHandle.IsValid(); }
	void Invalidate() { ViewerHandle.Invalidate(); ClientHandle.Invalidate(); }

	FMassViewerHandle ViewerHandle;
	FMassClientHandle ClientHandle;
};

USTRUCT()
struct FMassClientBubbleInfoData
{
	GENERATED_BODY()

	FMassClientBubbleInfoData() = default;

	FMassClientBubbleInfoData(TSubclassOf<AMassClientBubbleInfoBase> InBubbleClass)
		: BubbleClass(InBubbleClass)
	{}

	/** A free list array of AMassClientBubbleInfos. This is organised so the index is that of the client FMassClientHandle */
	UPROPERTY(Transient)
	TArray<TObjectPtr<AMassClientBubbleInfoBase>> Bubbles;

	UPROPERTY(Transient)
	TSubclassOf<AMassClientBubbleInfoBase> BubbleClass;
};

namespace UE::MassReplication
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FMassAgentDelegate, FMassNetworkID, FMassEntityHandle);

} // UE::MassReplication

/**
 *  Manages the creation of NetworkIDs, ClientBubbles and ClientReplicationInfo.
 *  NetworkIDs are per replicated Agent Entity and are unique and replicated between server and clients.
 *  ClientBubbles relate to the player controller that owns the parent UNetConnection to a Client machine.
 *  ClientReplicationInfo relate to all the player controllers that have a parent or child UNetConnection to a single Client machine (split screen etc).
 */
UCLASS(MinimalAPI)
class UMassReplicationSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:

	UE_API UMassReplicationSubsystem();

	enum class EFindOrAddMassEntityInfo : uint8
	{
		FoundOlderReplicationID,
		FoundNewerReplicationID,
		Added,
	};

	bool IsBubbleClassHandleValid(FMassBubbleInfoClassHandle Handle) const
	{
		return BubbleInfoArray.IsValidIndex(Handle.GetIndex());
	}

	/** Retrieve network id from Mass handle. */
	UE_API FMassNetworkID GetNetIDFromHandle(const FMassEntityHandle Handle) const;

	static inline FMassNetworkID GetNextAvailableMassNetID() { return FMassNetworkID(++CurrentNetMassCounter); }

	const TArray<FMassClientHandle>& GetClientReplicationHandles() const { return ClientHandleManager.GetHandles(); }

	bool IsValidClientHandle(FMassClientHandle ClientHandle) const
	{
		return ClientHandleManager.IsValidHandle(ClientHandle);
	}

	/** Gets the client bubble safely */
	AMassClientBubbleInfoBase* GetClientBubble(FMassBubbleInfoClassHandle BubbleClassHandle, FMassClientHandle ClientHandle) const
	{
		AMassClientBubbleInfoBase* ClientBubble = nullptr;

		if (IsBubbleClassHandleValid(BubbleClassHandle) && ClientHandleManager.IsValidHandle(ClientHandle))
		{
			ClientBubble = BubbleInfoArray[BubbleClassHandle.GetIndex()].Bubbles[ClientHandle.GetIndex()];
			check(ClientBubble);
		}
		return ClientBubble;
	}

	/** Get the client bubble. Faster version using check()s */
	AMassClientBubbleInfoBase* GetClientBubbleChecked(FMassBubbleInfoClassHandle BubbleClassHandle, FMassClientHandle ClientHandle) const
	{
		check(IsBubbleClassHandleValid(BubbleClassHandle));
		check(ClientHandleManager.IsValidHandle(ClientHandle));
		AMassClientBubbleInfoBase* ClientBubble = BubbleInfoArray[BubbleClassHandle.GetIndex()].Bubbles[ClientHandle.GetIndex()];

		check(ClientBubble);
		return ClientBubble;
	}

	/** Gets the client bubble safely returning the template type */
	template<typename TType>
	TType* GetTypedClientBubble(FMassBubbleInfoClassHandle BubbleClassHandle, FMassClientHandle ClientHandle) const
	{
		return Cast<TType>(GetClientBubble(BubbleClassHandle, ClientHandle));
	}

	/** Gets the client bubble returning the template type. Faster version using check()s */
	template<typename TType>
	TType* GetTypedClientBubbleChecked(FMassBubbleInfoClassHandle BubbleClassHandle, FMassClientHandle ClientHandle) const
	{
		AMassClientBubbleInfoBase* ClientBubble = GetClientBubbleChecked(BubbleClassHandle, ClientHandle);
		checkSlow(Cast<TType>(ClientBubble) != nullptr);

		return static_cast<TType *>(ClientBubble);
	}

	/** Gets the client replication info safely */
	const FMassClientReplicationInfo* GetClientReplicationInfo(FMassClientHandle Handle) const
	{
		return ClientHandleManager.IsValidHandle(Handle) ? &(ClientsReplicationInfo[Handle.GetIndex()]) : nullptr;
	}

	/** Gets the client replication info. Faster version using check()s */
	const FMassClientReplicationInfo& GetClientReplicationInfoChecked(FMassClientHandle Handle) const
	{
		check(ClientHandleManager.IsValidHandle(Handle));

		return ClientsReplicationInfo[Handle.GetIndex()];
	}

	/** Gets the client replication info. Faster version using check()s */
	FMassClientReplicationInfo* GetMutableClientReplicationInfo(FMassClientHandle Handle)
	{
		return const_cast<FMassClientReplicationInfo*>(GetClientReplicationInfo(Handle));
	}

	/** Gets the client replication info. Faster version using check()s */
	FMassClientReplicationInfo& GetMutableClientReplicationInfoChecked(FMassClientHandle Handle)
	{
		return *const_cast<FMassClientReplicationInfo*>(&GetClientReplicationInfoChecked(Handle));
	}

	UE_API void SynchronizeClientsAndViewers();

	/** 
	 * Registers BubbleInfoClass with the system. These are created one per client as clients join. This function can be called multiple times with
	 * different BubbleInfoClass types (but not the same type), in that case each client will get multiple BubbleInfoClasses. This can be useful for 
	 * replicating different Entity types in different bubbles, although its also possible to have multiple TClientBubbleHandlerBase derived class
	 * instances per BubbleInfoClass. This must not be called after AddClient or SynchronizeClientsAndViewers is called
	 * @return FMassBubbleInfoClassHandle Handle to the BubbleInfoClass, this will be an invalid handle if its not been created
	 */
	UE_API FMassBubbleInfoClassHandle RegisterBubbleInfoClass(const TSubclassOf<AMassClientBubbleInfoBase>& BubbleInfoClass);

	/** @return FMassBubbleInfoClassHandle Handle to the BubbleInfoClass, this will be an invalid handle if BubbleInfoClass can not be found */
	UE_API FMassBubbleInfoClassHandle GetBubbleInfoClassHandle(const TSubclassOf<AMassClientBubbleInfoBase>& BubbleInfoClass) const;

	const FReplicationHashGrid2D& GetGrid() const { return ReplicationGrid; }
	FReplicationHashGrid2D& GetGridMutable() { return ReplicationGrid; }

#if UE_REPLICATION_COMPILE_SERVER_CODE
	void NotifyEntityDestroyed(FMassEntityHandle EntityHandle);
#endif

#if UE_REPLICATION_COMPILE_CLIENT_CODE

	/** 
	 * Must be called immediately after an Entity is added to the simulation and after FindAndUpdateOrAddMassEntityInfo has been called
	 */
	UE_API void SetEntity(const FMassNetworkID NetworkID, const FMassEntityHandle Entity);

	/** 
	 * Resets the item with NetworkID in the  EntityInfoMap if its currently Set / Valid and its the most recent ReplicationID. In this case the FMassEntityHandle that was reset is returned.
	 * Otherwise FMassEntityHandle::IsSet() will be false on the returned FMassEntityHandle
	 * Must be called after SetEntity and just before the Entity is removed on the client. This sets the Entity member of the AddToEntityInfoMap item to FMassEntityHandle so it will be not IsSet() 
	 */
	UE_API FMassEntityHandle ResetEntityIfValid(const FMassNetworkID NetworkID, int32 ReplicationID);

	UE_API void RemoveFromEntityInfoMap(const FMassNetworkID NetworkID);

	/** 
	 * Finds or adds an FMassReplicationEntityInfo to the EntityInfoMap for an Entity with NetWorkID. This will update the ReplicationID if an older one is found but it does not change the 
	 * FMassReplicationEntityInfo::Entity member. Returns whether we added an entry in EntityInfoMap, or if the found item has an older or newer ReplicationID than that that is passed in. 
	 * We shouldn't actually get an equal ReplicationID, but in this case we'll treat the existing EntityInfoMap entry as newer. SetEntity() must be called soon after this if the entity handle is !IsSet()
	 * @param NetworkID ID of the entity we are looking up OutMassEntityInfo for
	 * @param ReplicationID this is the ReplicationID of the FastArray item
	 * @param OutMassEntityInfo retrieved or added FMassReplicationEntityInfo
	 * @return EFindOrAddMassEntityInfo, result of the function call
	 */
	UE_API EFindOrAddMassEntityInfo FindAndUpdateOrAddMassEntityInfo(const FMassNetworkID NetworkID, int32 ReplicationID, const FMassReplicationEntityInfo*& OutMassEntityInfo);

	/** 
	 * Finds the FMassReplicationEntityInfo for the Entity with NetworkID. NetworkID must be valid and nullptr is returned if we can't find it. Note FMassReplicationEntityInfo::IsSet() may be false,
	 * in which case this Entity has been removed from the client simulation (but we are still storing the ReplicationID between Removes and Adds!
	 */
	UE_API const FMassReplicationEntityInfo* FindMassEntityInfo(const FMassNetworkID NetworkID) const;

	UE_API FMassEntityHandle FindEntity(const FMassNetworkID NetworkID) const;

	const TMap<FMassNetworkID, FMassReplicationEntityInfo>& GetEntityInfoMap() const { return EntityInfoMap; }

	UE::MassReplication::FMassAgentDelegate& GetOnMassAgentAdded()
	{
		return OnMassAgentAdded;
	}

	UE::MassReplication::FMassAgentDelegate& GetOnRemovingMassAgent()
	{
		return OnRemovingMassAgent;
	}

#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

protected:
	// USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// USubsystem END
	
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UE_API FMassReplicationEntityInfo* FindMassEntityInfoMutable(const FMassNetworkID NetworkID);
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

	/** Adds a Client and sets up all the data relevant to the bubble. Where there are multiple viewers for
	 *  one NetConnection only the parent net connection should be added here
	 *  @param ViewerHandle handle to viewer to add as a client. This must be UMassLODSubsystem->IsValidViewer()
	 *  @param InController associated with the viewer
	 */
	UE_API void AddClient(FMassViewerHandle ViewerHandle, APlayerController& InController);

	/** Removes a Client and removes up all the data relevant to the bubble
	 *  @param ClientHandle handle to client to remove. This must be ClientHandleManager.IsValidHandle()
	 */
	UE_API void RemoveClient(FMassClientHandle ClientHandle);

	/** @return true if we should shrink the number of handles. */
	UE_API bool SynchronizeClients(const TArray<FViewerInfo>& Viewers);
	UE_API void SynchronizeClientViewers(const TArray<FViewerInfo>& Viewers);

	UE_API void DebugCheckArraysAreInSync();

protected:
	static UE_API uint32 CurrentNetMassCounter;

	TSharedPtr<FMassEntityManager> EntityManager;

	/** Clients free list FMassClientHandle manager, handles will to the indices of FMassClientReplicationData::ClientBubbles */
	FMassClientHandleManager ClientHandleManager;

	UPROPERTY(Transient)
	TArray<FMassClientBubbleInfoData> BubbleInfoArray;

	/** An Array of each Clients viewer handles (split screen players sharing the same client connections). 
	 *  This will include both the parent and child NetConnections per client.
	 *	The array is organized so the array index is the same as the index of the client FMassClientHandle.
	 */
	TArray<FMassClientReplicationInfo> ClientsReplicationInfo;

	/** A free list array of FViewerClientPairs. This is organized so the index is that of the FMassViewerHandle for fast lookup of the related FMassClientHandle. 
	 *  This only contains viewers that are also clients (ie only parent NetConnections not child ones).
	 */
	TArray<FViewerClientPair> ViewerToClientHandleArray;

	/** A free list array of FViewerClientPairs. This is organized so the index is that of the FMassClientHandle.
	 *  For fast lookup of the related FMassViewerHandle. This only contains viewers that are also clients (ie only parent NetConnections not child ones).
	 */
	TArray<FViewerClientPair> ClientToViewerHandleArray;

	UPROPERTY()
	TObjectPtr<UWorld> World;

	UPROPERTY()
	TObjectPtr<UMassLODSubsystem> MassLODSubsystem;

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	TMap<FMassNetworkID, FMassReplicationEntityInfo> EntityInfoMap;

	// @todo MassReplication consider batching these
	/** Broadcast just after a mass agent is added to the client simulation */
	UE::MassReplication::FMassAgentDelegate OnMassAgentAdded;

	/** Broadcast just before a mass agent is removed from the client simulation */
	UE::MassReplication::FMassAgentDelegate OnRemovingMassAgent;
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

	/**
	 * Used to make sure the FMassEntityHandle_ClientReplications are synchronized immediately before they are needed
	 * @todo this comment is no longer accurate, needs fixing
	 */
	uint64 LastSynchronizedFrame = 0;

	FReplicationHashGrid2D ReplicationGrid;
};

template<>
struct TMassExternalSubsystemTraits<UMassReplicationSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};

#undef UE_API
