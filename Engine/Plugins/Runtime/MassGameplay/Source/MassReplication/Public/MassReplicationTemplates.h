// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassReplicationTypes.h"
#include "MassClientBubbleHandlerInterface.h"
#include "MassEntityQuery.h"
#include "MassReplicationFragments.h"
#include <tuple>

/**
 * Mass Replication Templates - Quick Setup Guide
 *
 * To replicate a new fragment type:
 *
 * 1. Define your replicated data struct - USTRUCT with UPROPERTY members for serialization
 *		This is the replication-friendly data that build from server-side Mass fragments
 *
 * 2. Specialize TReplicationTraits with ExtractData/ApplyData overloads:
 *
 *		namespace UE::Mass::Replication
 *		{
 *			template<>
 *			struct TReplicationTraits<FMyReplicatedData>
 *			{
 *				static void ExtractData(const FMyFragment& F, FMyReplicatedData& Out)
 *				{
 *					Out.Value = F.Value;
 *				}
 *				static void ApplyData(FMyFragment& F, const FMyReplicatedData& D)
 *				{
 *					F.Value = D.Value;
 *				}
 *				// Optional:
 *				// static bool IsDirty(...),
 *				// static constexpr bool bSkipEntityModification
 *			};
 *		}
 *
 * 3. Declare handler types:
 *
 *		UE_MASSREPLICATION_DECLARE_HANDLERS(MyData, FMyReplicatedData, FMyFragment);
 *		// Creates: FMassReplicationProcessorMyDataHandler, FMassClientBubbleMyDataHandler
 *
 * 4. Add handler instances to your TClientBubbleHandlerBase- and UMassReplicatorBase-based classes
 *		Optionally you can replace the handlers already present there with TClientBubbleDataHandlerPack
 *		and TReplicationProcessorHandlerPack respectively, which aggregate multiple fragment handlers.
 *		See FMassCrowdClientBubbleHandler and FMassCharacterClientBubbleHandler for an example of how to do that.
 *		
 */

namespace UE::Mass::Replication
{
	/**
	 * Template for replication traits. Specialize this for each replicated data type.
	 * The specialization defines how to convert between fragments and replicated data,
	 * with fragment types handled via function overloading.
	 *
	 * Required members in specialization:
	 *	- static void ExtractData(const TFragment& Fragment, TReplicatedDataType& OutData)
	 *		One overload per fragment type that contributes data. If not provided for a fragment,
	 *		TReplicatedDataType::operator=(const TFragment&) will be used.
	 *	- static void ApplyData(TFragment& Fragment, const TReplicatedDataType& Data)
	 *		One overload per fragment type that receives data. If not provided for a fragment,
	 *		TFragment::operator=(const TReplicatedDataType&) will be used.
	 *
	 * Optional members:
	 *	- static bool IsDirty(const TReplicatedDataType& OldData, const TReplicatedDataType& NewData)
	 *		Custom dirty checking. If not provided, will use TReplicatedDataType::IsDirty() member or
	 *		fall back to operator!=.
	 *	- static constexpr bool bSkipEntityModification = true/false
	 *		If true then SetModifiedEntityData calls won't be executed for that fragment
	 *		(use for "set once" data like visualization). Not defining SetModifiedEntityData will
	 *		have the same result as SetModifiedEntityData = false.
	 *
	 *	Note that missing both a specialized trait and member functions expected in default cases will
	 *	result in compilation errors.
	 *
	 * Example:
	 *	template<>
	 *	struct TReplicationTraits<FReplicatedHealthData>
	 *	{
	 *		static void ExtractData(const FHealthFragment& Frag, FReplicatedHealthData& Out)
	 *		{
	 *			Out.Health = Frag.Health;
	 *		}
	 *		static void ApplyData(FHealthFragment& Frag, const FReplicatedHealthData& Data)
	 *		{
	 *			Frag.Health = Data.Health;
	 *		}
	 *	};
	 */
	template<typename TReplicatedDataType>
	struct TReplicationTraits;

	//-------------------------------------------------------------------------
	// Concepts for detecting trait capabilities
	//-------------------------------------------------------------------------

	/** Checks if TReplicationTraits<TReplicatedDataType> has ExtractData overload for TFragment */
	template<typename TReplicatedDataType, typename TFragment>
	concept TraitsHaveExtractData = requires(TReplicatedDataType& Data, const TFragment& Frag)
		{
			TReplicationTraits<TReplicatedDataType>::ExtractData(Frag, Data);
		}
		&& CFragment<TFragment>;

	/** Checks if TReplicationTraits<TReplicatedDataType> has ApplyData overload for TFragment */
	template<typename TReplicatedDataType, typename TFragment>
	concept TraitsHaveApplyData = requires(const TReplicatedDataType& Data, TFragment& Frag)
		{
			TReplicationTraits<TReplicatedDataType>::ApplyData(Frag, Data);
		}
		&& CFragment<TFragment>;

	/** Checks if traits define bSkipEntityModification for a fragment type AND it's true */
	template<typename TReplicatedDataType>
	concept TraitsHaveSkipEntityModification = requires
		{
			requires TReplicationTraits<TReplicatedDataType>::bSkipEntityModification;
		};

	/** Checks if TReplicatedDataType has static IsDirty member */
	template<typename TReplicatedDataType>
	concept ReplicatedDataHasIsDirty = requires(const TReplicatedDataType& A, const TReplicatedDataType& B)
		{
			{ TReplicatedDataType::IsDirty(A, B) } -> CConvertibleTo<bool>;
		};

	/** Checks if TReplicationTraits<TReplicatedDataType> has IsDirty */
	template<typename TReplicatedDataType>
	concept TraitsHaveIsDirty = requires(const TReplicatedDataType& A, const TReplicatedDataType& B)
		{
			{ TReplicationTraits<TReplicatedDataType>::IsDirty(A, B) } -> CConvertibleTo<bool>;
		};

	//-------------------------------------------------------------------------
	// Dirty checking helper
	//-------------------------------------------------------------------------

	template<typename TReplicatedDataType>
	bool CheckIsDirty(const TReplicatedDataType& OldData, const TReplicatedDataType& NewData)
	{
		if constexpr (TraitsHaveIsDirty<TReplicatedDataType>)
		{
			return TReplicationTraits<TReplicatedDataType>::IsDirty(OldData, NewData);
		}
		else if constexpr (ReplicatedDataHasIsDirty<TReplicatedDataType>)
		{
			return TReplicatedDataType::IsDirty(OldData, NewData);
		}
		else
		{
			return !(OldData == NewData);
		}
	}

	//-------------------------------------------------------------------------
	// Server-side Handler Template
	//-------------------------------------------------------------------------

	/**
	 * Server-side fragment data handler template.
	 * Handles collections of fragments and converts fragment data to replication-friendly representation.
	 * Used in UMassReplicatorBase-derived classes.
	 *
	 * @param TReplicatedDataType - The replicated data struct
	 * @param TFragments - Fragment types that contribute to this replicated data
	 */
	template<typename TReplicatedDataType, typename... TFragments>
	struct TReplicationProcessorHandler
	{
		using FReplicatedDataType = TReplicatedDataType;

		template<typename TFragmentView>
		using TFragmentType = typename TRemoveReference<TFragmentView>::Type::ElementType;

		template<CFragment T>
		TArrayView<const T>& GetView()
		{
			return std::get<TArrayView<const T>>(FragmentViewLists);
		}

		template<CFragment T>
		const TArrayView<const T>& GetView() const
		{
			return std::get<TArrayView<const T>>(FragmentViewLists);
		}

		static void AddRequirements(FMassEntityQuery& InQuery)
		{
			(InQuery.AddRequirement<TFragments>(EMassFragmentAccess::ReadOnly), ...);
		}

		void CacheFragmentViews(FMassExecutionContext& ExecContext)
		{
			std::apply([&ExecContext](auto&... FragmentView)
				{
					((
						FragmentView = ExecContext.GetFragmentView<TFragmentType<decltype(FragmentView)>>()
					), ...);
				}, FragmentViewLists);
		}

		void AddEntity(const int32 EntityIndex, FReplicatedDataType& OutReplicatedData) const
		{
			((
				ExtractSingleFragment<TFragments>(EntityIndex, OutReplicatedData)
			), ...);
		}

		/**
		 * Updates replicated data for an existing entity.
		 * Calls SetBubbleData on the client handler which performs dirty checking.
		 * Note that the function is a no-op if TReplicationTraits<FReplicatedDataType>::bSkipEntityModification == true.
		 */
		template<typename TClientBubbleHandler>
		requires TIsDerivedFrom<TClientBubbleHandler, IClientBubbleHandlerInterface>::Value
		void ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIndex, TClientBubbleHandler& ClientBubble)
		{
			if constexpr (!TraitsHaveSkipEntityModification<FReplicatedDataType>)
			{
				FReplicatedDataType NewData;
				((
					ExtractSingleFragment<TFragments>(EntityIndex, NewData)
				), ...);
				ClientBubble.SetBubbleData(Handle, NewData);
			}
		}

	private:
		template<CFragment TFragment>
		void ExtractSingleFragment(const int32 EntityIndex, FReplicatedDataType& OutReplicatedData) const
		{
			const TFragment& Fragment = GetView<TFragment>()[EntityIndex];
			if constexpr (TraitsHaveExtractData<FReplicatedDataType, TFragment>)
			{
				TReplicationTraits<FReplicatedDataType>::ExtractData(Fragment, OutReplicatedData);
			}
			else
			{
				OutReplicatedData = Fragment;
			}
		}

		std::tuple<TArrayView<const TFragments>...> FragmentViewLists;
	};

	//-------------------------------------------------------------------------
	// Client-side Handler Template
	//-------------------------------------------------------------------------

	/**
	 * Client-side bubble handler for fragments.
	 * Automatically implements all client handler methods using TReplicationTraits.
	 * Used in TClientBubbleHandlerBase-derived types.
	 *
	 * @param TReplicatedDataType - The replicated data struct
	 * @param TFragments - Fragment types that receive this replicated data
	 */
	template<typename TReplicatedDataType, CFragment... TFragments>
	struct TClientBubbleFragmentHandler
	{
	#if UE_REPLICATION_COMPILE_CLIENT_CODE
		using FReplicatedDataType = TReplicatedDataType;

		template<typename TFragmentView>
		using TFragmentType = typename TRemoveReference<TFragmentView>::Type::ElementType;

		template<CFragment T>
		TArrayView<T>& GetView()
		{
			return std::get<TArrayView<T>>(FragmentViewLists);
		}

		template<CFragment T>
		const TArrayView<T>& GetView() const
		{
			return std::get<TArrayView<T>>(FragmentViewLists);
		}

		/** Adds fragment requirements for the spawn query */
		static void AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery)
		{
			(InQuery.AddRequirement<TFragments>(EMassFragmentAccess::ReadWrite), ...);
		}

		/** Caches mutable fragment views for spawning */
		void CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext)
		{
			std::apply([&InExecContext](auto&... FragmentView)
				{
					((
						FragmentView = InExecContext.GetMutableFragmentView<TFragmentType<decltype(FragmentView)>>()
					), ...);
				}, FragmentViewLists);
		}

		/** Clears cached fragment views */
		void ClearFragmentViewsForSpawnQuery()
		{
			std::apply([](auto&... FragmentView)
				{
					((
						FragmentView = {}
					), ...);
				}, FragmentViewLists);
		}

		/** Applies replicated data to a newly spawned entity */
		void SetSpawnedEntityData(const int32 EntityIndex, const FReplicatedDataType& Data)
		{
			(ApplySingleFragment<TFragments>(EntityIndex, Data), ...);
		}

		/** Applies replicated data to an already-existing entity */
		static void SetModifiedEntityData(const FMassEntityView& EntityView, const FReplicatedDataType& Data)
		{
			if constexpr (!TraitsHaveSkipEntityModification<FReplicatedDataType>)
			{
				(ApplyModifiedFragment<TFragments>(EntityView, Data), ...);
			}
		}

	private:
		template<CFragment TFragment>
		void ApplySingleFragment(const int32 EntityIndex, const FReplicatedDataType& Data)
		{
			TFragment& Fragment = GetView<TFragment>()[EntityIndex];
			ApplyToFragment(Fragment, Data);
		}

		template<CFragment TFragment>
		static void ApplyModifiedFragment(const FMassEntityView& EntityView, const FReplicatedDataType& Data)
		{
			TFragment& Fragment = EntityView.GetFragmentData<TFragment>();
			ApplyToFragment(Fragment, Data);
		}

		template<CFragment TFragment>
		static void ApplyToFragment(TFragment& Fragment, const FReplicatedDataType& Data)
		{
			if constexpr (TraitsHaveApplyData<FReplicatedDataType, TFragment>)
			{
				TReplicationTraits<FReplicatedDataType>::ApplyData(Fragment, Data);
			}
			else
			{
				Fragment = Data;
			}
		}

		std::tuple<TArrayView<TFragments>...> FragmentViewLists;
	#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
	};

	//-------------------------------------------------------------------------
	// Handler Packs (for multiple replicated data types)
	//-------------------------------------------------------------------------

	/**
	 * Server-side handler pack for multiple replicated data types.
	 * Orchestrates multiple TReplicationProcessorHandler instances.
	 */
	template<typename... TFragmentHandlers>
	struct TReplicationProcessorHandlerPack
	{
		template<typename TDataHandler>
		using TReplicatedData = typename TRemoveReference<TDataHandler>::Type::FReplicatedDataType;

		static void AddRequirements(FMassEntityQuery& InQuery)
		{
			(TFragmentHandlers::AddRequirements(InQuery), ...);
		}

		void CacheFragmentViews(FMassExecutionContext& InExecContext)
		{
			std::apply([&InExecContext](auto&... FragmentHandler)
				{
					((
						FragmentHandler.CacheFragmentViews(InExecContext)
					), ...);
				}, FragmentHandlers);
		}

		template<typename TReplicatedEntityData>
		void AddEntity(const int32 EntityIndex, TReplicatedEntityData& ReplicatedEntity) const
		{
			std::apply([&ReplicatedEntity, EntityIndex](auto&... FragmentHandler)
				{
					((
						FragmentHandler.AddEntity(EntityIndex, ReplicatedEntity
							.template GetReplicatedData<TReplicatedData<decltype(FragmentHandler)>>())
					), ...);
				}, FragmentHandlers);
		}

		template<typename TClientBubbleHandler>
		requires TIsDerivedFrom<TClientBubbleHandler, IClientBubbleHandlerInterface>::Value
		void ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIndex, TClientBubbleHandler& ClientBubble)
		{
			std::apply([Handle, EntityIndex, &ClientBubble](auto&... FragmentHandler)
				{
					((
						FragmentHandler.ModifyEntity(Handle, EntityIndex, ClientBubble)
					), ...);
				}, FragmentHandlers);
		}

	private:
		std::tuple<TFragmentHandlers...> FragmentHandlers;
	};

	/**
	 * Helper struct to be used as CRTP to supply types with SpawnQuery creation and caching functionality
	 * @param THandlers anything that implements AddRequirementsForSpawnQuery
	 */
	template<typename THandlers>
	struct TCachedSpawnQuery
	{
#if UE_REPLICATION_COMPILE_CLIENT_CODE
		/**
		 * Returns the cached SpawnQuery. If it's not initialized yet EntityManager will be used for the initialization,
		 * along with AddRequirementsForSpawnQuery member function
		 */
		FMassEntityQuery& GetSpawnQuery(const TSharedRef<FMassEntityManager>& EntityManager)
		{
			if (!SpawnQuery.IsInitialized())
			{
				SpawnQuery.Initialize(EntityManager);
				static_cast<THandlers*>(this)->AddRequirementsForSpawnQuery(SpawnQuery);
				SpawnQuery.AddRequirement<FMassNetworkIDFragment>(EMassFragmentAccess::ReadWrite);
			}
			return SpawnQuery;
		}
	private:
		/** Entity query used to configure entities added to the client */
		FMassEntityQuery SpawnQuery;
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE
	};

	/**
	 * Client-side handler pack for multiple replicated data types.
	 * Orchestrates multiple TClientBubbleFragmentHandler instances.
	 */
	template<typename... TFragmentHandlers>
	struct TClientBubbleDataHandlerPack : TCachedSpawnQuery<TClientBubbleDataHandlerPack<TFragmentHandlers...>>
	{
		template<typename TDataHandler>
		using TReplicatedData = typename TRemoveReference<TDataHandler>::Type::FReplicatedDataType;
		using TCachedSpawnQuery<TClientBubbleDataHandlerPack>::GetSpawnQuery;

		static void AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery)
		{
			(TFragmentHandlers::AddRequirementsForSpawnQuery(InQuery), ...);
		}

		void CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext)
		{
			std::apply([&InExecContext](auto&... FragmentHandler)
				{
					((
						FragmentHandler.CacheFragmentViewsForSpawnQuery(InExecContext)
					), ...);
				}, FragmentHandlers);
		}

		void ClearFragmentViewsForSpawnQuery()
		{
			std::apply([](auto&... FragmentHandler)
				{
					((
						FragmentHandler.ClearFragmentViewsForSpawnQuery()
					), ...);
				}, FragmentHandlers);
		}

		template<typename TReplicatedEntityData>
		void SetSpawnedEntityData(const FMassEntityView& EntityView, const TReplicatedEntityData& ReplicatedEntity, const int32 EntityIndex)
		{
			std::apply([&ReplicatedEntity, EntityIndex](auto&... FragmentHandler)
				{
					((
						FragmentHandler.SetSpawnedEntityData(EntityIndex, ReplicatedEntity
							.template GetReplicatedData<TReplicatedData<decltype(FragmentHandler)>>())
					), ...);
				}, FragmentHandlers);
		}

		template<typename TReplicatedEntityData>
		void SetModifiedEntityData(const FMassEntityView& EntityView, const TReplicatedEntityData& ReplicatedEntity)
		{
			((
				TFragmentHandlers::SetModifiedEntityData(EntityView, ReplicatedEntity
					.template GetReplicatedData<TReplicatedData<TFragmentHandlers>>())
			), ...);
		}

	private:
		std::tuple<TFragmentHandlers...> FragmentHandlers;
	};

} // UE::Mass::Replication

//-----------------------------------------------------------------------------
// Convenience Macros
//-----------------------------------------------------------------------------

/**
 * Declares both server and client handler types for replication of fragments.
 *
 * Example:
 *		UE_MASSREPLICATION_DECLARE_HANDLERS(Health, FReplicatedHealthData, FHealthFragment);
 *
 * Declares:
 *		using FMassReplicationProcessorHealthHandler = TReplicationProcessorHandler<FReplicatedHealthData, FHealthFragment>;
 *		using FMassClientBubbleHealthHandler = TClientBubbleFragmentHandler<FReplicatedHealthData, FHealthFragment>;
 *
 * For multiple fragments:
 *		UE_MASSREPLICATION_DECLARE_HANDLERS(Character, FReplicatedCharacterData, FTransformFragment, FVelocityFragment);
 *
 * @see TReplicationTraits for trait specialization details.
 */
#define UE_MASSREPLICATION_DECLARE_HANDLERS(Name, ReplicatedDataType, ...) \
	using FMassReplicationProcessor##Name##Handler = UE::Mass::Replication::TReplicationProcessorHandler<ReplicatedDataType, ##__VA_ARGS__>; \
	using FMassClientBubble##Name##Handler = UE::Mass::Replication::TClientBubbleFragmentHandler<ReplicatedDataType, ##__VA_ARGS__>

/**
 * Declares templated GetReplicatedData accessors in an agent struct.
 * Use UE_MASSREPLICATION_IMPLEMENT_ACCESSOR to provide implementations.
 *
 * Usage:
 *		struct FMyAgent : public FReplicatedAgentBase
 *		{
 *			GENERATED_BODY()
 *			UE_MASSREPLICATION_DECLARE_ACCESSORS()
 *		private:
 *			UPROPERTY(Transient)
 *			FMyReplicatedData MyData;
 *		};
 */
#define UE_MASSREPLICATION_DECLARE_ACCESSORS() \
	template<typename T> const T& GetReplicatedData() const; \
	template<typename T> T& GetReplicatedData()

/**
 * Specializes the templated accessor for a specific data type.
 * Must be placed outside of struct declaration body at namespace scope.
 *
 * Usage:
 *		UE_MASSREPLICATION_IMPLEMENT_ACCESSOR(FMyAgent, FMyReplicatedData, MyData)
 */
#define UE_MASSREPLICATION_IMPLEMENT_ACCESSOR(AgentType, DataType, MemberName) \
	template<> inline const DataType& AgentType::GetReplicatedData<DataType>() const { return MemberName; } \
	template<> inline DataType& AgentType::GetReplicatedData<DataType>() { return MemberName; }
