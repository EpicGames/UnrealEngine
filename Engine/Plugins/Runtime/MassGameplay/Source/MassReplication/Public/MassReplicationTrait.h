// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassReplicationFragments.h"
#include "MassReplicationTrait.generated.h"

#define UE_API MASSREPLICATION_API


UCLASS(MinimalAPI, meta=(DisplayName="Replication"))
class UMassReplicationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

public:

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(EditAnywhere, Category = "Mass|Replication")
	FMassReplicationParameters Params;
};

#undef UE_API
