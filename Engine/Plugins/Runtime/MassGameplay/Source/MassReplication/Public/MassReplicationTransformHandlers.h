// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationTypes.h"
#include "MassEntityView.h"
#include "MassClientBubbleHandler.h"
#include "MassCommonFragments.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "AIHelpers.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassReplicationTransformHandlers.generated.h"

struct FMassEntityQuery;

namespace UE::Mass::Replication
{
	constexpr float PositionReplicateTolerance = 1.f;
	constexpr float YawReplicateTolerance = 0.004363323f;
}; //namespace UE::Mass::Replication

//-----------------------------------------------------------------------------
// FReplicatedAgentPositionYawData
//-----------------------------------------------------------------------------
/**
 * To replicate position and yaw make a member variable of this class in your FReplicatedAgentBase derived class. In the FReplicatedAgentBase derived class you must also provide an accessor function
 * FReplicatedAgentPathData& GetReplicatedPositionYawDataMutable().
 */
USTRUCT()
struct FReplicatedAgentPositionYawData
{
	GENERATED_BODY()
public:
	
	FReplicatedAgentPositionYawData()
		: Position(ForceInitToZero)
	{}

	void SetPosition(const FVector& InPosition) { Position = InPosition; }
	const FVector& GetPosition() const { return Position; }

	void SetYaw(const float InYaw) { Yaw = InYaw; }
	float GetYaw() const { return Yaw; }

private:
	
	UPROPERTY(Transient)
	FVector Position;

	/** Yaw in radians */
	UPROPERTY(Transient)
	float Yaw = 0;
};


//-----------------------------------------------------------------------------
// TMassClientBubbleTransformHandler
//-----------------------------------------------------------------------------
/**
 * To replicate Transforms make a member variable of this class in your TClientBubbleHandlerBase derived class. This class is a friend of TMassClientBubblePathHandler.
 * It is meant to have access to the protected data declared there.
 */
template<typename AgentArrayItem>
class TMassClientBubbleTransformHandler
{
public:
	TMassClientBubbleTransformHandler(TClientBubbleHandlerBase<AgentArrayItem>& InOwnerHandler)
		: OwnerHandler(InOwnerHandler)
	{}

#if UE_REPLICATION_COMPILE_SERVER_CODE
	/** Sets the position and yaw data in the client bubble on the server */
	void SetBubblePositionYawFromTransform(const FMassReplicatedAgentHandle Handle, const FTransform& Transform);

	// Another function  SetBubbleTransform() could be added here if required
#endif // UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	/**
	 * When entities are spawned in Mass by the replication system on the client, a spawn query is used to set the data on the spawned entities.
	 * The following functions are used to configure the query and then set the position and yaw data.
	 */
	static void AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery);
	void CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext);
	void ClearFragmentViewsForSpawnQuery();

	void SetSpawnedEntityData(const int32 EntityIdx, const FReplicatedAgentPositionYawData& ReplicatedPathData) const;

	/** Call this when an Entity that has already been spawned is modified on the client */
	static void SetModifiedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPositionYawData& ReplicatedPathData);

	// We could easily add support replicating FReplicatedAgentTransformData here if required
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

protected:
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	static void SetEntityData(FTransformFragment& TransformFragment, const FReplicatedAgentPositionYawData& ReplicatedPositionYawData);
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

protected:
	TArrayView<FTransformFragment> TransformList;

	TClientBubbleHandlerBase<AgentArrayItem>& OwnerHandler;
};

#if UE_REPLICATION_COMPILE_SERVER_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::SetBubblePositionYawFromTransform(const FMassReplicatedAgentHandle Handle, const FTransform& Transform)
{
	bool bMarkDirty = false;
	
	AgentArrayItem& Item = OwnerHandler.GetAgentItem(Handle);

	checkf(Item.Agent.GetNetID().IsValid(), TEXT("Pos should not be updated on FCrowdFastArrayItem's that have an Invalid ID! First Add the Agent!"));

	// GetReplicatedPositionYawDataMutable() must be defined in your FReplicatedAgentBase derived class
	FReplicatedAgentPositionYawData& ReplicatedPositionYaw = Item.Agent.GetReplicatedPositionYawDataMutable();

	// Only update the Pos and mark the item as dirty if it has changed more than the tolerance
	const FVector Pos = Transform.GetLocation();
	if (!Pos.Equals(ReplicatedPositionYaw.GetPosition(), UE::Mass::Replication::PositionReplicateTolerance))
	{
		ReplicatedPositionYaw.SetPosition(Pos);
		bMarkDirty = true;
	}

	const float Yaw = static_cast<float>(FMath::DegreesToRadians(Transform.GetRotation().Rotator().Yaw));

	// Only update the Yaw and mark the item as dirty if it has changed more than the tolerance
	if (FMath::Abs(FMath::FindDeltaAngleRadians(Yaw, ReplicatedPositionYaw.GetYaw())) > UE::Mass::Replication::YawReplicateTolerance)
	{
		ReplicatedPositionYaw.SetYaw(Yaw);
		bMarkDirty = true;
	}

	if (bMarkDirty)
	{
		OwnerHandler.MarkItemDirty(Item);
	}

}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery)
{
	InQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext)
{
	TransformList = InExecContext.GetMutableFragmentView<FTransformFragment>();
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::ClearFragmentViewsForSpawnQuery()
{
	TransformList = TArrayView<FTransformFragment>();
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::SetSpawnedEntityData(const int32 EntityIdx, const FReplicatedAgentPositionYawData& ReplicatedPositionYawData) const
{
	FTransformFragment& TransformFragment = TransformList[EntityIdx];

	SetEntityData(TransformFragment, ReplicatedPositionYawData);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::SetModifiedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPositionYawData& ReplicatedPositionYawData)
{
	FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();

	SetEntityData(TransformFragment, ReplicatedPositionYawData);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubbleTransformHandler<AgentArrayItem>::SetEntityData(FTransformFragment& TransformFragment, const FReplicatedAgentPositionYawData& ReplicatedPositionYawData)
{
	TransformFragment.GetMutableTransform().SetLocation(ReplicatedPositionYawData.GetPosition());
	TransformFragment.GetMutableTransform().SetRotation(FQuat(FVector::UpVector, ReplicatedPositionYawData.GetYaw()));
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

//-----------------------------------------------------------------------------
// FMassReplicationProcessorTransformHandlerBase
//----------------------------------------------------------------------------- 
class FMassReplicationProcessorTransformHandlerBase
{
public:
	static MASSREPLICATION_API void AddRequirements(FMassEntityQuery& InQuery);
	MASSREPLICATION_API void CacheFragmentViews(FMassExecutionContext& ExecContext);

protected:
	TArrayView<FTransformFragment> TransformList;
};

//-----------------------------------------------------------------------------
// FMassReplicationProcessorPositionYawHandler
//----------------------------------------------------------------------------- 
/**
 * Used to replicate position and yaw by your UMassReplicationProcessorBase derived class. This class should only get used on the server.
 * @todo add #if UE_REPLICATION_COMPILE_SERVER_CODE
 */
class FMassReplicationProcessorPositionYawHandler : public FMassReplicationProcessorTransformHandlerBase
{
public:
	MASSREPLICATION_API void AddEntity(const int32 EntityIdx, FReplicatedAgentPositionYawData& InOUtReplicatedPathData) const;

	template<typename AgentArrayItem>
	void ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIdx, TMassClientBubbleTransformHandler<AgentArrayItem>& BubblePathHandler);
};

template<typename AgentArrayItem>
void FMassReplicationProcessorPositionYawHandler::ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIdx, TMassClientBubbleTransformHandler<AgentArrayItem>& BubbleTransformHandler)
{
	const FTransformFragment& TransformFragment = TransformList[EntityIdx];

	BubbleTransformHandler.SetBubblePositionYawFromTransform(Handle, TransformFragment.GetTransform());
}
