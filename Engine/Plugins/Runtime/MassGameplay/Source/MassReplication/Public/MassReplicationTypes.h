// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "IndexedHandle.h"
#include "MassEntityTemplate.h"
#include "MassLODLogic.h"
#include "HierarchicalHashGrid2D.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLODSubsystem.h"
#include "MassObserverProcessor.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassReplicationTypes.generated.h"


/** Debug option to have bubbles on all player controllers in a stand alone game. This is especially useful for testing and profiling
 *  these systems. This will also disable the ClientsViewers functionality. If UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE is set 
 *  UE_REPLICATION_COMPILE_SERVER_CODE will be defined as 1 so server only code is compiled in standalone builds. 
 */ 
#ifndef UE_DEBUG_REPLICATION_BUBBLES_STANDALONE
#define UE_DEBUG_REPLICATION_BUBBLES_STANDALONE 0
#endif

#if UE_DEBUG_REPLICATION_BUBBLES_STANDALONE && !UE_BUILD_SHIPPING && (UE_GAME || UE_EDITOR)
#define UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE 1
#else
#define UE_ALLOW_DEBUG_REPLICATION_BUBBLES_STANDALONE 0
#endif

// WITH_SERVER_CODE is all builds except client only.
#if WITH_SERVER_CODE
#define UE_REPLICATION_COMPILE_SERVER_CODE 1
#else
#define UE_REPLICATION_COMPILE_SERVER_CODE 0
#endif

// If not a dedicated server build then we may need client code
#if !UE_SERVER
#define UE_REPLICATION_COMPILE_CLIENT_CODE 1
#else
#define UE_REPLICATION_COMPILE_CLIENT_CODE 0
#endif

#ifndef UE_DEBUG_SLOW_REPLICATION
#define UE_DEBUG_SLOW_REPLICATION 0
#endif

#if UE_DEBUG_SLOW_REPLICATION && !UE_BUILD_SHIPPING
#define UE_ALLOW_DEBUG_SLOW_REPLICATION 1
#else
#define UE_ALLOW_DEBUG_SLOW_REPLICATION 0
#endif

#ifndef UE_DEBUG_REPLICATION
#define UE_DEBUG_REPLICATION 0
#endif

#if (UE_DEBUG_REPLICATION && !UE_BUILD_SHIPPING) || UE_ALLOW_DEBUG_SLOW_REPLICATION
#define UE_ALLOW_DEBUG_REPLICATION 1
#else
#define UE_ALLOW_DEBUG_REPLICATION 0
#endif

class UMassReplicationSubsystem;

DECLARE_LOG_CATEGORY_EXTERN(LogMassReplication, Log, All);

/** Handle of each Agent per Client (bubble), these handles are not consistent between different client bubbles.
 *  Using the compact handle here will make the memory footprint smaller and help the ECS system be more cache friendly.
 *	TODO: in the long run we can probably use a non serial number handle here, I'd like to stick with the serial number for now though
 */
USTRUCT()
struct FMassReplicatedAgentHandle : public FCompactIndexedHandleBase
{
	GENERATED_BODY()

	FMassReplicatedAgentHandle() = default;

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	FMassReplicatedAgentHandle(int32 InIndex, uint32 InSerialNumber) : FCompactIndexedHandleBase(InIndex, InSerialNumber)
	{
	};
};

typedef FCompactIndexedHandleManager<FMassReplicatedAgentHandle> FMassReplicatedAgentHandleManager;

//Handle of each Client on the server. A Client is defined as a player controller with a non nullptr parent NetConnection
USTRUCT()
struct FMassClientHandle : public FIndexedHandleBase
{
	GENERATED_BODY()

	FMassClientHandle() = default;

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	FMassClientHandle(int32 InIndex, uint32 InSerialNumber) : FIndexedHandleBase(InIndex, InSerialNumber)
	{
	};
};

typedef FIndexedHandleManager<FMassClientHandle> FMassClientHandleManager;

USTRUCT()
struct FMassBubbleInfoClassHandle : public FSimpleIndexedHandleBase
{
	GENERATED_BODY()

	FMassBubbleInfoClassHandle() = default;

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	FMassBubbleInfoClassHandle(int32 InIndex) : FSimpleIndexedHandleBase(InIndex)
	{
	};
};

USTRUCT()
struct FReplicatedAgentBase
{
	GENERATED_BODY()

	FReplicatedAgentBase() = default;

	FReplicatedAgentBase(FMassNetworkID InNetID, FMassEntityTemplateID InTemplateID)
		: NetID(InNetID)
		, TemplateID(InTemplateID)
	{}

	void SetNetID(FMassNetworkID InNetID) { NetID = InNetID; }
	FMassNetworkID GetNetID() const { return NetID; }

	void SetTemplateID(FMassEntityTemplateID InTemplateID) { TemplateID = InTemplateID; }
	FMassEntityTemplateID GetTemplateID() const { return TemplateID; }

private:
	UPROPERTY()
	FMassNetworkID NetID;

	UPROPERTY()
	FMassEntityTemplateID TemplateID;
};

struct FReplicationLODLogic : public FLODDefaultLogic
{
};

struct FMassReplicatedAgentData
{
	void Invalidate()
	{
		Handle.Invalidate();
		LastUpdateTime = 0.;
		LOD = EMassLOD::Off;
		bPendingDestruction = false;
	}

	FMassReplicatedAgentHandle Handle;
	double LastUpdateTime = 0.;
	EMassLOD::Type LOD = EMassLOD::Off;
	bool bPendingDestruction = false;
};

#if UE_REPLICATION_COMPILE_CLIENT_CODE
/** Data that can be accessed from a FMassNetworkID on a client */
struct FMassReplicationEntityInfo
{
	FMassReplicationEntityInfo() = default;
	FMassReplicationEntityInfo(FMassEntityHandle InEntity, int32 InReplicationID)
		: Entity(InEntity)
		, ReplicationID(InReplicationID)
	{}

	/** If this is not IsSet() then the entity has been removed from the client simulation */
	FMassEntityHandle Entity; 

	/** This is stored between removes and adds, however this item in the UMassReplicationSubsystem::EntityInfoMap will eventually get cleaned up if Entity.IsSet() == false for a fairly substantial length of time. */
	int32 ReplicationID = INDEX_NONE;
};
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

typedef THierarchicalHashGrid2D<2, 4, FMassEntityHandle> FReplicationHashGrid2D;	// 2 levels of hierarchy, 4 ratio between levels
