// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassRepresentation : ModuleRules
	{
		public MassRepresentation(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassActors",
					"MassSpawner",
					"MassReplication",
					"MassLOD",
					"MassSimulation",
					"RenderCore",
					"RHI",
					"MassMovement",
					"MassSignals",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
