// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDistanceLODProcessor.h"
#include "MassRepresentationDebug.h"
#include "MassExecutionContext.h"
#include "MassCommonFragments.h"
#include "MassDebugger.h"
#include "MassLODFragments.h"
#include "MassLODTypes.h"
#include "MassEntityQuery.h"
#include "MassRepresentationFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDistanceLODProcessor)

UMassDistanceLODProcessor::UMassDistanceLODProcessor()
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = (int32)(EProcessorExecutionFlags::AllNetModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LOD;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);
}

void UMassDistanceLODProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseQuery(EntityManager);
	BaseQuery.AddTagRequirement<FMassDistanceLODProcessorTag>(EMassFragmentPresence::All);
	BaseQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadWrite);
	BaseQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddConstSharedRequirement<FMassDistanceLODParameters>();
	BaseQuery.AddSharedRequirement<FMassDistanceLODSharedFragment>(EMassFragmentAccess::ReadWrite);

	CloseEntityQuery = BaseQuery;
	CloseEntityQuery.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	CloseEntityQuery.RegisterWithProcessor(*this);

	FarEntityQuery = BaseQuery;
	FarEntityQuery.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	FarEntityQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadOnly);
	FarEntityQuery.SetChunkFilter(&FMassVisualizationChunkFragment::ShouldUpdateVisualizationForChunk);
	FarEntityQuery.RegisterWithProcessor(*this);

	DebugEntityQuery = BaseQuery;
	DebugEntityQuery.RegisterWithProcessor(*this);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassDistanceLODProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (bForceOFFLOD)
	{
		CloseEntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FMassDistanceLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassDistanceLODSharedFragment>();
			TArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			LODSharedFragment.LODCalculator.ForceOffLOD(Context, RepresentationLODList);
		});
		return;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareExecution)
		const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
		const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();
		EntityManager.ForEachSharedFragment<FMassDistanceLODSharedFragment>([this, &Viewers](FMassDistanceLODSharedFragment& LODSharedFragment)
		{
			if (FilterTag == LODSharedFragment.FilterTag)
			{
				LODSharedFragment.LODCalculator.PrepareExecution(Viewers);
			}
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CalculateLOD)

		auto CalculateLOD = [this](FMassExecutionContext& Context)
		{
			FMassDistanceLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassDistanceLODSharedFragment>();
			TArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			TConstArrayView<FMassViewerInfoFragment> ViewerInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			LODSharedFragment.LODCalculator.CalculateLOD(Context, ViewerInfoList, RepresentationLODList);
		};
		CloseEntityQuery.ForEachEntityChunk(Context, CalculateLOD);
		FarEntityQuery.ForEachEntityChunk(Context, CalculateLOD);
	}

#if WITH_MASSGAMEPLAY_DEBUG
	// Optional debug display
	if (UE::Mass::Representation::Debug::DebugRepresentationLOD == 1 || UE::Mass::Representation::Debug::DebugRepresentationLOD >= 3)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayLOD)
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			DebugEntityQuery.ForEachEntityChunk(Context, [DebugDrawingWorld](FMassExecutionContext& LocalContext)
			{
				FMassDistanceLODSharedFragment& LODSharedFragment = LocalContext.GetMutableSharedFragment<FMassDistanceLODSharedFragment>();
				TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = LocalContext.GetFragmentView<FMassRepresentationLODFragment>();
				TConstArrayView<FTransformFragment> TransformList = LocalContext.GetFragmentView<FTransformFragment>();
				LODSharedFragment.LODCalculator.DebugDisplaySignificantLOD(LocalContext, RepresentationLODList, TransformList, DebugDrawingWorld, UE::Mass::Representation::Debug::DebugRepresentationLODMaxSignificance);
		   });
		}
	}
	// Optional vislog
	if (UE::Mass::Representation::Debug::DebugRepresentationLOD >= 2)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VisLogLOD)
		DebugEntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FMassDistanceLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassDistanceLODSharedFragment>();
			TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
			TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			LODSharedFragment.LODCalculator.VisLogSignificantLOD(Context, RepresentationLODList, TransformList, this, UE::Mass::Representation::Debug::DebugRepresentationLODMaxSignificance);
		});
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}
