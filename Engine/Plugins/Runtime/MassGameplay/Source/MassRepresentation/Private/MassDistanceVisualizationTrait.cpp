// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDistanceVisualizationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassRepresentationSubsystem.h"
#include "MassCommonFragments.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationActorManagement.h"
#include "Engine/World.h"
#include "MassLODFragments.h"
#include "MassActorSubsystem.h"
#include "MassEntityUtils.h"
#include "MassDistanceLODProcessor.h"
#include "MassRepresentationProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDistanceVisualizationTrait)


UMassDistanceVisualizationTrait::UMassDistanceVisualizationTrait()
{
	RepresentationSubsystemClass = UMassRepresentationSubsystem::StaticClass();

	Params.RepresentationActorManagementClass = UMassRepresentationActorManagement::StaticClass();
	Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::HighResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::LowResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Low] = EMassRepresentationType::StaticMeshInstance;
	Params.LODRepresentation[EMassLOD::Off] = EMassRepresentationType::None;

	LODParams.LODDistance[EMassLOD::High] = 0.f;
	LODParams.LODDistance[EMassLOD::Medium] = 1000.f;
	LODParams.LODDistance[EMassLOD::Low] = 2500.f;
	LODParams.LODDistance[EMassLOD::Off] = 10000.f;

	LODParams.BufferHysteresisOnDistancePercentage = 10.0f;

	bAllowServerSideVisualization = false;
}

void UMassDistanceVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	// This should not be ran on NM_Server network mode
	if (World.IsNetMode(NM_DedicatedServer) && !bAllowServerSideVisualization
		&& !BuildContext.IsInspectingData())
	{
		return;
	}

	BuildContext.RequireFragment<FMassViewerInfoFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassActorFragment>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	UMassRepresentationSubsystem* RepresentationSubsystem = Cast<UMassRepresentationSubsystem>(World.GetSubsystemBase(RepresentationSubsystemClass));
	if (RepresentationSubsystem == nullptr && !BuildContext.IsInspectingData())
	{
		UE_LOGF(LogMassRepresentation, Error, "Expecting a valid class for the representation subsystem");
		RepresentationSubsystem = UWorld::GetSubsystem<UMassRepresentationSubsystem>(&World);
		check(RepresentationSubsystem);
	}

	FMassRepresentationSubsystemSharedFragment SubsystemSharedFragment;
	SubsystemSharedFragment.RepresentationSubsystem = RepresentationSubsystem;
	FSharedStruct SubsystemFragment = EntityManager.GetOrCreateSharedFragment(SubsystemSharedFragment);
	BuildContext.AddSharedFragment(SubsystemFragment);

	if (!Params.RepresentationActorManagementClass && !BuildContext.IsInspectingData())
	{
		UE_LOGF(LogMassRepresentation, Error, "Expecting a valid class for the representation actor management");
	}
	FConstSharedStruct ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(Params);
	ParamsFragment.Get<const FMassRepresentationParameters>().ComputeCachedValues();
	BuildContext.AddConstSharedFragment(ParamsFragment);

	FMassRepresentationFragment& RepresentationFragment = BuildContext.AddFragment_GetRef<FMassRepresentationFragment>();
	if (LIKELY(!BuildContext.IsInspectingData()))
	{
		if (bRegisterStaticMeshDesc && !BuildContext.IsInspectingData())
		{
			RepresentationFragment.StaticMeshDescHandle = RepresentationSubsystem->FindOrAddStaticMeshDesc(StaticMeshInstanceDesc);
		}
		RepresentationFragment.HighResTemplateActorIndex = HighResTemplateActor.Get() ? RepresentationSubsystem->FindOrAddTemplateActor(HighResTemplateActor.Get()) : INDEX_NONE;
		RepresentationFragment.LowResTemplateActorIndex = LowResTemplateActor.Get() ? RepresentationSubsystem->FindOrAddTemplateActor(LowResTemplateActor.Get()) : INDEX_NONE;
	}

	FConstSharedStruct LODParamsFragment = EntityManager.GetOrCreateConstSharedFragment(LODParams);
	BuildContext.AddConstSharedFragment(LODParamsFragment);

	FSharedStruct LODSharedFragment = EntityManager.GetOrCreateSharedFragment<FMassDistanceLODSharedFragment>(FConstStructView::Make(LODParams), LODParams);
	BuildContext.AddSharedFragment(LODSharedFragment);

	BuildContext.AddFragment<FMassRepresentationLODFragment>();
	BuildContext.AddTag<FMassVisibilityCulledByDistanceTag>();
	BuildContext.AddChunkFragment<FMassVisualizationChunkFragment>();

	BuildContext.AddTag<FMassDistanceLODProcessorTag>();
	BuildContext.AddTag<FMassVisualizationProcessorTag>();
}

void UMassDistanceVisualizationTrait::DestroyTemplate(const UWorld& World) const
{
	if (UMassRepresentationSubsystem* RepresentationSubsystem = Cast<UMassRepresentationSubsystem>(World.GetSubsystemBase(RepresentationSubsystemClass)))
	{
		RepresentationSubsystem->ReleaseTemplate(HighResTemplateActor);
		RepresentationSubsystem->ReleaseTemplate(LowResTemplateActor);
	}
}
