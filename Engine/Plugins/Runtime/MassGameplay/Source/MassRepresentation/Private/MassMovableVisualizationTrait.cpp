// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMovableVisualizationTrait.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassMovableVisualizationTrait)


void UMassMovableVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	if (!bAllowServerSideVisualization && World.IsNetMode(NM_DedicatedServer)
		&& !BuildContext.IsInspectingData())
	{
		return;
	}

	for (FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : StaticMeshInstanceDesc.Meshes)
	{
		MeshDesc.Mobility = EComponentMobility::Movable;
	}

	Super::BuildTemplate(BuildContext, World);
}
