// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationActorManagement.h"
#include "MassRepresentationSubsystem.h"
#include "MassRepresentationFragments.h"
#include "MassCommandBuffer.h"
#include "MassEntityView.h"
#include "MassActorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationActorManagement)

float UMassRepresentationActorManagement::GetSpawnPriority(const FMassRepresentationLODFragment& Representation) const
{
	// Bump up the spawning priority on the visible entities
	return Representation.LODSignificance - (Representation.Visibility == EMassVisibility::CanBeSeen ? 1.0f : 0.0f);
}

AActor* UMassRepresentationActorManagement::GetOrSpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem, FMassEntityManager& EntityManager
	, const FMassEntityHandle MassAgent, const FTransform& Transform, const int16 TemplateActorIndex
	, FMassActorSpawnRequestHandle& InOutSpawnRequestHandle, const float Priority) const
{
	TSharedRef<FMassEntityManager> SharedEntityManager = EntityManager.AsShared();

	return RepresentationSubsystem.GetOrRequestSpawnActorFromTemplate(MassAgent, Transform, TemplateActorIndex, InOutSpawnRequestHandle, Priority,
		FMassActorPreSpawnDelegate::CreateUObject(this, &UMassRepresentationActorManagement::OnPreActorSpawn, SharedEntityManager),
		FMassActorPostSpawnDelegate::CreateUObject(this, &UMassRepresentationActorManagement::OnPostActorSpawn, SharedEntityManager));
}


void UMassRepresentationActorManagement::SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx, FMassCommandBuffer& CommandBuffer) const
{
	const bool bEnabled = EnabledType != EMassActorEnabledType::Disabled;
	if (Actor.IsActorTickEnabled() != bEnabled)
	{
		Actor.SetActorTickEnabled(bEnabled);
	}
	if (Actor.GetActorEnableCollision() != bEnabled)
	{
		// Deferring this as there is a callback internally that could end up doing things outside of the game thread and will fire checks(Chaos mostly)
		CommandBuffer.PushCommand<FMassDeferredSetCommand>([&Actor, bEnabled](FMassEntityManager&)
		{
			Actor.SetActorEnableCollision(bEnabled);
		});
	}
}

void UMassRepresentationActorManagement::TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const
{
	if (!Actor.GetTransform().Equals(Transform))
	{
		CommandBuffer.PushCommand<FMassDeferredSetCommand>([&Actor, Transform](FMassEntityManager&)
		{
			Actor.SetActorTransform(Transform, /*bSweep*/false, /*OutSweepHitResult*/nullptr, ETeleportType::TeleportPhysics);
		});
	}
}

bool UMassRepresentationActorManagement::IsActorReadyForRepresentation(const AActor& Actor) const
{
	return true;
}

void UMassRepresentationActorManagement::OnPreActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const
{
	const FMassActorSpawnRequest& MassActorSpawnRequest = SpawnRequest.Get<const FMassActorSpawnRequest>();
	const FMassEntityView EntityView(*EntityManager, MassActorSpawnRequest.MassAgent);
	FMassActorFragment& ActorInfo = EntityView.GetFragmentData<FMassActorFragment>();
	FMassRepresentationFragment& Representation = EntityView.GetFragmentData<FMassRepresentationFragment>();
	UMassRepresentationSubsystem* RepresentationSubsystem = EntityView.GetSharedFragmentData<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
	check(RepresentationSubsystem);

	// Release any existing actor
	if (AActor* Actor = ActorInfo.GetMutable())
	{
		checkf(ActorInfo.IsOwnedByMass(), TEXT("If we reach here, we expect the actor to be owned by mass, otherwise we should not be spawning a new one one top of this one."));

		// WARNING!
		// Need to reset before ReleaseTemplateActor as this action might move the entity to a new archetype and
		// so the Fragment passed in parameters would not be valid anymore.
		ActorInfo.ResetAndUpdateHandleMap();

		if (!RepresentationSubsystem->ReleaseTemplateActor(MassActorSpawnRequest.MassAgent, Representation.HighResTemplateActorIndex, Actor, /*bImmediate*/ true))
		{
			if (!RepresentationSubsystem->ReleaseTemplateActor(MassActorSpawnRequest.MassAgent, Representation.LowResTemplateActorIndex, Actor, /*bImmediate*/ true))
			{
				checkf(false, TEXT("Expecting to be able to release spawned actor either the high res or low res one"));
			}
		}
	}
}

EMassActorSpawnRequestAction UMassRepresentationActorManagement::OnPostActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle
	, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const
{
	const FMassActorSpawnRequest& MassActorSpawnRequest = SpawnRequest.Get<const FMassActorSpawnRequest>();
	
	if (MassActorSpawnRequest.SpawnedActor)
	{
		// Might be already done if the actor has a MassAgentComponent via the callback OnMassAgentComponentEntityAssociated on the MassRepresentationSubsystem
		FMassActorFragment& ActorInfo = EntityManager->GetFragmentDataChecked<FMassActorFragment>(MassActorSpawnRequest.MassAgent);
		if (ActorInfo.IsValid())
		{
			// If already set, make sure it is pointing to the same actor.
			checkf(ActorInfo.Get() == MassActorSpawnRequest.SpawnedActor, TEXT("Expecting the pointer to the spawned actor in the actor fragment"));
		}
		else
		{
			ActorInfo.SetAndUpdateHandleMap(MassActorSpawnRequest.MassAgent, MassActorSpawnRequest.SpawnedActor, true/*bIsOwnedByMass*/);
		}

		// by default we keep the spawn request since there's code in other places that will consume this request by 
		// calling RemoveActorSpawnRequest (like UMassRepresentationSubsystem::GetOrRequestSpawnActorFromTemplate)
		return EMassActorSpawnRequestAction::Keep;
	}

	return EMassActorSpawnRequestAction::Remove;
}

void UMassRepresentationActorManagement::ReleaseAnyActorOrCancelAnySpawning(FMassEntityManager& EntityManager, const FMassEntityHandle MassAgent, const bool bImmediate /*= false*/)
{
	FMassEntityView EntityView(EntityManager, MassAgent);
	FMassActorFragment& ActorInfo = EntityView.GetFragmentData<FMassActorFragment>();
	FMassRepresentationFragment& Representation = EntityView.GetFragmentData<FMassRepresentationFragment>();
	UMassRepresentationSubsystem* RepresentationSubsystem = EntityView.GetSharedFragmentData<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
	check(RepresentationSubsystem);
	ReleaseAnyActorOrCancelAnySpawning(*RepresentationSubsystem, MassAgent, ActorInfo, Representation, /*CachedActorSubsystem=*/nullptr, bImmediate);
}

void UMassRepresentationActorManagement::ReleaseAnyActorOrCancelAnySpawning(UMassRepresentationSubsystem& RepresentationSubsystem
	, const FMassEntityHandle MassAgent, FMassActorFragment& ActorInfo, FMassRepresentationFragment& Representation
	, UMassActorSubsystem* CachedActorSubsystem, bool bImmediate /*= false*/)
{
	// This method can only release owned by mass actors
	AActor* Actor = ActorInfo.GetOwnedByMassMutable();
	if (Actor)
	{
		// WARNING!
		// Need to reset before ReleaseTemplateActorOrCancelSpawning as this action might move the entity to a new archetype and
		// so the Fragment passed in parameters would not be valid anymore.
		ActorInfo.ResetAndUpdateHandleMap(CachedActorSubsystem);
	}
	// Try releasing both as we can have a low res actor and a high res spawning request
	if (Representation.HighResTemplateActorIndex != INDEX_NONE)
	{
		RepresentationSubsystem.ReleaseTemplateActorOrCancelSpawning(MassAgent, Representation.HighResTemplateActorIndex, Actor, Representation.ActorSpawnRequestHandle, bImmediate);
	}	
	if (Representation.LowResTemplateActorIndex != Representation.HighResTemplateActorIndex && Representation.LowResTemplateActorIndex != INDEX_NONE)
	{
		RepresentationSubsystem.ReleaseTemplateActorOrCancelSpawning(MassAgent, Representation.LowResTemplateActorIndex, Actor, Representation.ActorSpawnRequestHandle, bImmediate);
	}
	check(!Representation.ActorSpawnRequestHandle.IsValid());
}
