// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationAnimationProcessor.h"

#include "Engine/World.h"
#include "MassCommonFragments.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassLODFragments.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationSubsystem.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceTransformProviderData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationAnimationProcessor)

UMassConsumeInstancedSkinnedMeshAnimationProcessor::UMassConsumeInstancedSkinnedMeshAnimationProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Representation;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::VisualizationProcessing);
	bRequiresGameThreadExecution = true;
}

void UMassConsumeInstancedSkinnedMeshAnimationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationAnimationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);

	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassConsumeInstancedSkinnedMeshAnimationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		check(RepresentationSubsystem);

		FMassInstancedSkinnedMeshInfoArrayView MeshInfos = RepresentationSubsystem->GetMutableInstancedSkinnedMeshInfos();

		const TConstArrayView<FMassRepresentationFragment> RepresentationList = Context.GetFragmentView<FMassRepresentationFragment>();
		const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
		const TConstArrayView<FMassRepresentationAnimationFragment> AnimationList = Context.GetFragmentView<FMassRepresentationAnimationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
			const FMassRepresentationAnimationFragment& Animation = AnimationList[EntityIt];
			const FMassRepresentationFragment& Representation = RepresentationList[EntityIt];

			if (Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance)
			{
				const int32 MeshInfoIndex = Representation.SkinnedMeshDescHandle.ToIndex();
				if (ensureMsgf(MeshInfos.IsValidIndex(MeshInfoIndex), TEXT("Invalid handle index %u for MeshInfos View"), MeshInfoIndex))
				{
					UpdateMeshAnimation(Context.GetEntity(EntityIt), MeshInfos[MeshInfoIndex], Animation.AnimData, RepresentationLOD.LODSignificance, Representation.PrevLODSignificance);
				}
			}
		}
	});
}

void UMassConsumeInstancedSkinnedMeshAnimationProcessor::UpdateMeshAnimation(FMassEntityHandle EntityHandle, FMassInstancedSkinnedMeshInfo& MeshInfo, FAnimSequenceTrackAutoPlayData AnimationData, const float LODSignificance, const float PrevLODSignificance/* = -1.0f*/)
{
	MeshInfo.AddBatchedAnimationData(EntityHandle, AnimationData, LODSignificance, PrevLODSignificance);
}