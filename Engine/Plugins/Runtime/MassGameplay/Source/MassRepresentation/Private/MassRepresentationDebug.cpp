// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationDebug.h"

#if WITH_MASSGAMEPLAY_DEBUG

#include "DrawDebugHelpers.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassRepresentationFragments.h"
#include "Math/Color.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::Mass::Representation::Debug
{
	FColor RepresentationColors[] =
	{
		FColor::Purple, // HighResSpawnedActor,
		FColor::Turquoise, // LowResSpawnedActor
		FColor::Orange, // StaticMeshInstance
		FColor::White, // None
	};

	int32 DebugRepresentation = 0;
	FAutoConsoleVariableRef CVarDebugRepresentation(TEXT("mass.debug.Representation"), DebugRepresentation, TEXT("Debug current representation 0 = Off, 1 = Debug Draw, 2 = VisLog, 3 = Both"), ECVF_Cheat);

	float DebugRepresentationMaxSignificance = float(EMassLOD::Max);
	FAutoConsoleVariableRef CVarDebugRepresentationMaxSignificance(TEXT("mass.debug.Representation.MaxSignificance"), DebugRepresentationMaxSignificance, TEXT("Max LODSignificance for entities to draw / vislog with mass.debug.Representation enabled"), ECVF_Cheat);

	int32 DebugRepresentationLOD = 0;
	FAutoConsoleVariableRef CVarDebugRepresentationLOD(TEXT("mass.debug.RepresentationLOD"), DebugRepresentationLOD, TEXT("Debug representation LOD 0 = Off, 1 = Debug Draw, 2 = VisLog, 3 = Both"), ECVF_Cheat);

	float DebugRepresentationLODMaxSignificance = float(EMassLOD::Max);
	FAutoConsoleVariableRef CVarDebugRepresentationLODMaxSignificance(TEXT("mass.debug.RepresentationLOD.MaxSignificance"), DebugRepresentationLODMaxSignificance, TEXT("Max LODSignificance for entities to draw / vislog with mass.debug.RepresentationLOD enabled"), ECVF_Cheat);

	void DebugDisplayRepresentation(FMassExecutionContext& Context, TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList, TConstArrayView<FMassRepresentationFragment> RepresentationList, TConstArrayView<FTransformFragment> LocationList, UWorld* World)
	{
#if UE_ENABLE_DEBUG_DRAWING
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
			if (RepresentationLOD.LODSignificance <= UE::Mass::Representation::Debug::DebugRepresentationMaxSignificance)
			{
				const FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
				const FTransformFragment& EntityLocation = LocationList[EntityIt];
				int32 CurrentRepresentationIdx = (int32)Representation.CurrentRepresentation;
				DrawDebugSolidBox(World, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 150.0f), FVector(25.0f), UE::Mass::Representation::Debug::RepresentationColors[CurrentRepresentationIdx]);
			}
		}
#endif // UE_ENABLE_DEBUG_DRAWING
	}

	void VisLogRepresentation(FMassExecutionContext& Context, TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList, TConstArrayView<FMassRepresentationFragment> RepresentationList, TConstArrayView<FTransformFragment> LocationList, UObject* LogOwner)
	{
#if ENABLE_VISUAL_LOG
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
			if (RepresentationLOD.LODSignificance <= UE::Mass::Representation::Debug::DebugRepresentationMaxSignificance)
			{
				const FTransformFragment& EntityLocation = LocationList[EntityIt];
				const FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
				int32 CurrentRepresentationIdx = (int32)Representation.CurrentRepresentation;
				int32 PrevRepresentationIdx = (int32)Representation.PrevRepresentation;
				// Add 20cm +Z offset to draw above mass.debug.RepresentationLOD so they can be viewed together
				UE_CVLOG_LOCATION(Representation.CurrentRepresentation == Representation.PrevRepresentation, LogOwner, LogMassRepresentation, Verbose, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 20.0f), 20, UE::Mass::Representation::Debug::RepresentationColors[CurrentRepresentationIdx], TEXT("%s %d"), *Context.GetEntity(EntityIt).DebugGetDescription(), CurrentRepresentationIdx);
				UE_CVLOG_LOCATION(Representation.CurrentRepresentation != Representation.PrevRepresentation, LogOwner, LogMassRepresentation, Verbose, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 20.0f), 20, UE::Mass::Representation::Debug::RepresentationColors[CurrentRepresentationIdx], TEXT("%s %d -> %d"), *Context.GetEntity(EntityIt).DebugGetDescription(), PrevRepresentationIdx, CurrentRepresentationIdx);
			}
		}
#endif // ENABLE_VISUAL_LOG
	}
}

#endif // WITH_MASSGAMEPLAY_DEBUG
