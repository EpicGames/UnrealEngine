// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationFragments)

FMassVisualizationLODSharedFragment::FMassVisualizationLODSharedFragment(const FMassVisualizationLODParameters& LODParams)
{
	LODCalculator.Initialize(LODParams.BaseLODDistance, LODParams.BufferHysteresisOnDistancePercentage / 100.f, LODParams.LODMaxCount
		, /*InLODMaxCountPerViewer=*/nullptr, LODParams.DistanceToFrustum, LODParams.DistanceToFrustumHysteresis, LODParams.VisibleLODDistance);
	FilterTag = LODParams.FilterTag;
}

FMassDistanceLODSharedFragment::FMassDistanceLODSharedFragment(const FMassDistanceLODParameters& LODParams)
{
	LODCalculator.Initialize(LODParams.LODDistance, LODParams.BufferHysteresisOnDistancePercentage / 100.f, /*InLODMaxCount=*/nullptr);
	FilterTag = LODParams.FilterTag;
}
