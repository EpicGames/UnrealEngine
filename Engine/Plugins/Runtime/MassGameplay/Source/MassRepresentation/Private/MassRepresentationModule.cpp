// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IMassRepresentationModule.h"
#include "UObject/CoreRedirects.h"

class FMassRepresentationModule : public IMassRepresentationModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassRepresentationModule, MassRepresentation)



void FMassRepresentationModule::StartupModule()
{
}


void FMassRepresentationModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}



