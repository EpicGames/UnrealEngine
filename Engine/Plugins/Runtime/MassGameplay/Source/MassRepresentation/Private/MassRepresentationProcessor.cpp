// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationProcessor.h"
#include "MassRepresentationDebug.h"
#include "MassRepresentationSubsystem.h"
#include "MassRepresentationUtils.h"
#include "MassActorSubsystem.h"
#include "MassCommonFragments.h"
#include "MassCommandBuffer.h"
#include "MassEntityManager.h"
#include "MassEntityView.h"
#include "Engine/World.h"
#include "MassRepresentationActorManagement.h"
#include "MassCommonUtils.h"
#include "MassDebugger.h"
#include "MassEntityUtils.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationProcessor)


DECLARE_CYCLE_STAT(TEXT("Mass Visualization Execute"), STAT_Mass_VisProcessor_Execute, STATGROUP_Mass);

namespace UE::Mass::Representation
{
	int32 bAllowKeepActorExtraFrame = 1;
	FAutoConsoleVariableRef CVarAllowKeepActorExtraFrame(TEXT("ai.massrepresentation.AllowKeepActorExtraFrame"), bAllowKeepActorExtraFrame, TEXT("Allow the mass representation to keep actor an extra frame when switching to ISM"), ECVF_Default);

	int32 bAllowWaitForActorVisualReadiness = 1;
	FAutoConsoleVariableRef CVarAllowWaitForActorVisualReadiness(
		TEXT("ai.massrepresentation.AllowWaitForActorVisualReadiness"),
		bAllowWaitForActorVisualReadiness,
		TEXT("Cvar gate for the FMassRepresentationParameters::bWaitForActorVisualReadiness opt-in. When non-zero ")
		TEXT("the per-archetype param controls whether the ISM/SkinnedMesh stays visible while the spawned actor's ")
		TEXT("IsActorReadyForRepresentation hook reports not-ready. Set to 0 to force-disable globally."),
		ECVF_Default);

	int32 MaxActorVisualReadinessWaitFrames = FMassRepresentationFragment::MaxActorVisualReadinessWaitFrames;
	FAutoConsoleVariableRef CVarMaxActorVisualReadinessWaitFrames(
		TEXT("ai.massrepresentation.MaxActorVisualReadinessWaitFrames"),
		MaxActorVisualReadinessWaitFrames,
		TEXT("Safety cap on how many frames Mass will keep an entity on its mesh representation while waiting for ")
		TEXT("IsActorReadyForRepresentation to return true. After the cap the actor is force-enabled and a warning ")
		TEXT("is logged. Set to 0 to disable the cap. Max value 255."),
		ECVF_Default);
}

//----------------------------------------------------------------------//
// UMassRepresentationProcessor(); 
//----------------------------------------------------------------------//
UMassRepresentationProcessor::UMassRepresentationProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = false;

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Representation;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
}

void UMassRepresentationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassActorSubsystem>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassRepresentationProcessor::UpdateRepresentation(FMassExecutionContext& Context, const FMassRepresentationUpdateParams& Params)
{
	UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetMutableSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
	check(RepresentationSubsystem);

	const FMassRepresentationParameters& RepresentationParams = Context.GetConstSharedFragment<FMassRepresentationParameters>();
	UMassRepresentationActorManagement* RepresentationActorManagement = RepresentationParams.CachedRepresentationActorManagement;
	check(RepresentationActorManagement);

	UMassActorSubsystem* MassActorSubsystem = Context.GetMutableSubsystem<UMassActorSubsystem>();

	FMassEntityManager& CachedEntityManager = Context.GetEntityManagerChecked();
	
	// Get Transform, Representation, RepresentationLOD and Actor fragments from Context
	const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
	const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
	const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
	const TArrayView<FMassActorFragment> ActorList = Context.GetMutableFragmentView<FMassActorFragment>();

	const bool bDoKeepActorExtraFrame = UE::Mass::Representation::ShouldKeepActorExtraFrame(RepresentationParams);
	const bool bDoWaitForActorVisualReadiness = UE::Mass::Representation::bAllowWaitForActorVisualReadiness
		? RepresentationParams.bWaitForActorVisualReadiness : false;

	// Iterate over all entities, and:
	// 1. Find their current EMassRepresentationType value based on their current RepresentationLOD;
	// 2. Change EMassRepresentationType value based on some configs (not all flows will care about this);
	// 3. Switch the in-game instance representation depending on the EMassRepresentationType;
	// 		a. If EMassRepresentationType == HighResSpawnedActor or LowResSpawnedActor, sends an Actor SpawnRequest to MassRepresentationActorManagement;
	// 		b. If EMassRepresentationType == StaticMeshInstance or SkinnedMeshInstance, sends an Actor disable request to MassRepresentationActorManagement;
	//		c. If EMassRepresentationType == StaticMeshInstance or SkinnedMeshInstance, sends an Actor disable request to MassRepresentationActorManagement.
	// 		NOTE: I guess this system assumes all instances are already represented by ISMs, which is why we're only dealing with Actor spawn/deactivation
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const FMassEntityHandle MassAgentEntityHandle = Context.GetEntity(EntityIt);
		const FTransformFragment& TransformFragment = TransformList[EntityIt];
		const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
		FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
		FMassActorFragment& ActorInfo = ActorList[EntityIt];
		AActor* Actor = ActorInfo.GetMutable();

		if (Representation.bIsPendingDestruction)
		{
			continue;
		}

		// Keeping a copy of the that last calculated previous representation
		const EMassRepresentationType PrevRepresentationCopy = Representation.PrevRepresentation;
		if (!Representation.IsWaitingForActorVisualReadiness())
		{
			Representation.PrevRepresentation = Representation.CurrentRepresentation;
		}

		// === 1. Find the current EMassRepresentationType value based on their current RepresentationLOD		
		EMassRepresentationType WantedRepresentationType = RepresentationParams.LODRepresentation[FMath::Min((int32)RepresentationLOD.LOD, (int32)EMassLOD::Off)];
		// === 1 end

		// === 2. Change EMassRepresentationType value based on some configs (not all flows will care about this)
		// Make sure we do not have actor spawned in areas not fully loaded
		if (Params.bTestCollisionAvailibilityForActorVisualization
			&& (WantedRepresentationType == EMassRepresentationType::HighResSpawnedActor || WantedRepresentationType == EMassRepresentationType::LowResSpawnedActor)
			&& !RepresentationSubsystem->IsCollisionLoaded(RepresentationParams.WorldPartitionGridNameContainingCollision, TransformFragment.GetTransform()))
		{
			WantedRepresentationType = RepresentationParams.CachedDefaultRepresentationType;
		}

		// If bForceActorRepresentationForExternalActors is enabled and we have an Actor reference for this entity, forcibly use it
		// by enforcing an actor representation as the WantedRepresentation. If we're coming from ISMC, we'll remove the instance
		// and switch to this actor, commiting either LowResSpawnedActor or HighResSpawnedActor as the new CurrentRepresentation.
		// Once the Actor is destroyed however, this override stops, allowing the natural WantedRepresentationType to return.
		//
		// Useful for server-authoritative Actor spawning, with replicated Actors inserting themselves into Mass whilst they're
		// replicated, enforcing actor representation on clients whilst they're present.
		//
		// NOTE:
		// IsOwnedByMass = Hydrated by mass
		// !IsOwnedByMass = Hydrated by some external system
		if (IsValid(Actor))
		{
			if (RepresentationParams.bForceActorRepresentationForExternalActors && !ActorInfo.IsOwnedByMass())
			{
				WantedRepresentationType = (Representation.CurrentRepresentation == EMassRepresentationType::LowResSpawnedActor)
					? EMassRepresentationType::LowResSpawnedActor
					: EMassRepresentationType::HighResSpawnedActor;
			}
		}
		// Has Actor unexpectedly been unset / destroyed since we last ran?
		else if (Representation.CurrentRepresentation == EMassRepresentationType::LowResSpawnedActor
			|| Representation.CurrentRepresentation == EMassRepresentationType::HighResSpawnedActor)
		{
			// Set CurrentRepresentation = None so we get a chance to see CurrentRepresentation != WantedRepresentationType and spawn
			// another actor.
			Representation.CurrentRepresentation = EMassRepresentationType::None;
			Representation.SetIsWaitingForActorVisualReadiness(false);
		}
		// === 2 end

		auto DisableActorForISM = [&](AActor*& Actor)
		{
			if (Actor != nullptr)
			{
				RepresentationActorManagement->SetActorEnabled(EMassActorEnabledType::Disabled, *Actor, EntityIt, Context.Defer());
			}
			if (!Actor || ActorInfo.IsOwnedByMass())
			{
				// Execute only if the high res is different than the low res Actor 
				// Or if we do not wish to keep the low res actor while in TransformList
				if (Representation.HighResTemplateActorIndex != Representation.LowResTemplateActorIndex || !RepresentationParams.bKeepLowResActors)
				{
					// Try releasing the high actor or any high res spawning request
					ReleaseActorOrCancelSpawning(*RepresentationSubsystem, MassActorSubsystem, MassAgentEntityHandle, ActorInfo, Representation.HighResTemplateActorIndex, Representation.ActorSpawnRequestHandle, Context.Defer());
					// Do not do the same with low res if indicated so
					if (!RepresentationParams.bKeepLowResActors)
					{
						ReleaseActorOrCancelSpawning(*RepresentationSubsystem, MassActorSubsystem, MassAgentEntityHandle, ActorInfo, Representation.LowResTemplateActorIndex, Representation.ActorSpawnRequestHandle, Context.Defer());
					}
				}
				// If we already queued spawn request but have changed our mind, continue with it but once we get the actor back, disable it immediately
				if (Representation.ActorSpawnRequestHandle.IsValid())
				{
					Actor = RepresentationActorManagement->GetOrSpawnActor(*RepresentationSubsystem, CachedEntityManager, MassAgentEntityHandle, TransformFragment.GetTransform(), Representation.LowResTemplateActorIndex, Representation.ActorSpawnRequestHandle, RepresentationActorManagement->GetSpawnPriority(RepresentationLOD));
					if (Actor != nullptr)
					{
						RepresentationActorManagement->SetActorEnabled(EMassActorEnabledType::Disabled, *Actor, EntityIt, Context.Defer());
					}
				}
			}
		};

		// === 3. Switch the in-game instance representation depending on the EMassRepresentationType;
		// Process switch between representation if there is a change in the representation or there is a pending spawning request
		if (WantedRepresentationType != Representation.CurrentRepresentation || Representation.ActorSpawnRequestHandle.IsValid())
		{
			if (Representation.CurrentRepresentation == EMassRepresentationType::None)
			{
				Representation.PrevTransform = TransformFragment.GetTransform();
				Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;
			}

			switch (WantedRepresentationType)
			{
				case EMassRepresentationType::HighResSpawnedActor:
				case EMassRepresentationType::LowResSpawnedActor:
				{
					const bool bHighResActor = WantedRepresentationType == EMassRepresentationType::HighResSpawnedActor;

					if (Representation.IsWaitingForActorVisualReadiness()
						&& Representation.LowResTemplateActorIndex == Representation.HighResTemplateActorIndex)
					{
						// LOD oscillated between LowRes/HighRes while we were waiting for actor readiness, AND both
						// variants share the same template actor - same physical actor, just a different "enabled type"
						// label. The actor is still disabled and the prior mesh representation is still rendering. Just
						// retarget the wanted variant; the bIsWaitingForActorVisualReadiness branch below will re-poll readiness and
						// enable as the new variant.
						//
						// When the templates differ we deliberately fall through to the normal spawn path so it can
						// re-issue a spawn against the correct template (releasing the wrong-template actor along the
						// way). That resets the pending state.
						Representation.CurrentRepresentation = WantedRepresentationType;
						break;
					}

					// Reuse actor, if it is valid and not owned by mass or same representation as low res without a valid spawning request
					AActor* NewActor = nullptr;
					if (!Actor || ActorInfo.IsOwnedByMass())
					{
						const int16 WantedTemplateActorIndex = bHighResActor ? Representation.HighResTemplateActorIndex : Representation.LowResTemplateActorIndex;

						// If the low res is different than the high res, cancel any pending spawn request that is the opposite of what is needed.
						if (Representation.LowResTemplateActorIndex != Representation.HighResTemplateActorIndex)
						{
							ReleaseActorOrCancelSpawning(*RepresentationSubsystem, MassActorSubsystem, MassAgentEntityHandle, ActorInfo, bHighResActor ? Representation.LowResTemplateActorIndex : Representation.HighResTemplateActorIndex, Representation.ActorSpawnRequestHandle, Context.Defer(), /*bCancelSpawningOnly*/true);
							Actor = ActorInfo.GetOwnedByMassMutable();
						}

						// If there isn't any actor yet or
						// If the actor isn't matching the one needed or
						// If there is still a pending spawn request
						// Then try to retrieve/spawn the new actor
						if (!Actor ||
							!RepresentationSubsystem->DoesActorMatchTemplate(*Actor, WantedTemplateActorIndex) ||
							Representation.ActorSpawnRequestHandle.IsValid())
						{
							NewActor = RepresentationActorManagement->GetOrSpawnActor(*RepresentationSubsystem, CachedEntityManager, MassAgentEntityHandle, TransformFragment.GetTransform(), WantedTemplateActorIndex, Representation.ActorSpawnRequestHandle, RepresentationActorManagement->GetSpawnPriority(RepresentationLOD));
						}
						else
						{
							NewActor = Actor;
						}
					}
					else
					{
						NewActor = Actor;
					}

					if (NewActor)
					{
						// Make sure our (re)activated actor is at the simulated position
						// Needs to be done before enabling the actor so the animation initialization can use the new values
						if (UE::Mass::Representation::IsValidMeshRepresentation(Representation.CurrentRepresentation))
						{
							RepresentationActorManagement->TeleportActor(TransformFragment.GetTransform(), *NewActor, Context.Defer());
						}

						const bool bActorReady = RepresentationActorManagement->IsActorReadyForRepresentation(*NewActor);
						const bool bHasMeshFallback = UE::Mass::Representation::IsValidMeshRepresentation(Representation.CurrentRepresentation);
						// Two reasons to defer enabling the actor:
						//   - bDoWaitForActorVisualReadiness: wait one frame so downstream observers/initializers run before the actor is shown.
						//     Only honoured when an ISM/SkinnedMesh is actually covering the gap; without one the deferral would just
						//     create a 1-frame invisibility hole for no benefit.
						//   - !bActorReady: wait until IsActorReadyForRepresentation reports true (e.g. async-streamed mesh asset loaded).
						//     This case is honoured even without a mesh fallback - accepting brief invisibility is preferred over
						//     showing a not-yet-ready (typically blank) actor. The MaxActorVisualReadinessWaitFrames cap bounds the wait.
						const bool bShouldDefer = (bDoWaitForActorVisualReadiness && bHasMeshFallback) || !bActorReady;

						if (bShouldDefer)
						{
							// If we're re-entering after a differing-template fall-through, the prior template's frame count
							// would otherwise carry over against the new actor's wait - reset to keep the timeout faithful to
							// the new actor.
							Representation.SetIsWaitingForActorVisualReadiness(true);
							Representation.CurrentRepresentation = WantedRepresentationType;
						}
						else
						{
							const EMassActorEnabledType EnabledType = bHighResActor
								? EMassActorEnabledType::HighRes : EMassActorEnabledType::LowRes;
							RepresentationActorManagement->SetActorEnabled(EnabledType, *NewActor, EntityIt, Context.Defer());
							Representation.CurrentRepresentation = WantedRepresentationType;
							// If we got here via differing-template fall-through with bIsWaitingForActorVisualReadiness carried over from
							// the prior template's wait, the new actor is ready and just got enabled - clear pending so the
							// re-poll branch doesn't fire a redundant SetActorEnabled next frame.
							Representation.SetIsWaitingForActorVisualReadiness(false);
						}
					}
					else if (!Actor)
					{
						Representation.CurrentRepresentation = RepresentationParams.CachedDefaultRepresentationType;
						// We arrive here when the spawn is async-pending (NewActor null) and there's no live Actor on the
						// fragment. If we entered via a differing-template fall-through with bIsWaitingForActorVisualReadiness still set
						// from the prior template's wait, leaving the flag set would cause next frame's CachedDefault case
						// (typically StaticMeshInstance) to hit its bIsWaitingForActorVisualReadiness guard and cancel the legitimate
						// in-flight new-template spawn. Clear pending state so the spawn handle drives to completion.
						Representation.SetIsWaitingForActorVisualReadiness(false);
					}
					else if (Representation.IsWaitingForActorVisualReadiness())
					{
						// NewActor is null (async spawn pending against the new template) but Actor is still the
						// wrong-template actor that hasn't yet been released by OnPreActorSpawn. We need to:
						//   1. Avoid the bIsWaitingForActorVisualReadiness re-poll branch enabling the wrong-template actor, AND
						//   2. Avoid the Switcher removing the ISM that's currently covering the visual gap.
						// Rolling CurrentRepresentation back to PrevRepresentation (still the prior mesh, frozen by the
						// guard at the top of the loop while pending) makes Wanted != Current next frame so the spawn
						// switch re-runs, while keeping Prev == Current so the Switcher sees no transition. Clearing
						// pending lets the NewActor-non-null path re-set it cleanly when OnPreActorSpawn fires.
						Representation.CurrentRepresentation = Representation.PrevRepresentation;
						Representation.SetIsWaitingForActorVisualReadiness(false);
					}
					break;
				}
				case EMassRepresentationType::StaticMeshInstance:
					if (Representation.IsWaitingForActorVisualReadiness())
					{
						// LOD oscillated back to ISM while waiting for actor visual readiness. ISM was never removed,
						// so cancel the pending init and release the never-enabled actor.
						DisableActorForISM(Actor);
						Representation.SetIsWaitingForActorVisualReadiness(false);
						Representation.CurrentRepresentation = EMassRepresentationType::StaticMeshInstance;
						break;
					}
					if (!bDoKeepActorExtraFrame || !UE::Mass::Representation::IsValidActorRepresentation(Representation.PrevRepresentation))
					{
						DisableActorForISM(Actor);
					}

					Representation.CurrentRepresentation = EMassRepresentationType::StaticMeshInstance;
					// Seed PrevTransform so the first frame of ISM rendering has a valid
					// previous transform (avoids motion vector pop from stale data).
					Representation.PrevTransform = TransformFragment.GetTransform();
					break;
				case EMassRepresentationType::SkinnedMeshInstance:
					if (Representation.IsWaitingForActorVisualReadiness())
					{
						DisableActorForISM(Actor);
						Representation.SetIsWaitingForActorVisualReadiness(false);
						Representation.CurrentRepresentation = EMassRepresentationType::SkinnedMeshInstance;
						break;
					}
					if (!bDoKeepActorExtraFrame || !UE::Mass::Representation::IsValidActorRepresentation(Representation.PrevRepresentation))
					{
						DisableActorForISM(Actor);
					}

					Representation.CurrentRepresentation = EMassRepresentationType::SkinnedMeshInstance;
					Representation.PrevTransform = TransformFragment.GetTransform();
					break;
				case EMassRepresentationType::None:
					if (!Actor || ActorInfo.IsOwnedByMass())
					{
						// Try releasing both, could have an high res spawned actor and a spawning request for a low res one
						ReleaseActorOrCancelSpawning(*RepresentationSubsystem, MassActorSubsystem, MassAgentEntityHandle, ActorInfo, Representation.LowResTemplateActorIndex, Representation.ActorSpawnRequestHandle, Context.Defer());
						ReleaseActorOrCancelSpawning(*RepresentationSubsystem, MassActorSubsystem, MassAgentEntityHandle, ActorInfo, Representation.HighResTemplateActorIndex, Representation.ActorSpawnRequestHandle, Context.Defer());
					}
					else
					{
						RepresentationActorManagement->SetActorEnabled(EMassActorEnabledType::Disabled, *Actor, EntityIt, Context.Defer());
					}
					Representation.CurrentRepresentation = EMassRepresentationType::None;
					Representation.SetIsWaitingForActorVisualReadiness(false);
					break;
				default:
					checkf(false, TEXT("Unsupported LOD type"));
					break;
			}
		}
		else if (Representation.IsWaitingForActorVisualReadiness())
		{
			// We get here when the spawn block on a previous frame deferred enabling the actor (either the
			// 1-frame init delay or the wait-for-actor-readiness path). Re-poll readiness each frame; once
			// the actor reports ready (or has gone away), enable it and clear the pending flag so the ISM
			// Switcher removes the mesh instance this frame.
			// !Actor is treated as "ready" so the wait terminates: the next frame the !IsValid(Actor)
			// destruction-recovery branch above (CurrentRepresentation == LowRes/HighRes) will reset
			// CurrentRepresentation back to None and trigger a fresh spawn cycle.
			const bool bActorReady = !Actor || RepresentationActorManagement->IsActorReadyForRepresentation(*Actor);

			// Safety cap: if an override never reports ready (broken asset reference, missing component, etc.) we'd
			// burn a virtual call per frame forever. After EffectiveMaxFrames frames we force-enable and warn.
			// The counter is uint8 (saturates at 255), so the cap is clamped to 255 - log the effective value
			// rather than the raw cvar value so the warning is honest about what we waited for.
			const int32 EffectiveMaxFrames = FMath::Min(UE::Mass::Representation::MaxActorVisualReadinessWaitFrames, FMassRepresentationFragment::MaxActorVisualReadinessWaitFrames);
			const bool bHitTimeout = !bActorReady && EffectiveMaxFrames > 0
				&& Representation.IsActorVisualRedinessTimedOut(EffectiveMaxFrames);

			if (bHitTimeout)
			{
				UE_LOGF(LogMassRepresentation, Warning,
					"%s: IsActorReadyForRepresentation never returned true within %d frames for entity %ls, "
					"force-enabling actor %ls. The actor will likely render with missing/blank mesh until streaming "
					"completes. Verify your override's logic.",
					__FUNCTION__, EffectiveMaxFrames,
					*MassAgentEntityHandle.DebugGetDescription(),
					Actor ? *Actor->GetName() : TEXT("<null>"));
			}

			if (bActorReady || bHitTimeout)
			{
				if (Actor)
				{
					// Re-teleport to the entity's current transform. The pending wait can span many frames and the
					// underlying entity (driving the ISM) may have moved since the deferral frame's TeleportActor call.
					// Without this the actor would render and collide at the stale spawn-frame position.
					//
					// Caveat: SetActorEnabled below calls SetActorTickEnabled SYNCHRONOUSLY (only collision-toggle is
					// deferred), so a tick fired between this call returning and the deferred command flush would still
					// see the stale transform. Override authors who do position-sensitive work in tick should not rely
					// on the actor's transform being correct at the first tick after re-enable.
					RepresentationActorManagement->TeleportActor(TransformFragment.GetTransform(), *Actor, Context.Defer());

					const bool bHighResActor = Representation.CurrentRepresentation == EMassRepresentationType::HighResSpawnedActor;
					RepresentationActorManagement->SetActorEnabled(
						bHighResActor ? EMassActorEnabledType::HighRes : EMassActorEnabledType::LowRes,
						*Actor, EntityIt, Context.Defer());
				}
				Representation.SetIsWaitingForActorVisualReadiness(false);
			}
			else
			{
				// Still not ready - keep ISM visible, actor disabled, flag set; we'll re-check next frame.
				Representation.IncreaseActorVisualReadinessWaitFrames();
			}
		}
		else if (bDoKeepActorExtraFrame
				&& UE::Mass::Representation::IsValidMeshRepresentation(Representation.PrevRepresentation)
				&& (PrevRepresentationCopy == EMassRepresentationType::HighResSpawnedActor
					|| PrevRepresentationCopy == EMassRepresentationType::LowResSpawnedActor))
		{
			DisableActorForISM(Actor);
		}
		// === 3 end
	}

#if WITH_MASSGAMEPLAY_DEBUG
	// Optional debug display
	if (UE::Mass::Representation::Debug::DebugRepresentation == 1 || UE::Mass::Representation::Debug::DebugRepresentation >= 3)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayRepresentation)
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			UE::Mass::Representation::Debug::DebugDisplayRepresentation(Context, RepresentationLODList, RepresentationList, TransformList, DebugDrawingWorld);
		}
	}
	// Optional vislog
	if (UE::Mass::Representation::Debug::DebugRepresentation >= 2)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VisLogRepresentation)
		UE::Mass::Representation::Debug::VisLogRepresentation(Context, RepresentationLODList, RepresentationList, TransformList, RepresentationSubsystem);
	}
#endif
}

void UMassRepresentationProcessor::Execute(FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
{
	// Update entities representation
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UpdateRepresentation(Context, UpdateParams);
	});
}

bool UMassRepresentationProcessor::ReleaseActorOrCancelSpawning(UMassRepresentationSubsystem& RepresentationSubsystem, UMassActorSubsystem* MassActorSubsystem
	, const FMassEntityHandle MassAgent, FMassActorFragment& ActorInfo, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle
	, FMassCommandBuffer& CommandBuffer, bool bCancelSpawningOnly /*= false*/)
{
	if (TemplateActorIndex == INDEX_NONE)
	{
		// Nothing to release
		return false;
	}
	check(!ActorInfo.IsValid() || ActorInfo.IsOwnedByMass());

	AActor* Actor = ActorInfo.GetOwnedByMassMutable();
	// note that it's fine for Actor to be null. That means the RepresentationSubsystem will try to stop 
	// the spawning of whatever SpawnRequestHandle reference to
	const bool bSuccess = bCancelSpawningOnly ? RepresentationSubsystem.CancelSpawning(MassAgent, TemplateActorIndex, SpawnRequestHandle) :
			RepresentationSubsystem.ReleaseTemplateActorOrCancelSpawning(MassAgent, TemplateActorIndex, Actor, SpawnRequestHandle);
	if (bSuccess)
	{
		Actor = ActorInfo.GetOwnedByMassMutable();
		if (Actor && RepresentationSubsystem.DoesActorMatchTemplate(*Actor, TemplateActorIndex))
		{
			ActorInfo.ResetNoHandleMapUpdate();
			
			TObjectKey<const AActor> ActorKey(Actor); 
			if (MassActorSubsystem)
			{
				CommandBuffer.PushCommand<FMassDeferredSetCommand>([MassActorSubsystem, ActorKey](FMassEntityManager&)
				{
					MassActorSubsystem->RemoveHandleForActor(ActorKey);
				});
			}
		}
		return true;
	}
	return false;
}

//----------------------------------------------------------------------//
// UMassVisualizationProcessor 
//----------------------------------------------------------------------//
UMassVisualizationProcessor::UMassVisualizationProcessor()
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::VisualizationProcessing;
}

void UMassVisualizationProcessor::UpdateVisualization(FMassExecutionContext& Context)
{
	FMassVisualizationChunkFragment& ChunkData = UpdateChunkVisibility(Context);
	if (!ChunkData.ShouldUpdateVisualization())
	{
		return;
	}

	UpdateRepresentation(Context, UpdateParams);

	// Update entity visibility
	const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
	const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();

	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
		FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
		const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
		UpdateEntityVisibility(Entity, Representation, RepresentationLOD, ChunkData, Context.Defer());
	}
}

void UMassVisualizationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	EntityQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassVisualizationProcessorTag>(EMassFragmentPresence::All);
}

FMassVisualizationChunkFragment& UMassVisualizationProcessor::UpdateChunkVisibility(FMassExecutionContext& Context) const
{
	const FMassRepresentationParameters& RepresentationParams = Context.GetConstSharedFragment<FMassRepresentationParameters>();
	bool bFirstUpdate = false;

	// Setup chunk fragment data about visibility
	FMassVisualizationChunkFragment& ChunkData = Context.GetMutableChunkFragment<FMassVisualizationChunkFragment>();
	EMassVisibility ChunkVisibility = ChunkData.GetVisibility();
	if (ChunkVisibility == EMassVisibility::Max)
	{
		// The visibility on the chunk fragment data isn't set yet, let see if the Archetype has an visibility tag and set it on the ChunkData
		ChunkVisibility = UE::Mass::Representation::GetVisibilityFromArchetype(Context);
		ChunkData.SetVisibility(ChunkVisibility);
		bFirstUpdate = RepresentationParams.bSpreadFirstVisualizationUpdate;
	}
	else
	{
		checkfSlow(UE::Mass::Representation::IsVisibilityTagSet(Context, ChunkVisibility), TEXT("Expecting the same Visibility as what we saved in the chunk data, maybe external code is modifying the tags"))
	}

	if (ChunkVisibility == EMassVisibility::CulledByDistance)
	{
		float DeltaTime = ChunkData.GetDeltaTime();
		if (bFirstUpdate)
		{
			// A DeltaTime of 0.0f means it will tick this frame.
			// @todo: Add some randomization for deterministic runs too. The randomization is used to distribute the infrequent ticks evenly on different frames.
			DeltaTime = UE::Mass::Utils::IsDeterministic() ? RepresentationParams.NotVisibleUpdateRate * 0.5f : FMath::RandRange(0.0f, RepresentationParams.NotVisibleUpdateRate);
		}
		else 
		{
			if (DeltaTime < 0.0f)
			{
				// @todo: Add some randomization for deterministic runs too. The randomization is used to distribute the infrequent ticks evenly on different frames.
				DeltaTime += UE::Mass::Utils::IsDeterministic() ? RepresentationParams.NotVisibleUpdateRate : (RepresentationParams.NotVisibleUpdateRate * (1.0f + FMath::RandRange(-0.1f, 0.1f)));
			}
			DeltaTime -= Context.GetDeltaTimeSeconds();
		}

		ChunkData.Update(DeltaTime);
	}

	return ChunkData;
}

void UMassVisualizationProcessor::UpdateEntityVisibility(const FMassEntityHandle Entity, const FMassRepresentationFragment& Representation, const FMassRepresentationLODFragment& RepresentationLOD, FMassVisualizationChunkFragment& ChunkData, FMassCommandBuffer& CommandBuffer)
{
	// Move the visible entities together into same chunks so we can skip entire chunk when not visible as an optimization
	const EMassVisibility Visibility = Representation.CurrentRepresentation != EMassRepresentationType::None ? 
		EMassVisibility::CanBeSeen : RepresentationLOD.Visibility;
	const EMassVisibility ChunkVisibility = ChunkData.GetVisibility();
	if (ChunkVisibility != Visibility)
	{
		UE::Mass::Representation::PushSwapTagsCommand(CommandBuffer, Entity, ChunkVisibility, Visibility);
		ChunkData.SetContainsNewlyVisibleEntity(Visibility == EMassVisibility::CanBeSeen);
	}
}

void UMassVisualizationProcessor::Execute(FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_Mass_VisProcessor_Execute);

	int32 TotalEntitiesProcessed = 0;
	// Update entities visualization
	EntityQuery.ForEachEntityChunk(Context, [this, &TotalEntitiesProcessed](FMassExecutionContext& Context)
	{
		TotalEntitiesProcessed += Context.GetNumEntities();
		UpdateVisualization(Context);
	});

	UE_VLOG(this, LogMassRepresentation, Verbose, TEXT("UMassVisualizationProcessor::Execute processed %d entities"), TotalEntitiesProcessed);
}

//----------------------------------------------------------------------//
// UMassRepresentationFragmentDestructor 
//----------------------------------------------------------------------//
UMassRepresentationFragmentDestructor::UMassRepresentationFragmentDestructor()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassRepresentationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	bRequiresGameThreadExecution = true; // due to FMassRepresentationSubsystemSharedFragment.RepresentationSubsystem use
}

void UMassRepresentationFragmentDestructor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassActorSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassRepresentationFragmentDestructor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetMutableSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		check(RepresentationSubsystem);
		UMassActorSubsystem* ActorSubsystem = Context.GetMutableSubsystem<UMassActorSubsystem>();

		const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
		const TArrayView<FMassActorFragment> ActorList = Context.GetMutableFragmentView<FMassActorFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
			FMassActorFragment& ActorInfo = ActorList[EntityIt];

			const FMassEntityHandle MassAgentEntityHandle = Context.GetEntity(EntityIt);

			UMassRepresentationActorManagement::ReleaseAnyActorOrCancelAnySpawning(*RepresentationSubsystem, MassAgentEntityHandle, ActorInfo, Representation, ActorSubsystem, /*bImmediate=*/true);
		}
	});
}
