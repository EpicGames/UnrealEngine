// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationSubsystem.h"
#include "Engine/World.h"
#include "MassActorSpawnerSubsystem.h"
#include "MassActorSubsystem.h"
#include "MassSimulationSubsystem.h"
#include "MassVisualizationComponent.h"
#include "MassVisualizer.h"
#include "MassRepresentationTypes.h"
#include "MassSimulationSettings.h"
#include "MassAgentComponent.h"
#include "MassAgentSubsystem.h"
#include "MassEntityManager.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationActorManagement.h"
#include "MassEntityView.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationSubsystem)


FStaticMeshInstanceVisualizationDescHandle UMassRepresentationSubsystem::FindOrAddStaticMeshDesc(const FStaticMeshInstanceVisualizationDesc& Desc)
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->FindOrAddVisualDesc(Desc);
	}

	return FStaticMeshInstanceVisualizationDescHandle();
}

FStaticMeshInstanceVisualizationDescHandle UMassRepresentationSubsystem::AddVisualDescWithISMComponent(const FStaticMeshInstanceVisualizationDesc& Desc, UInstancedStaticMeshComponent& ISMComponent)
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->AddVisualDescWithISMComponent(Desc, ISMComponent);
	}

	return FStaticMeshInstanceVisualizationDescHandle();
}

FStaticMeshInstanceVisualizationDescHandle UMassRepresentationSubsystem::AddVisualDescWithISMComponents(const FStaticMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedStaticMeshComponent>> ISMComponents)
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->AddVisualDescWithISMComponents(Desc, ISMComponents);
	}

	return FStaticMeshInstanceVisualizationDescHandle();
}

const FMassISMCSharedData* UMassRepresentationSubsystem::GetISMCSharedDataForDescriptionIndex(const int32 DescriptionIndex) const
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->GetISMCSharedDataForDescriptionIndex(DescriptionIndex);
	}

	return nullptr;
}

const FMassISMCSharedData* UMassRepresentationSubsystem::GetISMCSharedDataForInstancedStaticMesh(const UInstancedStaticMeshComponent* ISMC) const
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->GetISMCSharedDataForInstancedStaticMesh(ISMC);
	}
	return nullptr;
}

void UMassRepresentationSubsystem::RemoveVisualDesc(const FStaticMeshInstanceVisualizationDescHandle VisualizationHandle)
{
	if (LIKELY(VisualizationComponent))
	{
		VisualizationComponent->RemoveVisualDesc(VisualizationHandle);
	}
}

FMassInstancedStaticMeshInfoArrayView UMassRepresentationSubsystem::GetMutableInstancedStaticMeshInfos()
{
	if (LIKELY(VisualizationComponent))
	{
		FMassInstancedStaticMeshInfoArrayView View = VisualizationComponent->GetMutableVisualInfos();
		return MoveTemp(View);
	}

	// This can happen if the visualizer actor is destroyed prematurely.
	// It can also happen during world teardown if this is called after actors are destroyed.
	// This is not hit during normal execution, but we still need a default return value.
	FMassInstancedStaticMeshInfoArrayView EmptyView = MAKE_MASS_INSTANCED_STATIC_MESH_INFO_ARRAY_VIEW(MakeArrayView<FMassInstancedStaticMeshInfo>(nullptr, 0), InstancedStaticMeshInfosDetector);
	return MoveTemp(EmptyView);
}

FMassInstancedSkinnedMeshInfoArrayView UMassRepresentationSubsystem::GetMutableInstancedSkinnedMeshInfos()
{
	if (LIKELY(VisualizationComponent))
	{
		FMassInstancedSkinnedMeshInfoArrayView View = VisualizationComponent->GetMutableInstanceVisualInfos();
		return MoveTemp(View);
	}

	// This can happen if the visualizer actor is destroyed prematurely.
	// It can also happen during world teardown if this is called after actors are destroyed.
	// This is not hit during normal execution, but we still need a default return value.
	FMassInstancedSkinnedMeshInfoArrayView EmptyView = MAKE_MASS_INSTANCED_SKINNED_MESH_INFO_ARRAY_VIEW(MakeArrayView<FMassInstancedSkinnedMeshInfo>(nullptr, 0), InstancedSkinnedMeshInfosDetector);
	return MoveTemp(EmptyView);
}

void UMassRepresentationSubsystem::DirtyStaticMeshInstances()
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->DirtyVisuals();
	}
}

int16 UMassRepresentationSubsystem::FindOrAddTemplateActor(const TSubclassOf<AActor>& ActorClass)
{
	UE_MT_SCOPED_WRITE_ACCESS(TemplateActorsMTAccessDetector);

	int32 VisualIndex = TemplateActors.IndexOfByPredicate(FTemplateActorEqualsPredicate{ActorClass});

	if (VisualIndex == INDEX_NONE)
	{
		VisualIndex = TemplateActors.Add({ActorClass, 1u});
	}
	else
	{
		++TemplateActors[VisualIndex].RefCount;
	}

	check(VisualIndex < INT16_MAX);
	return (int16)VisualIndex;
}

AActor* UMassRepresentationSubsystem::GetOrRequestSpawnActorFromTemplate(const FMassEntityHandle MassAgent, const FTransform& Transform
	, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& InOutSpawnRequestHandle, float Priority
	, FMassActorPreSpawnDelegate ActorPreSpawnDelegate, FMassActorPostSpawnDelegate ActorPostSpawnDelegate)
{
	UE_MT_SCOPED_READ_ACCESS(TemplateActorsMTAccessDetector);
	if (!TemplateActors.IsValidIndex(TemplateActorIndex))
	{
		UE_LOGF(LogMassRepresentation, Error, "Template actor type %i is not referring to a valid type", TemplateActorIndex);
		return nullptr;
	}

	//@todo: this would be a good place to do pooling of actors instead of spawning them every time
	check(ActorSpawnerSubsystem);
	const TSubclassOf<AActor> TemplateToSpawn = TemplateActors[TemplateActorIndex].Actor;

	if (InOutSpawnRequestHandle.IsValid()
		&& ensureMsgf(ActorSpawnerSubsystem->IsSpawnRequestHandleValid(InOutSpawnRequestHandle)
			, TEXT("Given FMassActorSpawnRequestHandle(%d,sn:%d) %s"), InOutSpawnRequestHandle.GetIndex(), InOutSpawnRequestHandle.GetSerialNumber()
			, ActorSpawnerSubsystem->IsHandleReleased(InOutSpawnRequestHandle) ? TEXT("has already been RELEASED") : TEXT("is OUTDATED")))
	{
		FMassActorSpawnRequest& SpawnRequest = ActorSpawnerSubsystem->GetMutableSpawnRequest<FMassActorSpawnRequest>(InOutSpawnRequestHandle);
		// Check if this existing spawn request is matching the template actor
		if (SpawnRequest.Template != TemplateToSpawn)
		{
			return nullptr;
		}
		switch (SpawnRequest.SpawnStatus)
		{
			case ESpawnRequestStatus::RetryPending:
			case ESpawnRequestStatus::Pending:
				// Update spawn request with latest information
				SpawnRequest.Transform = Transform;
				SpawnRequest.Priority = Priority;
				return nullptr;
			case ESpawnRequestStatus::Failed:
			{
				check(!SpawnRequest.SpawnedActor);
				// The most common case for failure is that collision was preventing the spawning, the agent might have moved or the thing preventing it from spawning might have moved too.
				UWorld* World = GetWorld();
				check(World);

				// Limits the retry as they are bad for rendering performance
				if ((World->GetTimeSeconds() - SpawnRequest.RequestedTime) > RetryTimeInterval ||
					(SpawnRequest.Transform.GetLocation() - Transform.GetLocation()).SizeSquared() > RetryMovedDistanceSq)
				{
					// Update spawn request with latest information and retry
					SpawnRequest.Transform = Transform;
					SpawnRequest.Priority = Priority;
					ActorSpawnerSubsystem->RetryActorSpawnRequest(InOutSpawnRequestHandle);
				}
				return nullptr;
			}
			case ESpawnRequestStatus::Succeeded:
			{
				AActor* SpawnedActor = SpawnRequest.SpawnedActor;
				check(SpawnedActor);
				ensureMsgf(ActorSpawnerSubsystem->RemoveActorSpawnRequest(InOutSpawnRequestHandle), TEXT("Unable to remove a valid spawn request"));
				return SpawnedActor;
			}
			default:
				checkf(false, TEXT("Unexpected spawn request status!"));
				InOutSpawnRequestHandle.Invalidate();
				return nullptr;
		}
	}

	// If we reach here, means we need to create a spawn request
	FMassActorSpawnRequest SpawnRequest;
	SpawnRequest.MassAgent = MassAgent;
	if (FMassGuidFragment* GuidFragment = EntityManager->GetFragmentDataPtr<FMassGuidFragment>(MassAgent))
	{
		SpawnRequest.Guid = GuidFragment->Guid;
	}
	SpawnRequest.Template = TemplateToSpawn;
	SpawnRequest.Transform = Transform;
	SpawnRequest.Priority = Priority;
	SpawnRequest.ActorPreSpawnDelegate = ActorPreSpawnDelegate;
	SpawnRequest.ActorPostSpawnDelegate = ActorPostSpawnDelegate;
	InOutSpawnRequestHandle = ActorSpawnerSubsystem->RequestActorSpawn(SpawnRequest);

	++(HandledMassAgents.FindOrAdd(MassAgent));

	return nullptr;
}

TSubclassOf<AActor> UMassRepresentationSubsystem::GetTemplateActorClass(const int16 TemplateActorIndex)
{
	UE_MT_SCOPED_READ_ACCESS(TemplateActorsMTAccessDetector);
	if (!TemplateActors.IsValidIndex(TemplateActorIndex))
	{
		UE_LOGF(LogMassRepresentation, Error, "Template actor type %i is not referring to a valid type", TemplateActorIndex);
		return nullptr;
	}

	return TemplateActors[TemplateActorIndex].Actor;
}

bool UMassRepresentationSubsystem::IsCollisionLoaded(const FName TargetGrid, const FTransform& Transform) const
{
	if (!WorldPartitionSubsystem)
	{
		// Assuming that all collisions are loaded if not using WorldPartition.
		return true;
	}

	// @todo optimize by doing one query per cell
	// Build a query source
	TArray<FWorldPartitionStreamingQuerySource> QuerySources;
	FWorldPartitionStreamingQuerySource& QuerySource = QuerySources.Emplace_GetRef();
	QuerySource.bSpatialQuery = true;
	QuerySource.Location = Transform.GetLocation();
	QuerySource.Rotation = Transform.Rotator();
	if (!TargetGrid.IsNone())
	{
		QuerySource.TargetGrids.Add(TargetGrid);
	}
	QuerySource.bUseGridLoadingRange = false;
	QuerySource.Radius = 1.f; // 1cm should be enough to know if grid is loaded at specific area
	QuerySource.bDataLayersOnly = false;

	// Execute query
	return WorldPartitionSubsystem->IsStreamingCompleted(EWorldPartitionRuntimeCellState::Activated, QuerySources, /*bExactState*/ false);
}

void UMassRepresentationSubsystem::ReleaseTemplate(const TSubclassOf<AActor>& ActorClass)
{
	if (ActorClass)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TemplateActorsMTAccessDetector);
		
		const int32 TemplateActorIndex = TemplateActors.IndexOfByPredicate(FTemplateActorEqualsPredicate{ActorClass});
		check(TemplateActors.IsValidIndex(TemplateActorIndex));

		FTemplateActorData& TemplateActorData = TemplateActors[TemplateActorIndex];
		check(TemplateActorData.RefCount > 0u);
		--TemplateActorData.RefCount;

		if (TemplateActorData.RefCount == 0u)
		{
			TemplateActors.RemoveAt(TemplateActorIndex);
		}
	}
}

void UMassRepresentationSubsystem::ReleaseAllResources()
{
	ensureMsgf(HandledMassAgents.Num() == 0, TEXT("All entities must be released before releasing resources"));

	{
		// Release all template actors
		UE_MT_SCOPED_WRITE_ACCESS(TemplateActorsMTAccessDetector);
		TemplateActors.Reset();
	}

	// Release all static meshes resources
	if (LIKELY(VisualizationComponent))
	{
		VisualizationComponent->ClearAllVisualInstances();
	}
}

bool UMassRepresentationSubsystem::ReleaseTemplateActorOrCancelSpawning(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, AActor* ActorToRelease
	, FMassActorSpawnRequestHandle& SpawnRequestHandle, const bool bImmediate /*= false*/)
{
	// First try to cancel the spawning request, then try to release the actor
	if (CancelSpawningInternal(TemplateActorIndex, SpawnRequestHandle, bImmediate) 
		|| ReleaseTemplateActorInternal(TemplateActorIndex, ActorToRelease, bImmediate))
	{ 
		const int32 RefCount = --(HandledMassAgents.FindChecked(MassAgent));
		checkf(RefCount >= 0, TEXT("RefCount are expected to be greater than or equal to 0"));
		if (RefCount == 0)
		{
			HandledMassAgents.Remove(MassAgent);
		}
		return true;
	}

	return false;
}

bool UMassRepresentationSubsystem::ReleaseTemplateActor(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, AActor* ActorToRelease, bool bImmediate)
{
	if (ReleaseTemplateActorInternal(TemplateActorIndex, ActorToRelease, bImmediate))
	{
		const int32 RefCount = --HandledMassAgents.FindChecked(MassAgent);
		checkf(RefCount >= 0, TEXT("RefCount are expected to be greater than or equal to 0"));
		if (RefCount == 0)
		{
			HandledMassAgents.Remove(MassAgent);
		}
		return true;
	}

	return false;
}

bool UMassRepresentationSubsystem::CancelSpawning(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle)
{
	if (CancelSpawningInternal(TemplateActorIndex, SpawnRequestHandle))
	{
		const int32 RefCount = --HandledMassAgents.FindChecked(MassAgent);
		checkf(RefCount >= 0, TEXT("RefCount are expected to be greater than or equal to 0"));
		if (RefCount == 0)
		{
			HandledMassAgents.Remove(MassAgent);
		}
		return true;
	}
	return false;
}

bool UMassRepresentationSubsystem::ReleaseTemplateActorInternal(const int16 TemplateActorIndex, AActor* ActorToRelease, bool bImmediate)
{
	UE_MT_SCOPED_READ_ACCESS(TemplateActorsMTAccessDetector);
	if (!TemplateActors.IsValidIndex(TemplateActorIndex))
	{
		UE_LOGF(LogMassRepresentation, Error, "Template actor type %i is not referring a valid type", TemplateActorIndex);
		return false;
	}
	const TSubclassOf<AActor> TemplateToRelease = TemplateActors[TemplateActorIndex].Actor;

	// We can only release existing and matching template actors
	if (!ActorToRelease || ActorToRelease->GetClass() != TemplateToRelease)
	{
		return false;
	}
	ActorSpawnerSubsystem->DestroyActor(ActorToRelease, bImmediate);
	return true;
}

bool UMassRepresentationSubsystem::CancelSpawningInternal(const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle, const bool bImmediateActorRelease)
{
	UE_MT_SCOPED_READ_ACCESS(TemplateActorsMTAccessDetector);
	if (!TemplateActors.IsValidIndex(TemplateActorIndex))
	{
		UE_LOGF(LogMassRepresentation, Error, "Template actor type %i is not referring a valid type", TemplateActorIndex);
		return false;
	}
	const TSubclassOf<AActor> TemplateToRelease = TemplateActors[TemplateActorIndex].Actor;

	// Check if there is something to cancel
	if (SpawnRequestHandle.IsValid() == false)
	{
		return false;
	}

	check(ActorSpawnerSubsystem);
	if (ActorSpawnerSubsystem->IsSpawnRequestHandleValid(SpawnRequestHandle) == false)
	{
		UE_VLOG_UELOG(this, LogMassRepresentation, Error
			, TEXT("Attempting to cancel spawning using a NO LONGER VALID spawn request handle [i: %d, s/n: %d]")
			, SpawnRequestHandle.GetIndex(), SpawnRequestHandle.GetSerialNumber());
		return false;
	}

	const FMassActorSpawnRequest& SpawnRequest = ActorSpawnerSubsystem->GetMutableSpawnRequest<FMassActorSpawnRequest>(SpawnRequestHandle);
	// Check if the spawning request matches the template actor
	if (SpawnRequest.Template != TemplateToRelease)
	{
		return false;
	}

	if (SpawnRequest.SpawnStatus == ESpawnRequestStatus::Succeeded)
	{
		check(SpawnRequest.SpawnedActor);
		ReleaseTemplateActorInternal(TemplateActorIndex, SpawnRequest.SpawnedActor, bImmediateActorRelease);
	}

	// Remove the spawn request
	ensureMsgf(ActorSpawnerSubsystem->RemoveActorSpawnRequest(SpawnRequestHandle), TEXT("Unable to remove a valid spawn request"));

	return true;
}

bool UMassRepresentationSubsystem::DoesActorMatchTemplate(const AActor& Actor, const int16 TemplateActorIndex) const
{
	UE_MT_SCOPED_READ_ACCESS(TemplateActorsMTAccessDetector);
	if (!TemplateActors.IsValidIndex(TemplateActorIndex))
	{
		UE_LOGF(LogMassRepresentation, Error, "Template actor type %i is not referring to a valid type", TemplateActorIndex);
		return false;
	}

	const TSubclassOf<AActor> Template = TemplateActors[TemplateActorIndex].Actor;
	return Actor.GetClass() == Template;
}

void UMassRepresentationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency(UMassSimulationSubsystem::StaticClass());
	Collection.InitializeDependency(UMassActorSpawnerSubsystem::StaticClass());
	Collection.InitializeDependency(UMassAgentSubsystem::StaticClass());
	Super::Initialize(Collection);

	if (UWorld* World = GetWorld())
	{
		EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World).AsShared();

		ActorSpawnerSubsystem = World->GetSubsystem<UMassActorSpawnerSubsystem>();
		WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>();

		if (SpawnVisualizer(World))
		{
			UMassSimulationSubsystem* SimSystem = World->GetSubsystem<UMassSimulationSubsystem>();
			check(SimSystem);
			SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassRepresentationSubsystem::OnProcessingPhaseStarted, EMassProcessingPhase::PrePhysics);
			SimSystem->GetOnProcessingPhaseFinished(EMassProcessingPhase::PostPhysics).AddUObject(this, &UMassRepresentationSubsystem::OnProcessingPhaseStarted, EMassProcessingPhase::PostPhysics);
		}

		UMassAgentSubsystem* AgentSystem = World->GetSubsystem<UMassAgentSubsystem>();
		check(AgentSystem);
		AgentSystem->GetOnMassAgentComponentEntityAssociated().AddUObject(this, &UMassRepresentationSubsystem::OnMassAgentComponentEntityAssociated);
		AgentSystem->GetOnMassAgentComponentEntityDetaching().AddUObject(this, &UMassRepresentationSubsystem::OnMassAgentComponentEntityDetaching);
	}

	const UMassSimulationSettings* SimSettings = GetDefault<UMassSimulationSettings>();
	check(SimSettings);
	RetryMovedDistanceSq = FMath::Square(SimSettings->DesiredActorFailedSpawningRetryMoveDistance);
	RetryTimeInterval = SimSettings->DesiredActorFailedSpawningRetryTimeInterval;
}

void UMassRepresentationSubsystem::Deinitialize()
{
	if (const UWorld* World = GetWorld())
	{
		if (UMassSimulationSubsystem* SimSystem = World->GetSubsystem<UMassSimulationSubsystem>())
		{
			SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).RemoveAll(this);
			SimSystem->GetOnProcessingPhaseFinished(EMassProcessingPhase::PostPhysics).RemoveAll(this);
		}

		if (UMassAgentSubsystem* AgentSystem = World->GetSubsystem<UMassAgentSubsystem>())
		{
			AgentSystem->GetOnMassAgentComponentEntityAssociated().RemoveAll(this);
			AgentSystem->GetOnMassAgentComponentEntityDetaching().RemoveAll(this);
		}
	}
	EntityManager.Reset();

	Super::Deinitialize();
}

void UMassRepresentationSubsystem::HandleVisualizerEndPlay(AActor* Actor, const EEndPlayReason::Type EndPlayReason)
{
	Visualizer = nullptr;
	VisualizationComponent = nullptr;

	if (UWorld* World = GetWorld())
	{
		// attempt to create a new visualizer in the case that the old one was destroyed prematurely
		if (!SpawnVisualizer(World))
		{
			// visualizer could not be created, so try to unregister phase delegates
			if (UMassSimulationSubsystem* SimSystem = World->GetSubsystem<UMassSimulationSubsystem>())
			{
				SimSystem->GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).RemoveAll(this);
				SimSystem->GetOnProcessingPhaseFinished(EMassProcessingPhase::PostPhysics).RemoveAll(this);
			}
		}
	}
}

bool UMassRepresentationSubsystem::SpawnVisualizer(TNotNull<UWorld*> World)
{
	if (World->bIsTearingDown)
	{
		return false;
	}

	if (Visualizer == nullptr)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		// The helper actor is only once per world so we can allow it to spawn during construction script.
		SpawnInfo.bAllowDuringConstructionScript = true;
		Visualizer = World->SpawnActor<AMassVisualizer>(SpawnInfo);
		check(Visualizer);
		Visualizer->OnEndPlay.AddUniqueDynamic(this, &UMassRepresentationSubsystem::HandleVisualizerEndPlay);
		VisualizationComponent = &Visualizer->GetVisualizationComponent();

#if WITH_EDITOR
		Visualizer->SetActorLabel(FString::Printf(TEXT("%sVisualizer"), *GetClass()->GetName()), /*bMarkDirty*/false);
#endif
	}
	return true;
}

void UMassRepresentationSubsystem::OnProcessingPhaseStarted(const float /*DeltaSeconds*/, const EMassProcessingPhase Phase) const
{
	if (LIKELY(VisualizationComponent))
	{
		switch (Phase)
		{
		case EMassProcessingPhase::PrePhysics:
			VisualizationComponent->BeginVisualChanges();
			break;
		case EMassProcessingPhase::PostPhysics:/* Currently this is the end of phases signal */
			VisualizationComponent->EndVisualChanges();
			break;
		default:
			check(false); // Need to handle this case
			break;
		}
	}
}

void UMassRepresentationSubsystem::OnMassAgentComponentEntityAssociated(const UMassAgentComponent& AgentComponent)
{
	check(EntityManager);

	const FMassEntityHandle MassAgent = AgentComponent.GetEntityHandle();
	checkf(EntityManager->IsEntityValid(MassAgent), TEXT("Expecting a valid mass entity"));
	if (EntityManager->IsEntityValid(MassAgent) && AgentComponent.IsNetSimulating())
	{
		// Check if this mass agent already handled by this sub system, if yes than release any local spawned actor or cancel any spawn requests
		if (HandledMassAgents.Find(MassAgent))
		{
			UMassRepresentationActorManagement::ReleaseAnyActorOrCancelAnySpawning(*EntityManager.Get(), MassAgent);
		}
	}
}

void UMassRepresentationSubsystem::OnMassAgentComponentEntityDetaching(const UMassAgentComponent& AgentComponent)
{
	check(EntityManager);

	AActor* ComponentOwner = AgentComponent.GetOwner();
	check(ComponentOwner);

	const FMassEntityHandle MassAgent = AgentComponent.GetEntityHandle();
	checkf(EntityManager->IsEntityValid(MassAgent), TEXT("Expecting a valid mass entity"));
	if (EntityManager->IsEntityValid(MassAgent) && AgentComponent.IsNetSimulating())
	{
		const FMassEntityView EntityView(*EntityManager.Get(), MassAgent);
		if (FMassRepresentationFragment* Representation = EntityView.GetFragmentDataPtr<FMassRepresentationFragment>())
		{
			// Force a reevaluate of the current representation
			Representation->CurrentRepresentation = EMassRepresentationType::None;
		}
	}
}

void UMassRepresentationSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMassRepresentationSubsystem& TypedThis = *CastChecked<UMassRepresentationSubsystem>(InThis);
	for (FTemplateActorData& TemplateActorData : TypedThis.TemplateActors)
	{
		Collector.AddStableReference(&TemplateActorData.Actor.GetGCPtr());
	}
}

FSkinnedMeshInstanceVisualizationDescHandle UMassRepresentationSubsystem::FindOrAddSkinnedMeshVisualizationDesc(const FSkinnedMeshInstanceVisualizationDesc& Desc)
{
	if (LIKELY(VisualizationComponent))
	{
		return VisualizationComponent->FindOrAddSkinnedMeshInstanceVisualDesc(Desc);
	}

	return FSkinnedMeshInstanceVisualizationDescHandle();
}
