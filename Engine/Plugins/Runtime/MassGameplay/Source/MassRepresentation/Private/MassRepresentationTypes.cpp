// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRepresentationTypes.h"
#include "MassRepresentationUtils.h"
#include "MassCommandBuffer.h"
#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRepresentationTypes)

DEFINE_LOG_CATEGORY(LogMassRepresentation);

namespace UE::Mass::Representation
{
const FColor RepresentationTypeColors[] =
{
	FColor::Red,     // HighResSpawnedActor
	FColor::Yellow,  // LowResSpawnedActor
	FColor::Cyan,    // SkinnedMeshInstance
	FColor::Emerald, // StaticMeshInstance
	FColor::White,   // None
};

static_assert(UE_ARRAY_COUNT(RepresentationTypeColors) == static_cast<uint8>(EMassRepresentationType::None) + 1,
	"RepresentationTypeColors must have one entry per EMassRepresentationType value. "
	"If you added a new value to the enum, append a matching color entry here and update this assertion.");

void PushSwapTagsCommand(FMassCommandBuffer& CommandBuffer, const FMassEntityHandle Entity, const EMassVisibility PrevVisibility, const EMassVisibility NewVisibility)
{
#define CASE_SWAP_TAGS(OldVisibility, NewVisibility) \
	case NewVisibility: \
		CommandBuffer.SwapTags<TMassVisibilityTagForLevel<OldVisibility>::FTag, TMassVisibilityTagForLevel<NewVisibility>::FTag>(Entity); \
		break

#define CASE_ADD_TAG(NewVisibility) \
case NewVisibility: \
	CommandBuffer.AddTag<TMassVisibilityTagForLevel<NewVisibility>::FTag>(Entity); \
	break

#define DEFAULT_REMOVE_TAG(OldVisibility) \
case EMassVisibility::Max: /* fall through on purpose */ \
default: \
	CommandBuffer.RemoveTag<TMassVisibilityTagForLevel<OldVisibility>::FTag>(Entity); \
	break

	check(PrevVisibility != NewVisibility);

	switch (PrevVisibility)
	{
	case EMassVisibility::CanBeSeen:
		switch (NewVisibility)
		{
		CASE_SWAP_TAGS(EMassVisibility::CanBeSeen, EMassVisibility::CulledByFrustum);
		CASE_SWAP_TAGS(EMassVisibility::CanBeSeen, EMassVisibility::CulledByDistance);
		DEFAULT_REMOVE_TAG(EMassVisibility::CanBeSeen);
		}
		break;
	case EMassVisibility::CulledByFrustum:
		switch (NewVisibility)
		{
		CASE_SWAP_TAGS(EMassVisibility::CulledByFrustum, EMassVisibility::CanBeSeen);
		CASE_SWAP_TAGS(EMassVisibility::CulledByFrustum, EMassVisibility::CulledByDistance);
		DEFAULT_REMOVE_TAG(EMassVisibility::CulledByFrustum);
		}
		break;
	case EMassVisibility::CulledByDistance:
		switch (NewVisibility)
		{
		CASE_SWAP_TAGS(EMassVisibility::CulledByDistance, EMassVisibility::CanBeSeen);
		CASE_SWAP_TAGS(EMassVisibility::CulledByDistance, EMassVisibility::CulledByFrustum);
		DEFAULT_REMOVE_TAG(EMassVisibility::CulledByDistance);
		}
		break;
	case EMassVisibility::Max:
		switch (NewVisibility)
		{
		CASE_ADD_TAG(EMassVisibility::CanBeSeen);
		CASE_ADD_TAG(EMassVisibility::CulledByFrustum);
		CASE_ADD_TAG(EMassVisibility::CulledByDistance);
		default:
			checkf(false, TEXT("Unsupported Visibility types!"));
			break;
		}
		break;
	default:
		checkf(false, TEXT("Unsupported Visibility type!"));
		break;
	}

#undef CASE_SWAP_TAGS
}
} // UE::Mass::Representation

//-----------------------------------------------------------------------------
// FMassInstancedStaticMeshInfo
//-----------------------------------------------------------------------------
void FMassInstancedStaticMeshInfo::ClearVisualInstance(UInstancedStaticMeshComponent& ISMComponent)
{
	if (InstancedStaticMeshComponents.RemoveSingleSwap(&ISMComponent, EAllowShrinking::No))
	{
		ISMComponent.ClearInstances();
		ISMComponent.DestroyComponent();
	}
	else
	{
		InstancedStaticMeshComponents.Reset();
		LODSignificanceRanges.Reset();
	}
}

void FMassInstancedStaticMeshInfo::Reset()
{
	Desc.Reset();
	InstancedStaticMeshComponents.Reset();
	LODSignificanceRanges.Reset();
}

//-----------------------------------------------------------------------------
// FMassStaticMeshInstanceVisualizationMeshDesc
//-----------------------------------------------------------------------------
FMassStaticMeshInstanceVisualizationMeshDesc::FMassStaticMeshInstanceVisualizationMeshDesc()
{
	ISMComponentClass = UInstancedStaticMeshComponent::StaticClass();
}

//-----------------------------------------------------------------------------
// FStaticMeshInstanceVisualizationDesc
//-----------------------------------------------------------------------------
bool FStaticMeshInstanceVisualizationDesc::IsValid() const
{
	for (const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : Meshes)
	{
		if (MeshDesc.Mesh && MeshDesc.ISMComponentClass)
		{
			return true;
		}
	}
	return false;
}
