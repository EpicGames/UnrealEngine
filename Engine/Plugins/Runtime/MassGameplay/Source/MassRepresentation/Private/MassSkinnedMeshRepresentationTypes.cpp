// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSkinnedMeshRepresentationTypes.h"
#include "Components/InstancedSkinnedMeshComponent.h"

//-----------------------------------------------------------------------------
// FMassSkinnedMeshInstanceVisualizationMeshDesc
//-----------------------------------------------------------------------------
FMassSkinnedMeshInstanceVisualizationMeshDesc::FMassSkinnedMeshInstanceVisualizationMeshDesc()
{
	InstancedSkinnedMeshComponentClass = UInstancedSkinnedMeshComponent::StaticClass();
}

//-----------------------------------------------------------------------------
// FSkinnedMeshInstanceVisualizationDesc
//-----------------------------------------------------------------------------
bool FSkinnedMeshInstanceVisualizationDesc::IsValid() const
{
	for (const FMassSkinnedMeshInstanceVisualizationMeshDesc& MeshDesc : Meshes)
	{
		if (MeshDesc.Asset && MeshDesc.InstancedSkinnedMeshComponentClass)
		{
			return true;
		}
	}
	return false;
}

void FMassInstancedSkinnedMeshInfo::Reset()
{
	Desc.Reset();
	InstancedSkinnedMeshComponents.Reset();
	LODSignificanceRanges.Reset();
}
