// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStationaryDistanceVisualizationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "VisualLogger/VisualLogger.h"
#include "StationaryVisualizationTraitUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStationaryDistanceVisualizationTrait)


UMassStationaryDistanceVisualizationTrait::UMassStationaryDistanceVisualizationTrait(const FObjectInitializer& ObjectInitializer)
{
	bAllowServerSideVisualization = true;
}

void UMassStationaryDistanceVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	UE::Mass::LOD::Private::SetUpStationaryVisualizationTrait(*this, BuildContext, StaticMeshInstanceDesc);

	Super::BuildTemplate(BuildContext, World);

	BuildContext.RequireTag<FMassCollectDistanceLODViewerInfoTag>();
}

#if WITH_EDITOR
void UMassStationaryDistanceVisualizationTrait::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName StaticMeshInstanceDescName = GET_MEMBER_NAME_CHECKED(UMassStationaryDistanceVisualizationTrait, StaticMeshInstanceDesc);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == StaticMeshInstanceDescName)
	{
		for (FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : StaticMeshInstanceDesc.Meshes)
		{
			MeshDesc.Mobility = EComponentMobility::Stationary;
		}
	}
}
#endif // WITH_EDITOR
