// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStationaryISMRepresentationFragmentDestructor.h"
#include "MassRepresentationSubsystem.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "MassCommonFragments.h"
#include "MassRepresentationProcessor.h"

//-----------------------------------------------------------------------------
// UMassStationaryISMRepresentationFragmentDestructor
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStationaryISMRepresentationFragmentDestructor)
UMassStationaryISMRepresentationFragmentDestructor::UMassStationaryISMRepresentationFragmentDestructor()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassRepresentationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllWorldModes;
	bRequiresGameThreadExecution = true; // not sure about this
}

void UMassStationaryISMRepresentationFragmentDestructor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassStaticRepresentationTag>(EMassFragmentPresence::All);
}

void UMassStationaryISMRepresentationFragmentDestructor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetMutableSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		check(RepresentationSubsystem);
		FMassInstancedStaticMeshInfoArrayView ISMInfosView = RepresentationSubsystem->GetMutableInstancedStaticMeshInfos();
		FMassInstancedSkinnedMeshInfoArrayView MeshInfosView = RepresentationSubsystem->GetMutableInstancedSkinnedMeshInfos();

		if (ISMInfosView.Num() == 0 && MeshInfosView.Num() == 0)
		{
			return;
		}

		const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassRepresentationFragment& Representation = RepresentationList[EntityIt];
			// Also handle pending init: CurrentRepresentation is actor type but ISM instance is still live
			if (Representation.CurrentRepresentation == EMassRepresentationType::StaticMeshInstance
				|| (Representation.IsWaitingForActorVisualReadiness()
					&& Representation.PrevRepresentation == EMassRepresentationType::StaticMeshInstance))
			{
				const int32 ISMInfoIndex = Representation.StaticMeshDescHandle.ToIndex();
				if (!ISMInfosView.IsValidIndex(ISMInfoIndex))
				{
					UE_LOGF(LogMassRepresentation, Warning, "Invalid handle index %u for ISMInfosView", ISMInfoIndex);
					Representation.CurrentRepresentation = EMassRepresentationType::None;
					continue;
				}
				FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[ISMInfoIndex];
				if (FMassLODSignificanceRange* OldRange = ISMInfo.GetLODSignificanceRange(Representation.PrevLODSignificance))
				{
					const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
					if (OldRange)
					{
						OldRange->RemoveInstance(EntityHandle);
					}
				}
				Representation.CurrentRepresentation = EMassRepresentationType::None;
				Representation.SetIsWaitingForActorVisualReadiness(false);
			}
			else if (Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance
				|| (Representation.IsWaitingForActorVisualReadiness()
					&& Representation.PrevRepresentation == EMassRepresentationType::SkinnedMeshInstance))
			{
				const int32 MeshInfoIndex = Representation.SkinnedMeshDescHandle.ToIndex();
				if (!MeshInfosView.IsValidIndex(MeshInfoIndex))
				{
					UE_LOGF(LogMassRepresentation, Warning, "Invalid handle index %u for MeshInfosView", MeshInfoIndex);
					Representation.CurrentRepresentation = EMassRepresentationType::None;
					continue;
				}
				FMassInstancedSkinnedMeshInfo& MeshInfo = MeshInfosView[MeshInfoIndex];
				if (FMassLODInstancedSkinnedMeshSignificanceRange* OldRange = MeshInfo.GetLODSignificanceRange(Representation.PrevLODSignificance))
				{
					const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
					if (OldRange)
					{
						OldRange->RemoveInstance(EntityHandle);
					}
				}
				Representation.CurrentRepresentation = EMassRepresentationType::None;
				Representation.SetIsWaitingForActorVisualReadiness(false);
			}
		}
	});
}
