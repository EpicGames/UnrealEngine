// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStationaryISMSwitcherProcessor.h"
#include "MassRepresentationSubsystem.h"
#include "MassCommonFragments.h"
#include "MassRepresentationProcessor.h"
#include "MassRepresentationTypes.h"

#include "MassSignalSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStationaryISMSwitcherProcessor)


UMassStationaryISMSwitcherProcessor::UMassStationaryISMSwitcherProcessor(const FObjectInitializer& ObjectInitializer)
	: EntityQuery(*this)
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Representation;
	ExecutionOrder.ExecuteAfter.Add(UMassVisualizationProcessor::StaticClass()->GetFName());
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllNetModes);
}

void UMassStationaryISMSwitcherProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassStaticRepresentationTag>(EMassFragmentPresence::All);
	EntityQuery.AddTagRequirement<FMassStationaryISMSwitcherProcessorTag>(EMassFragmentPresence::All);
	EntityQuery.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassStationaryISMSwitcherProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, &UMassStationaryISMSwitcherProcessor::ProcessContext);
}
	
void UMassStationaryISMSwitcherProcessor::ProcessContext(FMassExecutionContext& Context)
{
	UMassSignalSubsystem& SignalSubsystem = Context.GetMutableSubsystemChecked<UMassSignalSubsystem>();

	UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetMutableSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
	check(RepresentationSubsystem);
	FMassInstancedStaticMeshInfoArrayView ISMInfosView = RepresentationSubsystem->GetMutableInstancedStaticMeshInfos();
	FMassInstancedSkinnedMeshInfoArrayView MeshInfosView = RepresentationSubsystem->GetMutableInstancedSkinnedMeshInfos();

	const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
	const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
	const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();

	const FMassRepresentationParameters& RepresentationParams = Context.GetConstSharedFragment<FMassRepresentationParameters>();
	const bool bDoKeepActorExtraFrame = UE::Mass::Representation::ShouldKeepActorExtraFrame(RepresentationParams);

	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
		const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
		const FTransformFragment& TransformFragment = TransformList[EntityIt];
		FMassRepresentationFragment& Representation = RepresentationList[EntityIt];

		if (Representation.bIsPendingDestruction)
		{
			// nothing to do here
			continue;
		}

		if (!UE::Mass::Representation::IsValidMeshRepresentation(Representation.PrevRepresentation) && !UE::Mass::Representation::IsValidMeshRepresentation(Representation.CurrentRepresentation))
{
			// nothing to do here
			continue;
		}

		bool bIsStaticMeshInstance = (Representation.PrevRepresentation == EMassRepresentationType::StaticMeshInstance || Representation.CurrentRepresentation == EMassRepresentationType::StaticMeshInstance);
		bool bIsSkinnedMeshInstance = (Representation.PrevRepresentation == EMassRepresentationType::SkinnedMeshInstance || Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance);

		if (bIsStaticMeshInstance && !ensureMsgf(Representation.StaticMeshDescHandle.IsValid() && ISMInfosView.IsValidIndex(Representation.StaticMeshDescHandle.ToIndex())
						, TEXT("Invalid handle index %u for ISMInfosView"), Representation.StaticMeshDescHandle.ToIndex()))
		{
			continue;
		}

		if (bIsSkinnedMeshInstance && !ensureMsgf(Representation.SkinnedMeshDescHandle.IsValid() && MeshInfosView.IsValidIndex(Representation.SkinnedMeshDescHandle.ToIndex())
						, TEXT("Invalid handle index %u for MeshInfosView"), Representation.SkinnedMeshDescHandle.ToIndex()))
		{
			continue;
		}

		if (const bool bSwitchedAwayFromStaticMesh = (Representation.PrevRepresentation == EMassRepresentationType::StaticMeshInstance
			&& Representation.CurrentRepresentation != EMassRepresentationType::StaticMeshInstance))
		{
			if (Representation.IsWaitingForActorVisualReadiness())
			{
				// Delay ISM removal - actor still initializing, mesh covers visual gap.
				// PrevLODSignificance already correctly tracks the range the ISM instance is in.
				continue;
			}

			FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[Representation.StaticMeshDescHandle.ToIndex()];

			// note that we're using the PrevLODSignificance here, and the reason for it is that the Prev value matches the
			// PrevRepresentation - thus we need to remove from the "previously" used LODSignificance range.
			ISMInfo.RemoveInstance(EntityHandle, Representation.PrevLODSignificance);

			// consume "prev" data
			Representation.PrevRepresentation = Representation.CurrentRepresentation;

			if (Representation.PrevRepresentation != EMassRepresentationType::None && !bIsSkinnedMeshInstance)
			{
				SignalSubsystem.SignalEntity(UE::Mass::Signals::SwitchedToActor, EntityHandle);
			}
		}
		else if (const bool bSwitchedAwayFromSkinnedMesh = (Representation.PrevRepresentation == EMassRepresentationType::SkinnedMeshInstance
			&& Representation.CurrentRepresentation != EMassRepresentationType::SkinnedMeshInstance))
		{
			if (Representation.IsWaitingForActorVisualReadiness())
			{
				continue;
			}

			FMassInstancedSkinnedMeshInfo& MeshInfo = MeshInfosView[Representation.SkinnedMeshDescHandle.ToIndex()];

			// note that we're using the PrevLODSignificance here, and the reason for it is that the Prev value matches the
			// PrevRepresentation - thus we need to remove from the "previously" used LODSignificance range.
			MeshInfo.RemoveInstance(EntityHandle, Representation.PrevLODSignificance);

			// consume "prev" data
			Representation.PrevRepresentation = Representation.CurrentRepresentation;

			if (Representation.PrevRepresentation != EMassRepresentationType::None && !bIsStaticMeshInstance)
			{
				SignalSubsystem.SignalEntity(UE::Mass::Signals::SwitchedToActor, EntityHandle);
			}
		}
		else if (const bool bSwitchedToStaticMesh = (Representation.PrevRepresentation != EMassRepresentationType::StaticMeshInstance
			&& Representation.CurrentRepresentation == EMassRepresentationType::StaticMeshInstance))
		{
			FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[Representation.StaticMeshDescHandle.ToIndex()];

			const FTransform& Transform = TransformFragment.GetTransform();
			const float LODSignificance = RepresentationLOD.LODSignificance;
				
			if (FMassLODSignificanceRange* NewRange = ISMInfo.GetLODSignificanceRange(RepresentationLOD.LODSignificance))
			{
				if (ISMInfo.ShouldUseTransformOffset())
				{
					const FTransform& TransformOffset = ISMInfo.GetTransformOffset();
					const FTransform SMTransform = TransformOffset * Transform;
					NewRange->AddInstance(EntityHandle, SMTransform);
				}
				else
				{
					NewRange->AddInstance(EntityHandle, Transform);
				}

				if (ISMInfo.GetDesc().CustomDataFloats.Num() > 0)
				{
					NewRange->AddBatchedCustomDataFloats(ISMInfo.GetDesc().CustomDataFloats, {});
				}
			}

			// Track which LODSignificance range the ISM instance was added to, so the "remain in ISM"
			// range-migration logic uses the correct range for removal if significance changes next frame.
			Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;

			// consume "prev" data
			// @note crazy hacky, but we don't want to consume if bDoKeepActorExtraFrame is true. In that case
			// UMassRepresentationProcessor::UpdateRepresentation expects the "prev" state not to be consumed
			// a frame longer so that it can do the consuming (and call "disable actor").
			if (bDoKeepActorExtraFrame == false)
			{
				Representation.PrevRepresentation = Representation.CurrentRepresentation;
			}

			if (!bIsSkinnedMeshInstance)
			{
				SignalSubsystem.SignalEntity(UE::Mass::Signals::SwitchedToISM, EntityHandle);
			}
		}
		else if (const bool bSwitchedToSkinnedMesh = (Representation.PrevRepresentation != EMassRepresentationType::SkinnedMeshInstance
			&& Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance))
		{
			FMassInstancedSkinnedMeshInfo& MeshInfo = MeshInfosView[Representation.SkinnedMeshDescHandle.ToIndex()];

			const FTransform& Transform = TransformFragment.GetTransform();
			const float LODSignificance = RepresentationLOD.LODSignificance;

			if (FMassLODInstancedSkinnedMeshSignificanceRange* NewRange = MeshInfo.GetLODSignificanceRange(RepresentationLOD.LODSignificance))
			{
				if (MeshInfo.ShouldUseTransformOffset())
				{
					const FTransform& TransformOffset = MeshInfo.GetTransformOffset();
					const FTransform SMTransform = TransformOffset * Transform;
					NewRange->AddInstance(EntityHandle, SMTransform);
				}
				else
				{
					NewRange->AddInstance(EntityHandle, Transform);
				}

				if (MeshInfo.GetDesc().CustomDataFloats.Num() > 0)
				{
					NewRange->AddBatchedCustomDataFloats(MeshInfo.GetDesc().CustomDataFloats, {});
				}
			}

			Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;

			// consume "prev" data
			// @note crazy hacky, but we don't want to consume if bDoKeepActorExtraFrame is true. In that case 
			// UMassRepresentationProcessor::UpdateRepresentation expects the "prev" state not to be consumed 
			// a frame longer so that it can do the consuming (and call "disable actor").
			if (bDoKeepActorExtraFrame == false)
			{
				Representation.PrevRepresentation = Representation.CurrentRepresentation;
			}

			if (bIsStaticMeshInstance)
			{
				SignalSubsystem.SignalEntity(UE::Mass::Signals::SwitchedToISM, EntityHandle);
			}
		}
		else if (bIsStaticMeshInstance)
		{
			FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[Representation.StaticMeshDescHandle.ToIndex()];

			if (ISMInfo.GetLODSignificanceRangesNum() > 1 && Representation.PrevLODSignificance != RepresentationLOD.LODSignificance)
			{
				// we remain in ISM land, but LODSignificance changed and we have multiple LODSignificance ranges for this entity
				FMassLODSignificanceRange* OldRange = ISMInfo.GetLODSignificanceRange(Representation.PrevLODSignificance);
				FMassLODSignificanceRange* NewRange = ISMInfo.GetLODSignificanceRange(RepresentationLOD.LODSignificance);
				if (OldRange != NewRange)
				{
					if (OldRange)
					{
						OldRange->RemoveInstance(EntityHandle);
					}
					if (NewRange)
					{
						const FTransform& Transform = TransformFragment.GetTransform();
						NewRange->AddInstance(EntityHandle, Transform);
					}
				}
		
				// consume "prev" data
				Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;
			}
		}
		else if (bIsSkinnedMeshInstance)
		{
			FMassInstancedSkinnedMeshInfo& MeshInfo = MeshInfosView[Representation.SkinnedMeshDescHandle.ToIndex()];

			if (MeshInfo.GetLODSignificanceRangesNum() > 1 && Representation.PrevLODSignificance != RepresentationLOD.LODSignificance)
			{
				// we remain in Skinned Mesh land, but LODSignificance changed and we have multiple LODSignificance ranges for this entity
				FMassLODInstancedSkinnedMeshSignificanceRange* OldRange = MeshInfo.GetLODSignificanceRange(Representation.PrevLODSignificance);
				FMassLODInstancedSkinnedMeshSignificanceRange* NewRange = MeshInfo.GetLODSignificanceRange(RepresentationLOD.LODSignificance);
				if (OldRange != NewRange)
				{
					if (OldRange)
					{
						OldRange->RemoveInstance(EntityHandle);
					}
					if (NewRange)
					{
						const FTransform& Transform = TransformFragment.GetTransform();
						NewRange->AddInstance(EntityHandle, Transform);
					}
				}

				// consume "prev" data
				Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;
			}
		}
	}
}