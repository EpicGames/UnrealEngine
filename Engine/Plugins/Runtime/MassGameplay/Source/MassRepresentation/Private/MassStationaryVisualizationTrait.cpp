// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStationaryVisualizationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStationaryVisualizationTrait)


namespace UE::Mass::LOD::Private
{
	void SetUpStationaryVisualizationTrait(const UMassEntityTraitBase& Trait, FMassEntityTemplateBuildContext& BuildContext, FStaticMeshInstanceVisualizationDesc& StaticMeshInstanceDesc)
	{
		bool bIssuesFound = false;
		for (FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : StaticMeshInstanceDesc.Meshes)
		{
			bIssuesFound = bIssuesFound || (MeshDesc.Mobility != EComponentMobility::Stationary);
			MeshDesc.Mobility = EComponentMobility::Stationary;
			MeshDesc.bRequiresExternalInstanceIDTracking = true;
		}
		UE_CVLOG_UELOG(bIssuesFound, &Trait, LogMass, Log, TEXT("%s some Meshes' mobility has been set to non-Stationary. These settings will be overridden."), *Trait.GetPathName());
		BuildContext.AddTag<FMassStaticRepresentationTag>();
		BuildContext.AddTag<FMassStationaryISMSwitcherProcessorTag>();
	}
}

//-----------------------------------------------------------------------------
// UMassStationaryVisualizationTrait
//-----------------------------------------------------------------------------
UMassStationaryVisualizationTrait::UMassStationaryVisualizationTrait(const FObjectInitializer& ObjectInitializer)
{
	bAllowServerSideVisualization = true;
}

void UMassStationaryVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	UE::Mass::LOD::Private::SetUpStationaryVisualizationTrait(*this, BuildContext, StaticMeshInstanceDesc);

	Super::BuildTemplate(BuildContext, World);

	BuildContext.RequireTag<FMassCollectLODViewerInfoTag>();
}

#if WITH_EDITOR
void UMassStationaryVisualizationTrait::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName StaticMeshInstanceDescName = GET_MEMBER_NAME_CHECKED(UMassStationaryVisualizationTrait, StaticMeshInstanceDesc);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == StaticMeshInstanceDescName)
	{
		for (FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : StaticMeshInstanceDesc.Meshes)
		{
			MeshDesc.Mobility = EComponentMobility::Stationary;
		}
	}
}
#endif // WITH_EDITOR
