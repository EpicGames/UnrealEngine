// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassUpdateISMProcessor.h"
#include "MassVisualizationComponent.h"
#include "MassRepresentationSubsystem.h"
#include "MassEntityManager.h"
#include "MassRepresentationFragments.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassLODFragments.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassUpdateISMProcessor)

UMassUpdateISMProcessor::UMassUpdateISMProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Representation);
	bRequiresGameThreadExecution = true;
}

void UMassUpdateISMProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.SetChunkFilter(&FMassVisualizationChunkFragment::AreAnyEntitiesVisibleInChunk);
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);

	// ignore entities configured to have their representation static (@todo maybe just check if there's not movement fragment?)
	EntityQuery.AddTagRequirement<FMassStaticRepresentationTag>(EMassFragmentPresence::None);
}

void UMassUpdateISMProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		check(RepresentationSubsystem);
		FMassInstancedStaticMeshInfoArrayView ISMInfo = RepresentationSubsystem->GetMutableInstancedStaticMeshInfos();

		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
		const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
		const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FTransformFragment& TransformFragment = TransformList[EntityIt];
			const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];
			FMassRepresentationFragment& Representation = RepresentationList[EntityIt];

			if (Representation.CurrentRepresentation == EMassRepresentationType::StaticMeshInstance)
			{
				const int32 ISMInfoIndex = Representation.StaticMeshDescHandle.ToIndex();
				if (ensureMsgf(ISMInfo.IsValidIndex(ISMInfoIndex), TEXT("Invalid handle index %u for ISMInfosView"), ISMInfoIndex))
				{
					UpdateISMTransform(Context.GetEntity(EntityIt), ISMInfo[ISMInfoIndex], TransformFragment.GetTransform(), Representation.PrevTransform, RepresentationLOD.LODSignificance, Representation.PrevLODSignificance);
				}

				Representation.PrevTransform = TransformFragment.GetTransform();
				Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;
			}
		}
	});
}

void UMassUpdateISMProcessor::UpdateISMTransform(FMassEntityHandle EntityHandle, FMassInstancedStaticMeshInfo& ISMInfo, const FTransform& Transform, const FTransform& PrevTransform, const float LODSignificance, const float PrevLODSignificance/* = -1.0f*/)
{
	if (ISMInfo.ShouldUseTransformOffset())
	{
		const FTransform& TransformOffset = ISMInfo.GetTransformOffset();
		const FTransform SMTransform = TransformOffset * Transform;
		const FTransform SMPrevTransform = TransformOffset * PrevTransform;

		ISMInfo.AddBatchedTransform(EntityHandle, SMTransform, SMPrevTransform, LODSignificance, PrevLODSignificance);
	}
	else
	{
		ISMInfo.AddBatchedTransform(EntityHandle, Transform, PrevTransform, LODSignificance, PrevLODSignificance);
	}
}
