// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassUpdateSkinnedMeshProcessor.h"

#include "Engine/World.h"
#include "MassCommonFragments.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassLODFragments.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationSubsystem.h"
#include "MassVisualizationComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassUpdateSkinnedMeshProcessor)

UMassUpdateInstancedSkinnedMeshProcessor::UMassUpdateInstancedSkinnedMeshProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Representation;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::VisualizationProcessing);
	bRequiresGameThreadExecution = true;
}

void UMassUpdateInstancedSkinnedMeshProcessor::UpdateMeshTransform(FMassEntityHandle EntityHandle, FMassInstancedSkinnedMeshInfo& MeshInfo, const FTransform& Transform, const FTransform& PrevTransform, const float LODSignificance, const float PrevLODSignificance /*= -1.0f*/)
{
	if (MeshInfo.ShouldUseTransformOffset())
	{
		const FTransform& TransformOffset = MeshInfo.GetTransformOffset();
		const FTransform SMTransform = TransformOffset * Transform;
		const FTransform SMPrevTransform = TransformOffset * PrevTransform;

		MeshInfo.AddBatchedTransform(EntityHandle, SMTransform, SMPrevTransform, LODSignificance, PrevLODSignificance);
	}
	else
	{
		MeshInfo.AddBatchedTransform(EntityHandle, Transform, PrevTransform, LODSignificance, PrevLODSignificance);
	}
}

void UMassUpdateInstancedSkinnedMeshProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.SetChunkFilter(&FMassVisualizationChunkFragment::AreAnyEntitiesVisibleInChunk);
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);

	// ignore entities configured to have their representation static (@todo maybe just check if there's not movement fragment?)
	EntityQuery.AddTagRequirement<FMassStaticRepresentationTag>(EMassFragmentPresence::None);
}

void UMassUpdateInstancedSkinnedMeshProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
			check(RepresentationSubsystem);

			FMassInstancedSkinnedMeshInfoArrayView SkinnedMeshInfos = RepresentationSubsystem->GetMutableInstancedSkinnedMeshInfos();

			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FMassRepresentationFragment> RepresentationList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
			const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FTransformFragment& TransformFragment = TransformList[EntityIt];
				const FMassRepresentationLODFragment& RepresentationLOD = RepresentationLODList[EntityIt];

				FMassRepresentationFragment& Representation = RepresentationList[EntityIt];

				if (Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance)
				{
					const int32 SkinnedMeshInfoIndex = Representation.SkinnedMeshDescHandle.ToIndex();
					if (ensureMsgf(SkinnedMeshInfos.IsValidIndex(SkinnedMeshInfoIndex), TEXT("Invalid handle index %u for MeshInfos View"), SkinnedMeshInfoIndex))
					{
						UpdateMeshTransform(Context.GetEntity(EntityIt), SkinnedMeshInfos[SkinnedMeshInfoIndex], TransformFragment.GetTransform(), Representation.PrevTransform, RepresentationLOD.LODSignificance, Representation.PrevLODSignificance);
					}

					Representation.PrevTransform = TransformFragment.GetTransform();
					Representation.PrevLODSignificance = RepresentationLOD.LODSignificance;
				}
			}
		});
}
