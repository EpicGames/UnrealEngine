// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassVisualizationComponent.h"
#include "CoreGlobals.h"
#include "Logging/LogMacros.h"
#include "MassEntityTypes.h"
#include "MassVisualizer.h"
#include "MassRepresentationTypes.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "AI/NavigationSystemBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassVisualizationComponent)


DECLARE_CYCLE_STAT(TEXT("Mass Visualization EndVisualChanges"), STAT_Mass_VisualizationComponent_EndVisualChanges, STATGROUP_Mass);
DECLARE_CYCLE_STAT(TEXT("Mass Visualization HandleIDs"), STAT_Mass_VisualizationComponent_HandleChangesWithExternalIDTracking, STATGROUP_Mass);

DECLARE_DWORD_COUNTER_STAT(TEXT("VisualizationComp Instances Removed"), STAT_Mass_VisualizationComponent_InstancesRemovedNum, STATGROUP_Mass);
DECLARE_DWORD_COUNTER_STAT(TEXT("VisualizationComp Instances Added"), STAT_Mass_VisualizationComponent_InstancesAddedNum, STATGROUP_Mass);

//---------------------------------------------------------------
// UMassVisualizationComponent
//---------------------------------------------------------------

namespace UE::Mass::Representation
{
	int32 GCallUpdateInstances = 1;
	FAutoConsoleVariableRef  CVarCallUpdateInstances(TEXT("Mass.CallUpdateInstances"), GCallUpdateInstances, TEXT("Toggle between UpdateInstances and BatchUpdateTransform."));

#if STATS
	uint32 LastStatsResetFrame = 0;
#endif // STATS
}  // UE::Mass::Representation

void UMassVisualizationComponent::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject) == false && GetOuter())
	{
		ensureMsgf(GetOuter()->GetClass()->IsChildOf(AMassVisualizer::StaticClass()), TEXT("UMassVisualizationComponent should only be added to AMassVisualizer-like instances"));
	}
}

FStaticMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddInstancedStaticMeshInfo(const FStaticMeshInstanceVisualizationDesc& Desc)
{
	FStaticMeshInstanceVisualizationDescHandle Handle;
	if (InstancedStaticMeshInfosFreeIndices.Num() > 0)
	{
		Handle = InstancedStaticMeshInfosFreeIndices.Pop(EAllowShrinking::No);
		new(&InstancedStaticMeshInfos[Handle.ToIndex()]) FMassInstancedStaticMeshInfo(Desc);
	}
	else
	{
		int32 AddedInfoIndex = InstancedStaticMeshInfos.Emplace(Desc);
		Handle = FStaticMeshInstanceVisualizationDescHandle(AddedInfoIndex);
	}

	return Handle;
}

FStaticMeshInstanceVisualizationDescHandle UMassVisualizationComponent::FindOrAddVisualDesc(const FStaticMeshInstanceVisualizationDesc& Desc)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);
	// First check to see if we already have a matching Desc already and reuse / return that
	// Note: FStaticMeshInstanceVisualizationDescHandle(int32) handles the INDEX_NONE case here, generating an invalid handle in this case
	FStaticMeshInstanceVisualizationDescHandle VisualDescHandle(InstancedStaticMeshInfos.IndexOfByPredicate([&Desc](const FMassInstancedStaticMeshInfo& Info) { return Info.GetDesc() == Desc; }));
	if (!VisualDescHandle.IsValid())
	{
		if (Desc.IsValid())
		{
			for (const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : Desc.Meshes)
			{
				if (MeshDesc.Mesh && MeshDesc.ISMComponentClass)
				{
					// if we've already encountered MeshDesc in the past MeshDescToISMCMap already contains information
					// about actual ISMC used to represent it, and at the same time indicates the ISMCSharedData data
					// tied to it. Regardless we need to process all MeshDesc instances here so that we have all the 
					// data ready when InstancedSMComponentsRequiringConstructing gets processed next time
					// UMassVisualizationComponent::ConstructStaticMeshComponents gets called.
					MeshDescToISMCMap.FindOrAdd(GetTypeHash(MeshDesc), FISMCSharedDataKey());
				}
			}

			VisualDescHandle = AddInstancedStaticMeshInfo(Desc);
			check(VisualDescHandle.IsValid());

			// VisualDescHandle is a valid handle now, but there's initialization pending, performed in ConstructStaticMeshComponents
			InstancedSMComponentsRequiringConstructing.Add(VisualDescHandle);
		}
		else
		{
			UE_LOGF(LogMassRepresentation, Warning, "%s: invalid FStaticMeshInstanceVisualizationDesc passed in. Check the contained meshes.", __FUNCTION__);
		}
	}

	return VisualDescHandle;
}

FStaticMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddVisualDescWithISMComponent(const FStaticMeshInstanceVisualizationDesc& Desc, UInstancedStaticMeshComponent& ISMComponent)
{
	TObjectPtr<UInstancedStaticMeshComponent> AsObjectPtr = &ISMComponent;
	return AddVisualDescWithISMComponents(Desc, MakeArrayView(&AsObjectPtr, 1));
}

FStaticMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddVisualDescWithISMComponents(const FStaticMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedStaticMeshComponent>> ISMComponents)
{
	check(Desc.Meshes.Num() == ISMComponents.Num());
	
	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);


	// 0. Iterate over all meshes in the visualization desc. Each mesh is a descriptor.
	FStaticMeshInstanceVisualizationDescHandle VisualHandle;
	TArray<UInstancedStaticMeshComponent*> ISMComponentsUsed;
	for (int32 EntryIndex = 0; EntryIndex < Desc.Meshes.Num(); ++EntryIndex)
	{
		const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc = Desc.Meshes[EntryIndex];
		if (MeshDesc.Mesh == nullptr || ISMComponents[EntryIndex] == nullptr)
		{
			// invalid description, log an continue.
			UE_VLOG_UELOG(this, LogMassRepresentation, Error, TEXT("Empty mesh at index %d while registering FStaticMeshInstanceVisualizationDesc instance"), EntryIndex);
			continue;
		}
	
		// 1. Creates a VisualHandle that will be used for all ISMCs
		if (!VisualHandle.IsValid())
		{
			VisualHandle = AddInstancedStaticMeshInfo(Desc);
			check(VisualHandle.IsValid());
		}

		// 2. Stores one FMassISMCSharedData for each ISMC in ISMCSharedData
		// NOTE: FMassISMCSharedData stores a reference to an ISMC and instance transform updates performed in the current frame.
		FMassISMCSharedData& NewData = ISMCSharedData.FindOrAdd(ISMComponents[EntryIndex], FMassISMCSharedData(ISMComponents[EntryIndex], /*bInRequiresExternalInstanceIDTracking=*/true, /*InTransformOffset=*/MeshDesc.LocalTransform));
		// 3. Stores the newly created FMassISMCSharedData in a VisualHandle -> FMassISMCSharedData array
		// NOTE: This means that one VisualHandle may reference multiple FMassISMCSharedData
		InstancedStaticMeshInfos[VisualHandle.ToIndex()].AddISMComponent(NewData);
		// 4. Stores an ISMC -> VisualHandle map, which allows us to later on use an ISMC to query for its FMassISMCSharedData
		ISMComponentMap.Add(ISMComponents[EntryIndex], VisualHandle);

		ISMComponentsUsed.Add(ISMComponents[EntryIndex]);
	}

	if (VisualHandle.IsValid())
	{
		BuildLODSignificanceForInfo(InstancedStaticMeshInfos[VisualHandle.ToIndex()], ISMComponentsUsed);
	}

	return VisualHandle;
}

const FMassISMCSharedData* UMassVisualizationComponent::GetISMCSharedDataForDescriptionIndex(const int32 DescriptionIndex) const
{
	return ISMCSharedData.GetDataForIndex(DescriptionIndex);
}

const FMassISMCSharedData* UMassVisualizationComponent::GetISMCSharedDataForInstancedStaticMesh(const UInstancedStaticMeshComponent* ISMC) const
{
	return ISMCSharedData.GetDataForKey(ISMC);
}

void UMassVisualizationComponent::RemoveVisualDesc(const FStaticMeshInstanceVisualizationDescHandle VisualizationHandle)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);
	
	if (ensure(InstancedStaticMeshInfos.IsValidIndex(VisualizationHandle.ToIndex()))
		&& ensureMsgf(InstancedStaticMeshInfos[VisualizationHandle.ToIndex()].IsValid(), TEXT("Trying to remove visualization data that has already been cleaned")))
	{
		for (TObjectPtr<UInstancedStaticMeshComponent>& ISMComponent : InstancedStaticMeshInfos[VisualizationHandle.ToIndex()].InstancedStaticMeshComponents)
		{
			const bool bValidKey = ISMComponentMap.Contains(ISMComponent);
			checkf(bValidKey, TEXT("Failed to find ISMC in ISMComponentMap, path: %s"), *ISMComponent.GetPathName());
			if (bValidKey)
			{
				const FStaticMeshInstanceVisualizationDescHandle StoredVisualizationDescHandle = ISMComponentMap.FindAndRemoveChecked(ISMComponent);
				ensure(StoredVisualizationDescHandle == VisualizationHandle);
			}
		
			ISMCSharedData.Remove(ISMComponent);
		}
		
		InstancedStaticMeshInfos[VisualizationHandle.ToIndex()].Reset();
		InstancedStaticMeshInfosFreeIndices.Add(VisualizationHandle);
	}
}

void UMassVisualizationComponent::ConstructStaticMeshComponents()
{
	AActor* ActorOwner = GetOwner();
	check(ActorOwner);
	
	TArray<UInstancedStaticMeshComponent*> TransientISMCs;

	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);
	for (const FStaticMeshInstanceVisualizationDescHandle VisualDescHandle : InstancedSMComponentsRequiringConstructing)
	{
		if (!ensureMsgf(InstancedStaticMeshInfos.IsValidIndex(VisualDescHandle.ToIndex())
			, TEXT("InstancedStaticMeshInfos (size: %d) is never expected to shrink, so VisualDescHandle (value: %u) being invalid indicates it was wrong from the start.")
			, InstancedStaticMeshInfos.Num(), VisualDescHandle.ToIndex()))
		{
			continue;
		}

		FMassInstancedStaticMeshInfo& Info = InstancedStaticMeshInfos[VisualDescHandle.ToIndex()];

		// Check if it is already created
		if (!Info.InstancedStaticMeshComponents.IsEmpty())
		{
			continue;
		}

		// Check if there are any specified meshes for this visual type
		if(Info.Desc.Meshes.Num() == 0)
		{
			UE_LOGF(LogMassRepresentation, Error, "No associated meshes for this instanced static mesh type");
			continue;
		}

		TransientISMCs.Reset();
		for (const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : Info.Desc.Meshes)
		{
			// MeshDescToISMCMap here lets us figure out whether for the given MeshDesc we need to create a new ISM component
			// or a one has already been created in the past. Note that we only need this intermediate map for 
			// FMassStaticMeshInstanceVisualizationMeshDesc that has been added to the system without specifying an
			// ISM component to handle the instances (i.e. added via FindOrAddVisualDesc rather than AddVisualDescWithISMComponents).
			// This is the only kind of FMassStaticMeshInstanceVisualizationMeshDesc were processing here. 
			FISMCSharedDataKey& ISMCKey = MeshDescToISMCMap.FindChecked(GetTypeHash(MeshDesc));
			FMassISMCSharedData* SharedData = ISMCSharedData.Find(ISMCKey);
			UInstancedStaticMeshComponent* ISMC = SharedData ? SharedData->GetMutableISMComponent() : nullptr;

			if (ISMC == nullptr)
			{
				ISMC = NewObject<UInstancedStaticMeshComponent>(ActorOwner, MeshDesc.ISMComponentClass);	
				CA_ASSUME(ISMC);
				REDIRECT_OBJECT_TO_VLOG(ISMC, this);

				// note that ISMCKey is a reference, so the assignment below actually sets a value in MeshDescToISMCMap
				// and all subsequent handling of a given MeshDesc configuration (i.e. containing same values) will 
				// result in referring to the ISMC we just created.
				ISMCKey = ISMC;

				ISMC->SetStaticMesh(MeshDesc.Mesh);
				for (int32 ElementIndex = 0; ElementIndex < MeshDesc.MaterialOverrides.Num(); ++ElementIndex)
				{
					if (UMaterialInterface* MaterialOverride = MeshDesc.MaterialOverrides[ElementIndex])
					{
						ISMC->SetMaterial(ElementIndex, MaterialOverride);
					}
				}
				ISMC->SetCullDistances(0, 1000000); // @todo: Need to figure out what to do here, either LOD or cull distances.
				ISMC->SetupAttachment(ActorOwner->GetRootComponent());
				ISMC->SetCanEverAffectNavigation(false);
				ISMC->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				ISMC->SetCastShadow(MeshDesc.bCastShadows);
				ISMC->Mobility = MeshDesc.Mobility;
				ISMC->SetReceivesDecals(false);
				ISMC->RegisterComponent();

				if (SharedData == nullptr)
				{
					SharedData = &ISMCSharedData.Add(ISMC, FMassISMCSharedData(ISMC, MeshDesc.bRequiresExternalInstanceIDTracking, MeshDesc.LocalTransform));
				}
				else
				{
					SharedData->SetISMComponent(*ISMC);
				}

				ensureMsgf(ISMComponentMap.Find(ISMC) == nullptr, TEXT("We've just created the ISMC that's being used here, so this check failing indicates hash-clash."));
				ISMComponentMap.Add(ISMC, VisualDescHandle); 
			}

			TransientISMCs.Add(ISMC);

			check(SharedData);
			Info.AddISMComponent(*SharedData);
		}

		// Build the LOD significance ranges
		if (TransientISMCs.Num())
		{
			check(Info.LODSignificanceRanges.Num() == 0);
			BuildLODSignificanceForInfo(Info, TransientISMCs);
		}
	}
}

void UMassVisualizationComponent::BuildLODSignificanceForInfo(FMassInstancedStaticMeshInfo& Info, TConstArrayView<UInstancedStaticMeshComponent*> StaticMeshRefKeys)
{
	TArray<float> AllLODSignificances;
	auto UniqueInsertOrdered = [&AllLODSignificances](const float Significance)
	{
		int i = 0;
		for (; i < AllLODSignificances.Num(); ++i)
		{
			// I did not use epsilon check here on purpose, because it will make it hard later meshes inside.
			if (Significance == AllLODSignificances[i])
			{
				return;
			}
			if (AllLODSignificances[i] > Significance)
			{
				break;
			}
		}
		AllLODSignificances.Insert(Significance, i);
	};
	for (const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc : Info.Desc.Meshes)
	{
		UniqueInsertOrdered(MeshDesc.MinLODSignificance);
		UniqueInsertOrdered(MeshDesc.MaxLODSignificance);
	}

	if (AllLODSignificances.Num() > 1)
	{
		Info.LODSignificanceRanges.SetNum(AllLODSignificances.Num() - 1);
		for (int RangeIndex = 0; RangeIndex < Info.LODSignificanceRanges.Num(); ++RangeIndex)
		{
			FMassLODSignificanceRange& Range = Info.LODSignificanceRanges[RangeIndex];
			Range.MinSignificance = AllLODSignificances[RangeIndex];
			Range.MaxSignificance = AllLODSignificances[RangeIndex + 1];
			Range.ISMCSharedDataPtr = &ISMCSharedData;

			for (int MeshIndex = 0; MeshIndex < Info.Desc.Meshes.Num(); ++MeshIndex)
			{
				const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc = Info.Desc.Meshes[MeshIndex];
				const bool bAddMeshInRange = (Range.MinSignificance >= MeshDesc.MinLODSignificance && Range.MinSignificance < MeshDesc.MaxLODSignificance);
				if (bAddMeshInRange)
				{
					checkf(StaticMeshRefKeys.IsValidIndex(MeshIndex) && StaticMeshRefKeys[MeshIndex]
						, TEXT("We don't expect receiving null ISMCs at this point"));
					Range.StaticMeshRefs.Add(StaticMeshRefKeys[MeshIndex]);
				}
			}
		}
	}
}

void UMassVisualizationComponent::ClearAllVisualInstances()
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);
	
	for (int32 SharedDataIndex = 0; SharedDataIndex < ISMCSharedData.Num(); ++SharedDataIndex)
	{
		if (UInstancedStaticMeshComponent* InstancedStaticMeshComponent = ISMCSharedData.GetAtIndex(SharedDataIndex).GetMutableISMComponent())
		{
			InstancedStaticMeshComponent->ClearInstances();
			InstancedStaticMeshComponent->DestroyComponent();
		}
	}

	MeshDescToISMCMap.Reset();
	ISMCSharedData.Reset();
	InstancedSMComponentsRequiringConstructing.Reset();
	InstancedStaticMeshInfos.Reset();

	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);

	for (int32 SharedDataIndex = 0; SharedDataIndex < InstancedSkinnedMeshComponentSharedData.Num(); ++SharedDataIndex)
	{
		if (UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = InstancedSkinnedMeshComponentSharedData.GetAtIndex(SharedDataIndex).GetMutableInstancedSkinnedMeshComponent())
		{
			InstancedSkinnedMeshComponent->ClearInstances();
			InstancedSkinnedMeshComponent->DestroyComponent();
		}
	}

	InstancedSkinnedMeshComponentMap.Reset();
	MeshDescToInstancedSkinnedMeshComponentMap.Reset();
	InstancedSkinnedMeshComponentSharedData.Reset();
	InstancedSkinnedMeshComponentsRequiringConstructing.Reset();
	InstancedSkinnedMeshInfos.Reset();
	InstancedSkinnedMeshInfosFreeIndices.Reset();
}

void UMassVisualizationComponent::DirtyVisuals()
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedStaticMeshInfosDetector);
	for (FMassInstancedStaticMeshInfo& Info : InstancedStaticMeshInfos)
	{
		for (UInstancedStaticMeshComponent* InstancedStaticMeshComponent : Info.InstancedStaticMeshComponents)
		{
			InstancedStaticMeshComponent->MarkRenderStateDirty();
		}
	}

	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);
	for (FMassInstancedSkinnedMeshInfo& Info : InstancedSkinnedMeshInfos)
	{
		for (UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent : Info.InstancedSkinnedMeshComponents)
		{
			InstancedSkinnedMeshComponent->MarkRenderStateDirty();
		}
	}
}

void UMassVisualizationComponent::BeginVisualChanges()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassVisualizationComponent BeginVisualChanges")

	// Conditionally construct static mesh components
	if (InstancedSMComponentsRequiringConstructing.Num())
	{
		ConstructStaticMeshComponents();
		InstancedSMComponentsRequiringConstructing.Reset();
	}

	// Conditionally construct skinned mesh components
	if (InstancedSkinnedMeshComponentsRequiringConstructing.Num())
	{
		ConstructSkinnedMeshComponents();
		InstancedSkinnedMeshComponentsRequiringConstructing.Reset();
	}
}

void UMassVisualizationComponent::HandleChangesWithExternalIDTracking(UInstancedStaticMeshComponent& ISMComponent, FMassISMCSharedData& SharedData)
{
	if (SharedData.HasUpdatesToApply() == false)
	{
		// nothing to do here. We most probably were called as the part of the very first tick of this given SharedData
		// since all the SharedData starts off as `dirty`.
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_Mass_VisualizationComponent_HandleChangesWithExternalIDTracking);

	// removing instances first, since this operation is more resilient to duplicates. Plus we make an arbitrary decision 
	// that it's better to have redundant things visible than not seeing required things
	ProcessRemoves(ISMComponent, SharedData, /*bUpdateNavigation=*/false);

	// NOTE: This code path is designed to only perform Adds, never updates so updates are filtered out along with duplicates.
	TArray<FMassEntityHandle>& EntityHandles = SharedData.UpdateInstanceIds;
	if (!EntityHandles.IsEmpty())
	{
		INC_DWORD_STAT_BY(STAT_Mass_VisualizationComponent_InstancesAddedNum, EntityHandles.Num());

		FMassISMCSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityPrimitiveToIdMap();
		TArray<Experimental::FHashElementId> ElementIds;
		ElementIds.SetNumUninitialized(EntityHandles.Num());
		// Filter out all updates & duplicate adds
		for (int32 IDIndex = EntityHandles.Num() - 1; IDIndex >= 0; --IDIndex)
		{
			bool bWasAlreadyInMap = false;
			Experimental::FHashElementId ElementId = SharedIdMap.FindOrAddId(EntityHandles[IDIndex], FPrimitiveInstanceId{INDEX_NONE}, bWasAlreadyInMap);

			if (bWasAlreadyInMap)
			{
				SharedData.RemoveUpdatedInstanceIdsAtSwap(IDIndex);
				ElementIds.RemoveAtSwap(IDIndex);
			}
			else
			{
				ElementIds[IDIndex] = ElementId;
			}
		}

		// it's possible the loop above removed all the data, so we do one last check
		if (!EntityHandles.IsEmpty())
		{
			check(ElementIds.Num() == EntityHandles.Num());

			const TArray<FTransform>& InstanceTransforms = SharedData.GetStaticMeshInstanceTransformsArray();
			TConstArrayView<float> CustomFloatData = SharedData.GetStaticMeshInstanceCustomFloats();
			const int32 NumCustomDataFloatsPerInstance = CustomFloatData.Num() / EntityHandles.Num();

			// if these are the first entities we're adding we need to set NumCustomDataFloats so that the PerInstanceSMCustomData
			// gets populated properly by the AddInstancesById call below
			const int32 StartingCount = ISMComponent.GetNumInstances();
			const bool bInitiallyEmpty = (StartingCount == 0);
			ensure(bInitiallyEmpty || ISMComponent.NumCustomDataFloats == NumCustomDataFloatsPerInstance);
			if (bInitiallyEmpty && NumCustomDataFloatsPerInstance > 0 && ISMComponent.Mobility != EComponentMobility::Static)
			{
				ISMComponent.SetNumCustomDataFloats(NumCustomDataFloatsPerInstance);
			}

			check(EntityHandles.Num() == InstanceTransforms.Num());
			TArray<FPrimitiveInstanceId> NewIds = ISMComponent.AddInstancesById(InstanceTransforms, /*bWorldSpace=*/true, /*bUpdateNavigation =*/bInitiallyEmpty);
			check(EntityHandles.Num() == NewIds.Num());
			int32 EntityHandleToUpdateIndex = StartingCount;
            for (int32 EntityHandleIndex = 0; EntityHandleIndex < EntityHandles.Num(); ++EntityHandleIndex, ++EntityHandleToUpdateIndex)
			{
				SharedIdMap.GetByElementId(ElementIds[EntityHandleIndex]).Value = NewIds[EntityHandleIndex];
			}

			ISMComponent.SetCustomData(StartingCount, StartingCount + EntityHandles.Num() - 1, CustomFloatData);
		}
	}

	if (bNavigationRelevant && ISMComponent.GetInstanceCount() == 0)
	{
		FNavigationSystem::UnregisterComponent(ISMComponent);
	}
}


void UMassVisualizationComponent::ProcessRemoves(UInstancedStaticMeshComponent& ISMComponent, FMassISMCSharedData& SharedData, const bool bUpdateNavigation /*= true*/)
{
	if (!SharedData.GetRemoveInstanceIds().IsEmpty())
	{
		FMassISMCSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityPrimitiveToIdMap();
		INC_DWORD_STAT_BY(STAT_Mass_VisualizationComponent_InstancesRemovedNum, SharedData.GetRemoveInstanceIds().Num());

		TConstArrayView<FMassEntityHandle> EntityHandles = SharedData.GetRemoveInstanceIds();

		TArray<FPrimitiveInstanceId> ISMInstanceIds;
		ISMInstanceIds.Reserve(EntityHandles.Num());
		
		// Translate Mass IDs to ISMC IDs
		for (const FMassEntityHandle MassInstanceId : EntityHandles)
		{
			Experimental::FHashElementId ElementId = SharedIdMap.FindId(MassInstanceId);
			if (ElementId.IsValid())
			{
				FPrimitiveInstanceId InstanceId = SharedIdMap.GetByElementId(ElementId).Value;
				check(InstanceId.IsValid());
				SharedIdMap.RemoveByElementId(ElementId);
				ISMInstanceIds.Add(InstanceId);
			}
		}

		ISMComponent.RemoveInstancesById(ISMInstanceIds, bUpdateNavigation);
	}
}

void UMassVisualizationComponent::EndVisualChanges()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassVisualizationComponent EndVisualChanges")
	SCOPE_CYCLE_COUNTER(STAT_Mass_VisualizationComponent_EndVisualChanges);

#if STATS
	if (UE::Mass::Representation::LastStatsResetFrame != GFrameNumber)
	{
		SET_DWORD_STAT(STAT_Mass_VisualizationComponent_InstancesRemovedNum, 0);
		SET_DWORD_STAT(STAT_Mass_VisualizationComponent_InstancesAddedNum, 0);
		UE::Mass::Representation::LastStatsResetFrame = GFrameNumber;
	}
#endif // STATS

	// Batch update gathered instance transforms
	for (FMassISMCSharedDataMap::FDirtyIterator It(ISMCSharedData); It; ++It)
	{
		FMassISMCSharedData& SharedData = *It;

		UInstancedStaticMeshComponent* ISMComponent = SharedData.GetMutableISMComponent();
		// @todo need to check validity this way since Mass used to rely on the assumption that all the ISM components used were
		// under its control. That's no longer the case, but the system has not been updated to take that into consideration.
		// This is a temporary fix. 
		if (IsValid(ISMComponent))
		{
			ensureMsgf(!Cast<UHierarchicalInstancedStaticMeshComponent>(ISMComponent), TEXT("The UMassVisualizationComponent does not support driving a HISM, since it is not suitable for rapid updates, replace `%s`."), *ISMComponent->GetFullName());

			if (SharedData.RequiresExternalInstanceIDTracking())
			{
				HandleChangesWithExternalIDTracking(*ISMComponent, SharedData);
				It.ClearDirtyFlag();
			}
			else
			{
				// Process all removes.
				ProcessRemoves(*ISMComponent, SharedData);

				const int32 NumCustomDataFloats = SharedData.StaticMeshInstanceCustomFloats.Num() / (FMath::Max(1, SharedData.UpdateInstanceIds.Num()));

				// Ensure InstanceCustomData is passed if NumCustomDataFloats > 0. If it is, also make sure
				// its length is NumCustomDataFloats * InstanceTransforms.Num()
				ensure(NumCustomDataFloats == 0 || (SharedData.StaticMeshInstanceCustomFloats.Num() == NumCustomDataFloats * SharedData.UpdateInstanceIds.Num()));
				ISMComponent->SetNumCustomDataFloats(NumCustomDataFloats);
				TArray<FMassEntityHandle>& EntityHandles = SharedData.UpdateInstanceIds;
				{
					// Loop over all the instances in the update and 
					// 1. Sort the data such that all Adds are last
					// 2. Remove any duplicates (unsure if they may exist)
					FMassISMCSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityPrimitiveToIdMap();
					// Filter out all updates & duplicate adds
					TBitArray<> Unprocessed;
					Unprocessed.SetNum(SharedIdMap.GetMaxIndex(), true);
					// Process interval

					TConstArrayView<FTransform> PrevInstanceTransforms = SharedData.GetStaticMeshInstancePrevTransforms();
					TConstArrayView<FTransform> InstanceTransforms = SharedData.GetStaticMeshInstanceTransformsArray();
					TConstArrayView<float> CustomDataFloats = SharedData.GetStaticMeshInstanceCustomFloats();

					// Enable support for per-instance prev transforms, if it was not already enabled it will copy the current transforms.
					ISMComponent->SetHasPerInstancePrevTransforms(!PrevInstanceTransforms.IsEmpty());

					struct FAddItem
					{
						Experimental::FHashElementId ElementId;
						int32 IDIndex;
					};
					TArray<FAddItem> ToAdd;
					ToAdd.Reserve(EntityHandles.Num());
					for (int32 IDIndex = 0; IDIndex < EntityHandles.Num(); ++IDIndex)
					{
						bool bWasAlreadyInMap = false;
						Experimental::FHashElementId ElementId = SharedIdMap.FindOrAddId(EntityHandles[IDIndex], FPrimitiveInstanceId{INDEX_NONE}, bWasAlreadyInMap);

						// if it was already in the map, it may be a duplicate if we have processed it already
						bool bIsDuplicate = bWasAlreadyInMap && !Unprocessed[ElementId.GetIndex()];
						if (bIsDuplicate)
						{
							continue;
						}

						FPrimitiveInstanceId Id = SharedIdMap.GetByElementId(ElementId).Value;
						if (!Id.IsValid())
						{
							check(!bWasAlreadyInMap);
							ToAdd.Emplace(FAddItem{ElementId, IDIndex});
						}
						else
						{
							ISMComponent->UpdateInstanceTransformById(Id, InstanceTransforms[IDIndex]);
							if (!PrevInstanceTransforms.IsEmpty())
							{
								ISMComponent->SetPreviousTransformById(Id, PrevInstanceTransforms[IDIndex]);
							}
							if (!CustomDataFloats.IsEmpty())
							{
								ISMComponent->SetCustomDataById(Id, MakeArrayView(CustomDataFloats.GetData() + IDIndex * NumCustomDataFloats, NumCustomDataFloats));
							}
						}

						// Make sure we have enough space to track the already processed IDs
						Unprocessed.SetNum(SharedIdMap.GetMaxIndex(), true);
						Unprocessed[ElementId.GetIndex()] = false;
					}
					// Collect unwanted items & remove
					TArray<FPrimitiveInstanceId> RemovedISMInstanceIds;
					RemovedISMInstanceIds.Reserve(Unprocessed.Num());
					{
						for(TConstSetBitIterator<> BitIt(Unprocessed); BitIt; ++BitIt)
						{
							Experimental::FHashElementId ElementId(BitIt.GetIndex());
							if (SharedIdMap.ContainsElementId(ElementId))
							{
								FPrimitiveInstanceId InstanceId = SharedIdMap.GetByElementId(ElementId).Value;
								check(InstanceId.IsValid());
								SharedIdMap.RemoveByElementId(ElementId);
								RemovedISMInstanceIds.Add(InstanceId);
							}
						}		
						ISMComponent->RemoveInstancesById(RemovedISMInstanceIds);
					}
					// Process deferred adds.
					for (FAddItem AddItem : ToAdd)
					{
						FPrimitiveInstanceId Id = ISMComponent->AddInstanceById(InstanceTransforms[AddItem.IDIndex]);
						check(!SharedIdMap.GetByElementId(AddItem.ElementId).Value.IsValid());
						SharedIdMap.GetByElementId(AddItem.ElementId).Value = Id;

						if (!PrevInstanceTransforms.IsEmpty())
						{
							ISMComponent->SetPreviousTransformById(Id, PrevInstanceTransforms[AddItem.IDIndex]);
						}
						if (!CustomDataFloats.IsEmpty())
						{
							ISMComponent->SetCustomDataById(Id, MakeArrayView(CustomDataFloats.GetData() + AddItem.IDIndex * NumCustomDataFloats, NumCustomDataFloats));
						}

					}
					// note that we're not clearing the dirty flag on purpose - these components require constant updates
				}
			}

			// bump the touch counter so that anyone caching data based on contents of this SharedData can detect the change
			++SharedData.ComponentInstanceIdTouchCounter;
		}
		
		SharedData.ResetAccumulatedData();
	}

	// Batch update gathered instance transforms
	for (FMassInstancedSkinnedMeshComponentSharedDataMap::FDirtyIterator It(InstancedSkinnedMeshComponentSharedData); It; ++It)
	{
		FMassInstancedSkinnedMeshComponentSharedData& SharedData = *It;

		UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = SharedData.GetMutableInstancedSkinnedMeshComponent();
		// @todo need to check validity this way since Mass used to rely on the assumption that all the ISM components used were
		// under its control. That's no longer the case, but the system has not been updated to take that into consideration.
		// This is a temporary fix. 
		if (IsValid(InstancedSkinnedMeshComponent))
		{
			if (SharedData.RequiresExternalInstanceIDTracking())
			{
				HandleChangesWithExternalIDTrackingForComponent(*InstancedSkinnedMeshComponent, SharedData);
				It.ClearDirtyFlag();
			}
			else
			{
				// Process all removes.
				ProcessRemovesForComponent(*InstancedSkinnedMeshComponent, SharedData);

				const int32 NumCustomDataFloats = SharedData.MeshInstanceCustomFloats.Num() / (FMath::Max(1, SharedData.EntitiesRequiringUpdate.Num()));

				// Ensure InstanceCustomData is passed if NumCustomDataFloats > 0. If it is, also make sure
				// its length is NumCustomDataFloats * InstanceTransforms.Num()
				ensure(NumCustomDataFloats == 0 || (SharedData.MeshInstanceCustomFloats.Num() == NumCustomDataFloats * SharedData.EntitiesRequiringUpdate.Num()));
				InstancedSkinnedMeshComponent->SetNumCustomDataFloats(NumCustomDataFloats);
				TArray<FMassEntityHandle>& EntityHandlesWithPendingUpdates = SharedData.EntitiesRequiringUpdate;
				{
					// Loop over all the instances in the update and 
					// 1. Sort the data such that all Adds are last
					// 2. Remove any duplicates (unsure if they may exist)
					FMassInstancedSkinnedMeshComponentSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityToPrimitiveIdMap();
					FMassInstancedSkinnedMeshComponentSharedData::FEntityToTrackIdMap& TrackMap = SharedData.GetMutableEntityToTrackMap();
					// Filter out all updates & duplicate adds
					TBitArray<> Unprocessed;
					Unprocessed.SetNum(SharedIdMap.GetMaxIndex(), true);
					// Process interval

					TConstArrayView<FTransform> PrevInstanceTransforms = SharedData.GetMeshInstancePrevTransforms();
					TConstArrayView<FTransform> InstanceTransforms = SharedData.GetMeshInstanceTransformsArray();
					TConstArrayView<FAnimSequenceTrackAutoPlayData> InstanceAnimationData = SharedData.GetMeshInstanceAnimationData();
					TConstArrayView<float> CustomDataFloats = SharedData.GetMeshInstanceCustomFloats();

					struct FAddItem
					{
						Experimental::FHashElementId ElementId;
						int32 IDIndex;
					};
					TArray<FAddItem> ToAdd;
					ToAdd.Reserve(EntityHandlesWithPendingUpdates.Num());
					for (int32 IDIndex = 0; IDIndex < EntityHandlesWithPendingUpdates.Num(); ++IDIndex)
					{
						bool bWasAlreadyInMap = false;
						Experimental::FHashElementId ElementId = SharedIdMap.FindOrAddId(EntityHandlesWithPendingUpdates[IDIndex], FPrimitiveInstanceId{ INDEX_NONE }, bWasAlreadyInMap);

						// if it was already in the map, it may be a duplicate if we have processed it already
						bool bIsDuplicate = bWasAlreadyInMap && !Unprocessed[ElementId.GetIndex()];
						if (bIsDuplicate)
						{
							continue;
						}

						FPrimitiveInstanceId Id = SharedIdMap.GetByElementId(ElementId).Value;
						if (!Id.IsValid())
						{
							check(!bWasAlreadyInMap);
							ToAdd.Emplace(FAddItem{ ElementId, IDIndex });
						}
						else
						{
							ensure(InstanceTransforms.IsValidIndex(IDIndex));
							InstancedSkinnedMeshComponent->SetInstanceTransform(Id, InstanceTransforms[IDIndex]);
							if (!PrevInstanceTransforms.IsEmpty())
							{
								InstancedSkinnedMeshComponent->SetInstancePrevTransform(Id, PrevInstanceTransforms[IDIndex]);
							}
							if (InstanceAnimationData.IsValidIndex(IDIndex))
							{
								int DesiredAnimIndex = -1;
								int CurrentAnimIndex = -1;
								if (UAnimSequenceTransformProviderDataInstance* ASTPDI = Cast<UAnimSequenceTransformProviderDataInstance>(InstancedSkinnedMeshComponent->GetTransformProvider()))
								{
									bool bAlreadyPresent = false;
									auto TrackElementId = TrackMap.FindOrAddId(EntityHandlesWithPendingUpdates[IDIndex], 0, bAlreadyPresent);
									int32 TrackId = INDEX_NONE;
									if (bAlreadyPresent)
									{
										TrackId = TrackMap.GetByElementId(TrackElementId).Value;
										if (ASTPDI->GetSequenceIndex(TrackId, 0) != InstanceAnimationData[IDIndex].SequenceIndex)
										{
											ASTPDI->SetAutoPlayData(TrackId, 0, InstanceAnimationData[IDIndex]);
										}
									}
									else
									{
										TrackId = TrackMap.GetByElementId(TrackElementId).Value = ASTPDI->AllocateTrack();
										ASTPDI->SetAutoPlayData(TrackId, 0, InstanceAnimationData[IDIndex]);
									}
							
									// Readback resolved animation state for UAF handoff
									const TArray<FAnimSequenceTransformProviderSequence>& Sequences = ASTPDI->GetSequences();
									FMassSkinnedMeshResolvedAnimState State = {
										.SequenceIndex = ASTPDI->GetSequenceIndex(TrackId, 0),
										.Position = ASTPDI->GetPosition(TrackId, 0).TargetPosition,
										.PlayRate = ASTPDI->GetPlayRate(TrackId, 0),
										.AnimSequence = Sequences.IsValidIndex(State.SequenceIndex) ? Sequences[State.SequenceIndex].Sequence.Get() : nullptr
									};
									bool bAnimStatePresent = false;
									auto AnimStateElementId = ResolvedAnimStates.FindOrAddId(EntityHandlesWithPendingUpdates[IDIndex], FMassSkinnedMeshResolvedAnimState(), bAnimStatePresent);
									ResolvedAnimStates.GetByElementId(AnimStateElementId).Value = State;
							
									// Set the anim index on the instance to be the trackId
									DesiredAnimIndex = TrackId;
								}
								else
								{
									DesiredAnimIndex = InstanceAnimationData[IDIndex].SequenceIndex;
								}
							
								if (InstancedSkinnedMeshComponent->GetInstanceAnimationIndex(Id,CurrentAnimIndex) && CurrentAnimIndex != DesiredAnimIndex)
								{
									InstancedSkinnedMeshComponent->SetInstanceAnimationIndex(Id, DesiredAnimIndex);
								}
							}
							if (!CustomDataFloats.IsEmpty())
							{
								InstancedSkinnedMeshComponent->SetCustomData(Id, MakeArrayView(CustomDataFloats.GetData() + IDIndex * NumCustomDataFloats, NumCustomDataFloats));
							}
						}

						// Make sure we have enough space to track the already processed IDs
						Unprocessed.SetNum(SharedIdMap.GetMaxIndex(), true);
						Unprocessed[ElementId.GetIndex()] = false;
					}
					// Collect unwanted items & remove
					TArray<FPrimitiveInstanceId> RemovedInstancedSkinnedMeshInstanceIds;
					RemovedInstancedSkinnedMeshInstanceIds.Reserve(Unprocessed.Num());
					{
						for (TConstSetBitIterator<> BitIt(Unprocessed); BitIt; ++BitIt)
						{
							Experimental::FHashElementId ElementId(BitIt.GetIndex());
							if (SharedIdMap.ContainsElementId(ElementId))
							{
								FPrimitiveInstanceId InstanceId = SharedIdMap.GetByElementId(ElementId).Value;
								check(InstanceId.IsValid());
								SharedIdMap.RemoveByElementId(ElementId);
								RemovedInstancedSkinnedMeshInstanceIds.Add(InstanceId);
							}
						}
						InstancedSkinnedMeshComponent->RemoveInstances(RemovedInstancedSkinnedMeshInstanceIds);
					}
					// Process deferred adds.
					for (FAddItem AddItem : ToAdd)
					{
						FPrimitiveInstanceId Id = InstancedSkinnedMeshComponent->AddInstance(InstanceTransforms[AddItem.IDIndex], 0);
						check(!SharedIdMap.GetByElementId(AddItem.ElementId).Value.IsValid());
						SharedIdMap.GetByElementId(AddItem.ElementId).Value = Id;

						if (!PrevInstanceTransforms.IsEmpty())
						{
							InstancedSkinnedMeshComponent->SetInstancePrevTransform(Id, PrevInstanceTransforms[AddItem.IDIndex]);
						}
						if (!CustomDataFloats.IsEmpty())
						{
							InstancedSkinnedMeshComponent->SetCustomData(Id, MakeArrayView(CustomDataFloats.GetData() + AddItem.IDIndex * NumCustomDataFloats, NumCustomDataFloats));
						}
						if (InstanceAnimationData.IsValidIndex(AddItem.IDIndex))
						{
							int DesiredAnimIndex = -1;
							int CurrentAnimIndex = -1;
							auto EntityHandle = SharedIdMap.GetByElementId(AddItem.ElementId).Key;
							if (UAnimSequenceTransformProviderDataInstance* ASTPDI = Cast<UAnimSequenceTransformProviderDataInstance>(InstancedSkinnedMeshComponent->GetTransformProvider()))
							{
								bool bAlreadyPresent = false;
								auto TrackElementId = TrackMap.FindOrAddId(EntityHandle, 0, bAlreadyPresent);
								int32 TrackId = INDEX_NONE;
								if (bAlreadyPresent)
								{
									TrackId = TrackMap.GetByElementId(TrackElementId).Value;
									if (ASTPDI->GetSequenceIndex(TrackId, 0) != InstanceAnimationData[AddItem.IDIndex].SequenceIndex)
									{
										ASTPDI->SetAutoPlayData(TrackId, 0, InstanceAnimationData[AddItem.IDIndex]);
									}
								}
								else
								{
									TrackId = TrackMap.GetByElementId(TrackElementId).Value = ASTPDI->AllocateTrack();
									ASTPDI->SetAutoPlayData(TrackId, 0, InstanceAnimationData[AddItem.IDIndex]);
								}
						
								// Readback resolved animation state for UAF handoff
								const TArray<FAnimSequenceTransformProviderSequence>& Sequences = ASTPDI->GetSequences();
								FMassSkinnedMeshResolvedAnimState State = {
									.SequenceIndex = ASTPDI->GetSequenceIndex(TrackId, 0),
									.Position = ASTPDI->GetPosition(TrackId, 0).TargetPosition,
									.PlayRate = ASTPDI->GetPlayRate(TrackId, 0),
									.AnimSequence = Sequences.IsValidIndex(State.SequenceIndex) ? Sequences[State.SequenceIndex].Sequence.Get() : nullptr
								};
								bool bAnimStatePresent = false;
								auto AnimStateElementId = ResolvedAnimStates.FindOrAddId(EntityHandlesWithPendingUpdates[AddItem.IDIndex], FMassSkinnedMeshResolvedAnimState(), bAnimStatePresent);
								ResolvedAnimStates.GetByElementId(AnimStateElementId).Value = State;
						
								// Set the anim index on the instance to be the trackId
								DesiredAnimIndex = TrackId;
							}
							else
							{
								DesiredAnimIndex = InstanceAnimationData[AddItem.IDIndex].SequenceIndex;
							}
						
							if (InstancedSkinnedMeshComponent->GetInstanceAnimationIndex(Id,CurrentAnimIndex) && CurrentAnimIndex != DesiredAnimIndex)
							{
								InstancedSkinnedMeshComponent->SetInstanceAnimationIndex(Id, DesiredAnimIndex);
							}
						}
					}
					// note that we're not clearing the dirty flag on purpose - these components require constant updates
				}
			}

			// bump the touch counter so that anyone caching data based on contents of this SharedData can detect the change
			++SharedData.ComponentInstanceIdTouchCounter;
		}
		SharedData.ResetAccumulatedData();
	}
}

//---------------------------------------------------------------
// FMassLODSignificanceRange
//---------------------------------------------------------------

void FMassLODSignificanceRange::AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, TConstArrayView<FISMCSharedDataKey> ExcludeStaticMeshRefs)
{
	check(ISMCSharedDataPtr);
	for (int32 StaticMeshIndex = 0; StaticMeshIndex < StaticMeshRefs.Num(); ++StaticMeshIndex)
	{
		if (ExcludeStaticMeshRefs.Contains(StaticMeshRefs[StaticMeshIndex]))
		{
			continue;
		}

		if (FMassISMCSharedData* SharedData = ISMCSharedDataPtr->GetAndMarkDirty(StaticMeshRefs[StaticMeshIndex]))
		{
			SharedData->UpdateInstanceIds.Add(EntityHandle);

			SharedData->StaticMeshInstanceTransforms.Add(SharedData->LocalTransform * Transform);
			SharedData->StaticMeshInstancePrevTransforms.Add(SharedData->LocalTransform * PrevTransform);
		}
	}
}

void FMassLODSignificanceRange::AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const TArray<FISMCSharedDataKey>& ExcludeStaticMeshRefs)
{
	check(ISMCSharedDataPtr);
	for (int32 StaticMeshIndex = 0; StaticMeshIndex < StaticMeshRefs.Num(); ++StaticMeshIndex)
	{
		if (ExcludeStaticMeshRefs.Contains(StaticMeshRefs[StaticMeshIndex]))
		{
			continue;
		}

		if (FMassISMCSharedData* SharedData = ISMCSharedDataPtr->GetAndMarkDirty(StaticMeshRefs[StaticMeshIndex]))
		{
			SharedData->StaticMeshInstanceCustomFloats.Append(CustomFloats);
		}
	}
}

void FMassLODSignificanceRange::AddInstance(const FMassEntityHandle EntityHandle, const FTransform& Transform)
{
	check(ISMCSharedDataPtr);
	for (int32 StaticMeshIndex = 0; StaticMeshIndex < StaticMeshRefs.Num(); ++StaticMeshIndex)
	{
		if (FMassISMCSharedData* SharedData = ISMCSharedDataPtr->GetAndMarkDirty(StaticMeshRefs[StaticMeshIndex]))
		{
			SharedData->UpdateInstanceIds.Add(EntityHandle);

			FTransform AdjustedTransform{SharedData->LocalTransform * Transform};
			
			SharedData->StaticMeshInstanceTransforms.Add(AdjustedTransform);
			SharedData->StaticMeshInstancePrevTransforms.Add(AdjustedTransform);
		}
	}
}

void FMassLODSignificanceRange::RemoveInstance(const FMassEntityHandle EntityHandle)
{
	check(ISMCSharedDataPtr);
	for (int32 StaticMeshIndex = 0; StaticMeshIndex < StaticMeshRefs.Num(); ++StaticMeshIndex)
	{
		if (FMassISMCSharedData* SharedData = ISMCSharedDataPtr->GetAndMarkDirty(StaticMeshRefs[StaticMeshIndex]))
		{
			SharedData->RemoveInstanceIds.Add(EntityHandle);
		}
	}
}

void FMassLODSignificanceRange::WriteCustomDataFloatsAtStartIndex(int32 StaticMeshIndex, const TArrayView<float>& CustomFloats, const int32 FloatsPerInstance, const int32 StartFloatIndex, const TArray<FISMCSharedDataKey>& ExcludeStaticMeshRefs)
{
	check(ISMCSharedDataPtr);
	if (StaticMeshRefs.IsValidIndex(StaticMeshIndex))
	{
		if (ExcludeStaticMeshRefs.Contains(StaticMeshRefs[StaticMeshIndex]))
		{
			return;
		}

		if (FMassISMCSharedData* SharedData = ISMCSharedDataPtr->GetAndMarkDirty(StaticMeshRefs[StaticMeshIndex]))
		{
			const int32 StartIndex = FloatsPerInstance * SharedData->WriteIterator + StartFloatIndex;

			ensure(SharedData->StaticMeshInstanceCustomFloats.Num() >= StartIndex + CustomFloats.Num());

			for (int CustomFloatIdx = 0; CustomFloatIdx < CustomFloats.Num(); CustomFloatIdx++)
			{
				SharedData->StaticMeshInstanceCustomFloats[StartIndex + CustomFloatIdx] = CustomFloats[CustomFloatIdx];
			}
			SharedData->WriteIterator++;
		}
	}
}

FSkinnedMeshInstanceVisualizationDescHandle UMassVisualizationComponent::FindOrAddSkinnedMeshInstanceVisualDesc(const FSkinnedMeshInstanceVisualizationDesc& Desc)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);
	// First check to see if we already have a matching Desc already and reuse / return that
	// Note: FSkinnedMeshInstanceVisualizationDescHandle(int32) handles the INDEX_NONE case here, generating an invalid handle in this case
	FSkinnedMeshInstanceVisualizationDescHandle VisualDescHandle(InstancedSkinnedMeshInfos.IndexOfByPredicate([&Desc](const FMassInstancedSkinnedMeshInfo& Info) { return Info.GetDesc() == Desc; }));
	if (!VisualDescHandle.IsValid())
	{
		if (Desc.IsValid())
		{
			for (const FMassSkinnedMeshInstanceVisualizationMeshDesc& MeshDesc : Desc.Meshes)
			{
				if (MeshDesc.Asset && MeshDesc.InstancedSkinnedMeshComponentClass)
				{
					// if we've already encountered MeshDesc in the past MeshDescToInstancedSkinnedMeshComponentMap already contains information
					// about actual Instanced Mesh Component used to represent it, and at the same time indicates the SharedData data
					// tied to it. Regardless we need to process all MeshDesc instances here so that we have all the 
					// data ready when InstancedSkinnedMeshComponentsRequiringConstructing gets processed next time
					// UMassVisualizationComponent::ConstructSkinnedMeshComponents gets called.
					MeshDescToInstancedSkinnedMeshComponentMap.FindOrAdd(GetTypeHash(MeshDesc), FInstancedSkinnedMeshComponentSharedDataKey());
				}
			}

			VisualDescHandle = AddInstancedSkinnedMeshInfo(Desc);
			check(VisualDescHandle.IsValid());

			// VisualDescHandle is a valid handle now, but there's initialization pending, performed in ConstructSkinnedMeshComponents
			InstancedSkinnedMeshComponentsRequiringConstructing.Add(VisualDescHandle);
		}
		else
		{
			UE_LOGF(LogMassRepresentation, Warning, "%s: invalid FSkinnedMeshInstanceVisualizationDesc passed in. Check the contained meshes.", __FUNCTION__);
		}
	}

	return VisualDescHandle;
}

FSkinnedMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddSkinnedMeshInstanceVisualDescWithComponent(const FSkinnedMeshInstanceVisualizationDesc& Desc, UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent)
{

	TObjectPtr<UInstancedSkinnedMeshComponent> AsObjectPtr = &InstancedSkinnedMeshComponent;
	return AddSkinnedMeshInstanceVisualDescWithComponents(Desc, MakeArrayView(&AsObjectPtr, 1));
}

FSkinnedMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddSkinnedMeshInstanceVisualDescWithComponents(const FSkinnedMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedSkinnedMeshComponent>> InstancedSkinnedMeshComponents)
{
	check(Desc.Meshes.Num() == InstancedSkinnedMeshComponents.Num());

	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);

	// 0. Iterate over all meshes in the visualization desc. Each mesh is a descriptor.
	FSkinnedMeshInstanceVisualizationDescHandle VisualHandle;

	TArray<UInstancedSkinnedMeshComponent*> InstancedSkinnedMeshComponentsUsed;
	for (int32 EntryIndex = 0; EntryIndex < Desc.Meshes.Num(); ++EntryIndex)
	{
		const FMassSkinnedMeshInstanceVisualizationMeshDesc& MeshDesc = Desc.Meshes[EntryIndex];
		if (MeshDesc.Asset == nullptr || InstancedSkinnedMeshComponents[EntryIndex] == nullptr)
		{
			// invalid description, log an continue.
			UE_VLOG_UELOG(this, LogMassRepresentation, Error, TEXT("Empty mesh at index %d while registering FSkinnedMeshInstanceVisualizationDesc instance"), EntryIndex);
			continue;
		}

		// 1. Creates a VisualHandle that will be used for all InstancedSkinnedMeshComponents
		if (!VisualHandle.IsValid())
		{
			VisualHandle = AddInstancedSkinnedMeshInfo(Desc);
			check(VisualHandle.IsValid());
		}

		// 2. Stores one FMassInstancedSkinnedMeshComponentSharedData for each InstancedSkinnedMeshComponent in FMassInstancedSkinnedMeshComponentSharedData
		// NOTE: FMassISMCSharedData stores a reference to an ISMC and instance transform updates performed in the current frame.
		FMassInstancedSkinnedMeshComponentSharedData& NewData = InstancedSkinnedMeshComponentSharedData.FindOrAdd(InstancedSkinnedMeshComponents[EntryIndex], FMassInstancedSkinnedMeshComponentSharedData(InstancedSkinnedMeshComponents[EntryIndex], /*bInRequiresExternalInstanceIDTracking=*/true, /*InTransformOffset=*/MeshDesc.LocalTransform));
		// 3. Stores the newly created FMassInstancedSkinnedMeshComponentSharedData in a VisualHandle -> FMassInstancedSkinnedMeshComponentSharedData array
		// NOTE: This means that one VisualHandle may reference multiple FMassInstancedSkinnedMeshComponentSharedData
		InstancedSkinnedMeshInfos[VisualHandle.ToIndex()].AddInstancedSkinnedMeshComponent(NewData);
		// 4. Stores an InstancedMeshComponent -> VisualHandle map, which allows us to later on use an InstancedMeshComponent to query for its FMassInstancedSkinnedMeshComponentSharedData
		InstancedSkinnedMeshComponentMap.Add(InstancedSkinnedMeshComponents[EntryIndex], VisualHandle);

		InstancedSkinnedMeshComponentsUsed.Add(InstancedSkinnedMeshComponents[EntryIndex]);
	}
	if (VisualHandle.IsValid())
	{
		BuildLODSignificanceForInstancedSkinnedMeshInfo(InstancedSkinnedMeshInfos[VisualHandle.ToIndex()], InstancedSkinnedMeshComponentsUsed);
	}

	return VisualHandle;
}

const FMassInstancedSkinnedMeshComponentSharedData* UMassVisualizationComponent::GetSharedDataForDescriptionIndex(const int32 DescriptionIndex) const
{
	return InstancedSkinnedMeshComponentSharedData.GetDataForIndex(DescriptionIndex);
}

const FMassInstancedSkinnedMeshComponentSharedData* UMassVisualizationComponent::GetSharedDataForInstancedSkinnedMeshComponent(const UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent) const
{
	return InstancedSkinnedMeshComponentSharedData.GetDataForKey(InstancedSkinnedMeshComponent);
}

void UMassVisualizationComponent::RemoveSkinnedMeshInstanceVisualDesc(const FSkinnedMeshInstanceVisualizationDescHandle VisualizationHandle)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);

	if (ensure(InstancedSkinnedMeshInfos.IsValidIndex(VisualizationHandle.ToIndex()))
		&& ensureMsgf(InstancedSkinnedMeshInfos[VisualizationHandle.ToIndex()].IsValid(), TEXT("Trying to remove visualization data that has already been cleaned")))
	{
		for (TObjectPtr<UInstancedSkinnedMeshComponent>& InstancedSkinnedMeshComponent : InstancedSkinnedMeshInfos[VisualizationHandle.ToIndex()].InstancedSkinnedMeshComponents)
		{
			const bool bValidKey = InstancedSkinnedMeshComponentMap.Contains(InstancedSkinnedMeshComponent);
			checkf(bValidKey, TEXT("Failed to find Instanced Mesh in InstancedSkinnedMeshComponentMap, path: %s"), *InstancedSkinnedMeshComponent.GetPathName());
			if (bValidKey)
			{
				const FSkinnedMeshInstanceVisualizationDescHandle StoredVisualizationDescHandle = InstancedSkinnedMeshComponentMap.FindAndRemoveChecked(InstancedSkinnedMeshComponent);
				ensure(StoredVisualizationDescHandle == VisualizationHandle);
			}

			InstancedSkinnedMeshComponentSharedData.Remove(InstancedSkinnedMeshComponent);
		}

		InstancedSkinnedMeshInfos[VisualizationHandle.ToIndex()].Reset();
		InstancedSkinnedMeshInfosFreeIndices.Add(VisualizationHandle);
	}
}

void UMassVisualizationComponent::ProcessRemovesForComponent(UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent, FMassInstancedSkinnedMeshComponentSharedData& SharedData)
{
	if (!SharedData.GetEntitiesRequiringRemoval().IsEmpty())
	{
		FMassInstancedSkinnedMeshComponentSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityToPrimitiveIdMap();
		FMassInstancedSkinnedMeshComponentSharedData::FEntityToTrackIdMap& SharedTrackMap = SharedData.GetMutableEntityToTrackMap();

		INC_DWORD_STAT_BY(STAT_Mass_VisualizationComponent_InstancesRemovedNum, SharedData.GetEntitiesRequiringRemoval().Num());

		TConstArrayView<FMassEntityHandle> EntityHandles = SharedData.GetEntitiesRequiringRemoval();

		TArray<FPrimitiveInstanceId> InstancedSkinnedMeshInstanceIds;
		InstancedSkinnedMeshInstanceIds.Reserve(EntityHandles.Num());

		// Translate Mass IDs to Instanced Mesh IDs
		for (const FMassEntityHandle MassInstanceId : EntityHandles)
		{
			Experimental::FHashElementId ElementId = SharedIdMap.FindId(MassInstanceId);
			if (ElementId.IsValid())
			{
				FPrimitiveInstanceId InstanceId = SharedIdMap.GetByElementId(ElementId).Value;
				check(InstanceId.IsValid());
				SharedIdMap.RemoveByElementId(ElementId);
				InstancedSkinnedMeshInstanceIds.Add(InstanceId);
			}
			Experimental::FHashElementId TrackElementId = SharedTrackMap.FindId(MassInstanceId);
			if (TrackElementId.IsValid())
			{
				if (UAnimSequenceTransformProviderDataInstance* ASTPDI = Cast<UAnimSequenceTransformProviderDataInstance>(InstancedSkinnedMeshComponent.GetTransformProvider()))
				{
					ASTPDI->DeallocateTrack(SharedTrackMap.GetByElementId(TrackElementId).Value);
				}
				SharedTrackMap.RemoveByElementId(TrackElementId);

				// Clean up resolved anim state
				Experimental::FHashElementId AnimStateId = ResolvedAnimStates.FindId(MassInstanceId);
				if (AnimStateId.IsValid())
				{
					ResolvedAnimStates.RemoveByElementId(AnimStateId);
				}
			}
		}

		InstancedSkinnedMeshComponent.RemoveInstances(InstancedSkinnedMeshInstanceIds);
	}
}

void UMassVisualizationComponent::HandleChangesWithExternalIDTrackingForComponent(UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent, FMassInstancedSkinnedMeshComponentSharedData& SharedData)
{
	if (SharedData.HasUpdatesToApply() == false)
	{
		// nothing to do here. We most probably were called as the part of the very first tick of this given SharedData
		// since all the SharedData starts off as `dirty`.
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_Mass_VisualizationComponent_HandleChangesWithExternalIDTracking);

	// removing instances first, since this operation is more resilient to duplicates. Plus we make an arbitrary decision 
	// that it's better to have redundant things visible than not seeing required things
	ProcessRemovesForComponent(InstancedSkinnedMeshComponent, SharedData);

	// NOTE: This code path is designed to only perform Adds, never updates so updates are filtered out along with duplicates.
	TArray<FMassEntityHandle>& EntityHandles = SharedData.EntitiesRequiringUpdate;
	if (!EntityHandles.IsEmpty())
	{
		INC_DWORD_STAT_BY(STAT_Mass_VisualizationComponent_InstancesAddedNum, EntityHandles.Num());

		FMassISMCSharedData::FEntityToPrimitiveIdMap& SharedIdMap = SharedData.GetMutableEntityToPrimitiveIdMap();
		TArray<Experimental::FHashElementId> ElementIds;
		ElementIds.SetNumUninitialized(EntityHandles.Num());
		// Filter out all updates & duplicate adds
		for (int32 IDIndex = EntityHandles.Num() - 1; IDIndex >= 0; --IDIndex)
		{
			bool bWasAlreadyInMap = false;
			Experimental::FHashElementId ElementId = SharedIdMap.FindOrAddId(EntityHandles[IDIndex], FPrimitiveInstanceId{ INDEX_NONE }, bWasAlreadyInMap);

			if (bWasAlreadyInMap)
			{
				SharedData.RemoveUpdatedInstanceIdsAtSwap(IDIndex);
				ElementIds.RemoveAtSwap(IDIndex);
			}
			else
			{
				ElementIds[IDIndex] = ElementId;
			}
		}

		// it's possible the loop above removed all the data, so we do one last check
		if (!EntityHandles.IsEmpty())
		{
			check(ElementIds.Num() == EntityHandles.Num());

			const TArray<FTransform>& InstanceTransforms = SharedData.GetMeshInstanceTransformsArray();
			
			TArray<int32> InstanceAnimationIndices;
			InstanceAnimationIndices.Init(0, InstanceTransforms.Num());

			TConstArrayView<float> CustomFloatData = SharedData.GetMeshInstanceCustomFloats();
			const int32 NumCustomDataFloatsPerInstance = CustomFloatData.Num() / EntityHandles.Num();

			// if these are the first entities we're adding we need to set NumCustomDataFloats so that the PerInstanceSMCustomData
			// gets populated properly by the AddInstancesById call below
			const int32 StartingCount = InstancedSkinnedMeshComponent.GetInstanceCount();
			const bool bInitiallyEmpty = (StartingCount == 0);
			ensure(bInitiallyEmpty || InstancedSkinnedMeshComponent.GetNumCustomDataFloats() == NumCustomDataFloatsPerInstance);
			if (bInitiallyEmpty && NumCustomDataFloatsPerInstance > 0 && InstancedSkinnedMeshComponent.Mobility != EComponentMobility::Static)
			{
				InstancedSkinnedMeshComponent.SetNumCustomDataFloats(NumCustomDataFloatsPerInstance);
			}

			check(EntityHandles.Num() == InstanceTransforms.Num());
			TArray<FPrimitiveInstanceId> NewIds = InstancedSkinnedMeshComponent.AddInstances(InstanceTransforms, InstanceAnimationIndices, /*bShouldReturnIds=*/true, /*bWorldSpace=*/true);
			check(EntityHandles.Num() == NewIds.Num());
			int32 EntityHandleToUpdateIndex = StartingCount;
			for (int32 EntityHandleIndex = 0; EntityHandleIndex < EntityHandles.Num(); ++EntityHandleIndex, ++EntityHandleToUpdateIndex)
			{
				SharedIdMap.GetByElementId(ElementIds[EntityHandleIndex]).Value = NewIds[EntityHandleIndex];	
				
				if (!CustomFloatData.IsEmpty())
				{
					InstancedSkinnedMeshComponent.SetCustomData(NewIds[EntityHandleIndex], MakeArrayView(CustomFloatData.GetData() + EntityHandleIndex * NumCustomDataFloatsPerInstance, NumCustomDataFloatsPerInstance));
				}
			}
		}
	}

	if (bNavigationRelevant && InstancedSkinnedMeshComponent.GetInstanceCount() == 0)
	{
		FNavigationSystem::UnregisterComponent(InstancedSkinnedMeshComponent);
	}
}

void UMassVisualizationComponent::ConstructSkinnedMeshComponents()
{
	AActor* ActorOwner = GetOwner();
	check(ActorOwner);

	TArray<UInstancedSkinnedMeshComponent*> TransientInstancedSkinnedMeshComponents;

	UE_MT_SCOPED_WRITE_ACCESS(InstancedSkinnedMeshInfosDetector);
	for (const FSkinnedMeshInstanceVisualizationDescHandle VisualDescHandle : InstancedSkinnedMeshComponentsRequiringConstructing)
	{
		if (!ensureMsgf(InstancedSkinnedMeshInfos.IsValidIndex(VisualDescHandle.ToIndex())
			, TEXT("InstancedSkinnedMeshInfos (size: %d) is never expected to shrink, so VisualDescHandle (value: %u) being invalid indicates it was wrong from the start.")
			, InstancedSkinnedMeshInfos.Num(), VisualDescHandle.ToIndex()))
		{
			continue;
		}

		FMassInstancedSkinnedMeshInfo& Info = InstancedSkinnedMeshInfos[VisualDescHandle.ToIndex()];

		// Check if it is already created
		if (!Info.InstancedSkinnedMeshComponents.IsEmpty())
		{
			continue;
		}

		// Check if there are any specified meshes for this visual type
		if (Info.Desc.Meshes.Num() == 0)
		{
			UE_LOGF(LogMassRepresentation, Error, "No associated meshes for this instanced static mesh type");
			continue;
		}

		TransientInstancedSkinnedMeshComponents.Reset();
		for (const FMassSkinnedMeshInstanceVisualizationMeshDesc& InstancedSkinnedMeshDesc : Info.Desc.Meshes)
		{
			// MeshDescToInstancedSkinnedMeshComponentMap here lets us figure out whether for the given InstancedSkinnedMeshDesc we need to create a new Instanced Mesh component
			// or a one has already been created in the past. Note that we only need this intermediate map for 
			// FMassSkinnedMeshInstanceVisualizationMeshDesc that has been added to the system without specifying an
			// ISM component to handle the instances (i.e. added via FindOrAddVisualDesc rather than AddVisualDescWithISMComponents).
			// This is the only kind of FMassSkinnedMeshInstanceVisualizationMeshDesc were processing here. 
			FInstancedSkinnedMeshComponentSharedDataKey& InstancedSkinnedMeshComponentKey = MeshDescToInstancedSkinnedMeshComponentMap.FindChecked(GetTypeHash(InstancedSkinnedMeshDesc));
			FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshComponentSharedData.Find(InstancedSkinnedMeshComponentKey);
			UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = SharedData ? SharedData->GetMutableInstancedSkinnedMeshComponent() : nullptr;

			if (InstancedSkinnedMeshComponent == nullptr)
			{
				InstancedSkinnedMeshComponent = NewObject<UInstancedSkinnedMeshComponent>(ActorOwner, InstancedSkinnedMeshDesc.InstancedSkinnedMeshComponentClass);
				CA_ASSUME(InstancedSkinnedMeshComponent);
				REDIRECT_OBJECT_TO_VLOG(InstancedSkinnedMeshComponent, this);

				// note that InstancedSkinnedMeshComponentKey is a reference, so the assignment below actually sets a value in MeshDescToInstancedSkinnedMeshComponentMap
				// and all subsequent handling of a given InstancedSkinnedMeshDesc configuration (i.e. containing same values) will 
				// result in referring to the InstancedSkinnedMeshComponent we just created.
				InstancedSkinnedMeshComponentKey = InstancedSkinnedMeshComponent;

				InstancedSkinnedMeshComponent->SetSkinnedAssetAndUpdate(InstancedSkinnedMeshDesc.Asset);
				for (int32 ElementIndex = 0; ElementIndex < InstancedSkinnedMeshDesc.MaterialOverrides.Num(); ++ElementIndex)
				{
					if (UMaterialInterface* MaterialOverride = InstancedSkinnedMeshDesc.MaterialOverrides[ElementIndex])
					{
						InstancedSkinnedMeshComponent->SetMaterial(ElementIndex, MaterialOverride);
					}
				}

				if (UAnimSequenceTransformProviderData* ASTPD = Cast<UAnimSequenceTransformProviderData>(InstancedSkinnedMeshDesc.TransformProvider))
				{
					InstancedSkinnedMeshComponent->SetTransformProvider(UAnimSequenceTransformProviderDataInstance::CreateAnimSequenceTransformProviderDataInstance(ASTPD, InstancedSkinnedMeshComponent));
				}
				else
				{
					InstancedSkinnedMeshComponent->SetTransformProvider(InstancedSkinnedMeshDesc.TransformProvider);
				}

				InstancedSkinnedMeshComponent->SetCullDistances(0, 1000000); // @todo: Need to figure out what to do here, either LOD or cull distances.
				InstancedSkinnedMeshComponent->SetAnimationMinScreenSize(InstancedSkinnedMeshDesc.AnimationMinScreenSize);
				InstancedSkinnedMeshComponent->SetupAttachment(ActorOwner->GetRootComponent());
				InstancedSkinnedMeshComponent->SetCanEverAffectNavigation(false);
				InstancedSkinnedMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				InstancedSkinnedMeshComponent->SetCastShadow(InstancedSkinnedMeshDesc.bCastShadows);
				InstancedSkinnedMeshComponent->Mobility = InstancedSkinnedMeshDesc.Mobility;
				InstancedSkinnedMeshComponent->SetReceivesDecals(false);
				InstancedSkinnedMeshComponent->SetHasPerInstancePrevTransforms(true);
				InstancedSkinnedMeshComponent->RegisterComponent();

				if (SharedData == nullptr)
				{
					SharedData = &InstancedSkinnedMeshComponentSharedData.Add(InstancedSkinnedMeshComponent, FMassInstancedSkinnedMeshComponentSharedData(InstancedSkinnedMeshComponent, InstancedSkinnedMeshDesc.bRequiresExternalInstanceIDTracking, InstancedSkinnedMeshDesc.LocalTransform));
				}
				else
				{
					SharedData->SetInstancedSkinnedMeshComponent(*InstancedSkinnedMeshComponent);
				}

				ensureMsgf(InstancedSkinnedMeshComponentMap.Find(InstancedSkinnedMeshComponent) == nullptr, TEXT("We've just created the ISMC that's being used here, so this check failing indicates hash-clash."));
				InstancedSkinnedMeshComponentMap.Add(InstancedSkinnedMeshComponent, VisualDescHandle);
			}

			TransientInstancedSkinnedMeshComponents.Add(InstancedSkinnedMeshComponent);

			check(SharedData);
			Info.AddInstancedSkinnedMeshComponent(*SharedData);
		}

		// Build the LOD significance ranges
		if (TransientInstancedSkinnedMeshComponents.Num())
		{
			check(Info.LODSignificanceRanges.Num() == 0);
			BuildLODSignificanceForInstancedSkinnedMeshInfo(Info, TransientInstancedSkinnedMeshComponents);
		}
	}
}

void UMassVisualizationComponent::BuildLODSignificanceForInstancedSkinnedMeshInfo(FMassInstancedSkinnedMeshInfo& Info, TConstArrayView<UInstancedSkinnedMeshComponent*> MeshRefKeys)
{
	TArray<float> AllLODSignificances;
	auto UniqueInsertOrdered = [&AllLODSignificances](const float Significance)
		{
			int i = 0;
			for (; i < AllLODSignificances.Num(); ++i)
			{
				// I did not use epsilon check here on purpose, because it will make it hard later meshes inside.
				if (Significance == AllLODSignificances[i])
				{
					return;
				}
				if (AllLODSignificances[i] > Significance)
				{
					break;
				}
			}
			AllLODSignificances.Insert(Significance, i);
		};
	for (const FMassSkinnedMeshInstanceVisualizationMeshDesc& MeshDesc : Info.Desc.Meshes)
	{
		UniqueInsertOrdered(MeshDesc.MinLODSignificance);
		UniqueInsertOrdered(MeshDesc.MaxLODSignificance);
	}

	if (AllLODSignificances.Num() > 1)
	{
		Info.LODSignificanceRanges.SetNum(AllLODSignificances.Num() - 1);
		for (int RangeIndex = 0; RangeIndex < Info.LODSignificanceRanges.Num(); ++RangeIndex)
		{
			FMassLODInstancedSkinnedMeshSignificanceRange& Range = Info.LODSignificanceRanges[RangeIndex];
			Range.MinSignificance = AllLODSignificances[RangeIndex];
			Range.MaxSignificance = AllLODSignificances[RangeIndex + 1];
			Range.InstancedSkinnedMeshSharedDataPtr = &InstancedSkinnedMeshComponentSharedData;

			for (int MeshIndex = 0; MeshIndex < Info.Desc.Meshes.Num(); ++MeshIndex)
			{
				const FMassSkinnedMeshInstanceVisualizationMeshDesc& SkinnedMeshDesc = Info.Desc.Meshes[MeshIndex];
				const bool bAddMeshInRange = (Range.MinSignificance >= SkinnedMeshDesc.MinLODSignificance && Range.MinSignificance < SkinnedMeshDesc.MaxLODSignificance);
				if (bAddMeshInRange)
				{
					checkf(MeshRefKeys.IsValidIndex(MeshIndex) && MeshRefKeys[MeshIndex]
						, TEXT("We don't expect receiving null ISMCs at this point"));
					Range.SkinnedMeshComponentRefs.Add(MeshRefKeys[MeshIndex]);
					Range.SourceDescMeshIndices.Add(MeshIndex);
				}
			}
		}
	}
}

FSkinnedMeshInstanceVisualizationDescHandle UMassVisualizationComponent::AddInstancedSkinnedMeshInfo(const FSkinnedMeshInstanceVisualizationDesc& Desc)
{
	FSkinnedMeshInstanceVisualizationDescHandle Handle;
	if (InstancedSkinnedMeshInfosFreeIndices.Num() > 0)
	{
		Handle = InstancedSkinnedMeshInfosFreeIndices.Pop(EAllowShrinking::No);
		new(&InstancedSkinnedMeshInfos[Handle.ToIndex()]) FMassInstancedSkinnedMeshInfo(Desc);
	}
	else
	{
		int32 AddedInfoIndex = InstancedSkinnedMeshInfos.Emplace(Desc);
		Handle = FSkinnedMeshInstanceVisualizationDescHandle(AddedInfoIndex);
	}

	return Handle;
}
			
void FMassLODInstancedSkinnedMeshSignificanceRange::AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, TConstArrayView<FInstancedSkinnedMeshComponentSharedDataKey> ExcludeMeshRefs)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	for (int32 MeshIndex = 0; MeshIndex < SkinnedMeshComponentRefs.Num(); ++MeshIndex)
	{
		if (ExcludeMeshRefs.Contains(SkinnedMeshComponentRefs[MeshIndex]))
		{
			continue;
		}

		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			SharedData->EntitiesRequiringUpdate.Add(EntityHandle);

			SharedData->MeshInstanceTransforms.Add(SharedData->LocalTransform * Transform);
			SharedData->MeshInstancePrevTransforms.Add(SharedData->LocalTransform * PrevTransform);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::AddBatchedAnimationData(const FMassEntityHandle EntityHandle, FAnimSequenceTrackAutoPlayData AnimationData, TConstArrayView<FInstancedSkinnedMeshComponentSharedDataKey> ExcludeMeshRefs)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	for (int32 MeshIndex = 0; MeshIndex < SkinnedMeshComponentRefs.Num(); ++MeshIndex)
	{
		if (ExcludeMeshRefs.Contains(SkinnedMeshComponentRefs[MeshIndex]))
		{
			continue;
		}

		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			SharedData->MeshInstanceAnimationData.Add(AnimationData);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	for (int32 MeshIndex = 0; MeshIndex < SkinnedMeshComponentRefs.Num(); ++MeshIndex)
	{
		if (ExcludeMeshRefs.Contains(SkinnedMeshComponentRefs[MeshIndex]))
		{
			continue;
		}

		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			SharedData->MeshInstanceCustomFloats.Append(CustomFloats);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::AddBatchedCustomDataFloats(TConstArrayView<TArray<float>> PerMeshFloats, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	check(SourceDescMeshIndices.Num() == SkinnedMeshComponentRefs.Num());

	for (int32 RangeMeshIndex = 0; RangeMeshIndex < SkinnedMeshComponentRefs.Num(); ++RangeMeshIndex)
	{
		const FInstancedSkinnedMeshComponentSharedDataKey& MeshKey = SkinnedMeshComponentRefs[RangeMeshIndex];
		if (ExcludeMeshRefs.Contains(MeshKey))
		{
			continue;
		}

		const int32 SourceIdx = SourceDescMeshIndices[RangeMeshIndex];
		if (!PerMeshFloats.IsValidIndex(SourceIdx) || PerMeshFloats[SourceIdx].IsEmpty())
		{
			continue;
		}

		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(MeshKey))
		{
			SharedData->MeshInstanceCustomFloats.Append(PerMeshFloats[SourceIdx]);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::AddInstance(const FMassEntityHandle EntityHandle, const FTransform& Transform)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	for (int32 MeshIndex = 0; MeshIndex < SkinnedMeshComponentRefs.Num(); ++MeshIndex)
	{
		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			SharedData->EntitiesRequiringUpdate.Add(EntityHandle);

			FTransform AdjustedTransform{ SharedData->LocalTransform * Transform };

			SharedData->MeshInstanceTransforms.Add(AdjustedTransform);
			SharedData->MeshInstancePrevTransforms.Add(AdjustedTransform);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::RemoveInstance(const FMassEntityHandle EntityHandle)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	for (int32 MeshIndex = 0; MeshIndex < SkinnedMeshComponentRefs.Num(); ++MeshIndex)
	{
		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			SharedData->EntitiesRequiringRemoval.Add(EntityHandle);
		}
	}
}

void FMassLODInstancedSkinnedMeshSignificanceRange::WriteCustomDataFloatsAtStartIndex(int32 MeshIndex, const TArrayView<float>& CustomFloats, const int32 FloatsPerInstance, const int32 StartFloatIndex, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs)
{
	check(InstancedSkinnedMeshSharedDataPtr);
	if (SkinnedMeshComponentRefs.IsValidIndex(MeshIndex))
	{
		if (ExcludeMeshRefs.Contains(SkinnedMeshComponentRefs[MeshIndex]))
		{
			return;
		}

		if (FMassInstancedSkinnedMeshComponentSharedData* SharedData = InstancedSkinnedMeshSharedDataPtr->GetAndMarkDirty(SkinnedMeshComponentRefs[MeshIndex]))
		{
			const int32 StartIndex = FloatsPerInstance * SharedData->WriteIterator + StartFloatIndex;

			ensure(SharedData->MeshInstanceCustomFloats.Num() >= StartIndex + CustomFloats.Num());

			for (int CustomFloatIdx = 0; CustomFloatIdx < CustomFloats.Num(); CustomFloatIdx++)
			{
				SharedData->MeshInstanceCustomFloats[StartIndex + CustomFloatIdx] = CustomFloats[CustomFloatIdx];
			}
			SharedData->WriteIterator++;
		}
	}
}