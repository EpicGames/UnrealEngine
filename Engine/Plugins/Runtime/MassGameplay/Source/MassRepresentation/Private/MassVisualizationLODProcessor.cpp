// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassVisualizationLODProcessor.h"
#include "MassCommonFragments.h"
#include "MassDebugger.h"
#include "MassRepresentationDebug.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassVisualizationLODProcessor)

UMassVisualizationLODProcessor::UMassVisualizationLODProcessor()
{
	bAutoRegisterWithProcessingPhases = false;

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LOD;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);
}

void UMassVisualizationLODProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseQuery(EntityManager);
	BaseQuery.AddTagRequirement<FMassVisualizationLODProcessorTag>(EMassFragmentPresence::All);
	BaseQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadWrite);
	BaseQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddConstSharedRequirement<FMassVisualizationLODParameters>();
	BaseQuery.AddSharedRequirement<FMassVisualizationLODSharedFragment>(EMassFragmentAccess::ReadWrite);

	CloseEntityQuery = BaseQuery;
	CloseEntityQuery.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::None);
	CloseEntityQuery.RegisterWithProcessor(*this);

	CloseEntityAdjustDistanceQuery = CloseEntityQuery;
	CloseEntityAdjustDistanceQuery.SetChunkFilter([](const FMassExecutionContext& Context)
	{
		const FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetSharedFragment<FMassVisualizationLODSharedFragment>();
		return LODSharedFragment.bHasAdjustedDistancesFromCount;
	});
	CloseEntityAdjustDistanceQuery.RegisterWithProcessor(*this);

	FarEntityQuery = BaseQuery;
	FarEntityQuery.AddTagRequirement<FMassVisibilityCulledByDistanceTag>(EMassFragmentPresence::All);
	FarEntityQuery.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadOnly);
	FarEntityQuery.SetChunkFilter(&FMassVisualizationChunkFragment::ShouldUpdateVisualizationForChunk);
	FarEntityQuery.RegisterWithProcessor(*this);

	DebugEntityQuery = BaseQuery;
	DebugEntityQuery.RegisterWithProcessor(*this);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassVisualizationLODProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (bForceOFFLOD)
	{
		CloseEntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
			TArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			LODSharedFragment.LODCalculator.ForceOffLOD(Context, RepresentationLODList);
		});
		return;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareExecution)
		const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
		const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();
		EntityManager.ForEachSharedFragment<FMassVisualizationLODSharedFragment>([this, &Viewers](FMassVisualizationLODSharedFragment& LODSharedFragment)
		{
			if (FilterTag == LODSharedFragment.FilterTag)
			{
				LODSharedFragment.LODCalculator.PrepareExecution(Viewers);
			}
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CalculateLOD)

		auto CalculateLOD = [this](FMassExecutionContext& Context)
		{
			FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
			TArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			TConstArrayView<FMassViewerInfoFragment> ViewerInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			LODSharedFragment.LODCalculator.CalculateLOD(Context, ViewerInfoList, RepresentationLODList);
		};
		CloseEntityQuery.ForEachEntityChunk(Context, CalculateLOD);
		FarEntityQuery.ForEachEntityChunk(Context, CalculateLOD);
	}

	if (bDoAdjustmentFromCount)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AdjustDistanceAndLODFromCount)
		EntityManager.ForEachSharedFragment<FMassVisualizationLODSharedFragment>([this](FMassVisualizationLODSharedFragment& LODSharedFragment)
		{
			if (FilterTag == LODSharedFragment.FilterTag)
			{
				LODSharedFragment.bHasAdjustedDistancesFromCount = LODSharedFragment.LODCalculator.AdjustDistancesFromCount();
			}
		});

		CloseEntityAdjustDistanceQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
			TConstArrayView<FMassViewerInfoFragment> ViewerInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			TArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			LODSharedFragment.LODCalculator.AdjustLODFromCount(Context, ViewerInfoList, RepresentationLODList);
		});
		// Far entities do not need to maximize count
	}

#if WITH_MASSGAMEPLAY_DEBUG
	// Optional debug display
	if (UE::Mass::Representation::Debug::DebugRepresentationLOD == 1 || UE::Mass::Representation::Debug::DebugRepresentationLOD >= 3)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayLOD)
		
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			DebugEntityQuery.ForEachEntityChunk(Context, [DebugDrawingWorld](FMassExecutionContext& Context)
			{
				FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
				TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
				TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
				LODSharedFragment.LODCalculator.DebugDisplaySignificantLOD(Context, RepresentationLODList, TransformList, DebugDrawingWorld, UE::Mass::Representation::Debug::DebugRepresentationLODMaxSignificance);
			});
		}
	}
	// Optional vislog
	if (UE::Mass::Representation::Debug::DebugRepresentationLOD >= 2)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VisLogLOD)
		DebugEntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
			TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
			TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			LODSharedFragment.LODCalculator.VisLogSignificantLOD(Context, RepresentationLODList, TransformList, this, UE::Mass::Representation::Debug::DebugRepresentationLODMaxSignificance);
		});
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}
