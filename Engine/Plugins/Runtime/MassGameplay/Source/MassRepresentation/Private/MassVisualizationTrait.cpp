// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassVisualizationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassRepresentationSubsystem.h"
#include "MassCommonFragments.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationActorManagement.h"
#include "Engine/World.h"
#include "MassLODFragments.h"
#include "MassActorSubsystem.h"
#include "MassEntityUtils.h"
#include "MassVisualizationLODProcessor.h"
#include "MassRepresentationProcessor.h"
#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Editor.h"
#include "UObject/UnrealType.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassVisualizationTrait)

#define LOCTEXT_NAMESPACE "Mass"


UMassVisualizationTrait::UMassVisualizationTrait()
{
	RepresentationSubsystemClass = UMassRepresentationSubsystem::StaticClass();

	Params.RepresentationActorManagementClass = UMassRepresentationActorManagement::StaticClass();
	Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::HighResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::LowResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Low] = EMassRepresentationType::StaticMeshInstance;
	Params.LODRepresentation[EMassLOD::Off] = EMassRepresentationType::None;

	LODParams.BaseLODDistance[EMassLOD::High] = 0.f;
	LODParams.BaseLODDistance[EMassLOD::Medium] = 1000.f;
	LODParams.BaseLODDistance[EMassLOD::Low] = 2500.f;
	LODParams.BaseLODDistance[EMassLOD::Off] = 10000.f;

	LODParams.VisibleLODDistance[EMassLOD::High] = 0.f;
	LODParams.VisibleLODDistance[EMassLOD::Medium] = 2000.f;
	LODParams.VisibleLODDistance[EMassLOD::Low] = 4000.f;
	LODParams.VisibleLODDistance[EMassLOD::Off] = 10000.f;

	LODParams.LODMaxCount[EMassLOD::High] = 50;
	LODParams.LODMaxCount[EMassLOD::Medium] = 100;
	LODParams.LODMaxCount[EMassLOD::Low] = 500;
	LODParams.LODMaxCount[EMassLOD::Off] = 0;

	LODParams.BufferHysteresisOnDistancePercentage = 10.0f;
	LODParams.DistanceToFrustum = 0.0f;
	LODParams.DistanceToFrustumHysteresis = 0.0f;

	bAllowServerSideVisualization = false;
}

void UMassVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	// This should not be ran on NM_Server network mode
	if (World.IsNetMode(NM_DedicatedServer) && !bAllowServerSideVisualization 
		&& !BuildContext.IsInspectingData())
	{
		return;
	}

	BuildContext.RequireFragment<FMassViewerInfoFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassActorFragment>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	UMassRepresentationSubsystem* RepresentationSubsystem = Cast<UMassRepresentationSubsystem>(World.GetSubsystemBase(RepresentationSubsystemClass));
	if (RepresentationSubsystem == nullptr && !BuildContext.IsInspectingData())
	{
		UE_LOGF(LogMassRepresentation, Error, "Expecting a valid class for the representation subsystem");
		RepresentationSubsystem = UWorld::GetSubsystem<UMassRepresentationSubsystem>(&World);
		check(RepresentationSubsystem);
	}

	FMassRepresentationSubsystemSharedFragment SubsystemSharedFragment;
	SubsystemSharedFragment.RepresentationSubsystem = RepresentationSubsystem;
	FSharedStruct SubsystemFragment = EntityManager.GetOrCreateSharedFragment(SubsystemSharedFragment);
	BuildContext.AddSharedFragment(SubsystemFragment);

	if (!Params.RepresentationActorManagementClass)
	{
		UE_LOGF(LogMassRepresentation, Error, "Expecting a valid class for the representation actor management");
	}

	FMassRepresentationFragment& RepresentationFragment = BuildContext.AddFragment_GetRef<FMassRepresentationFragment>();
	if (LIKELY(BuildContext.IsInspectingData() == false))
	{
		RepresentationFragment.HighResTemplateActorIndex = HighResTemplateActor.Get() ? RepresentationSubsystem->FindOrAddTemplateActor(HighResTemplateActor.Get()) : INDEX_NONE;
		RepresentationFragment.LowResTemplateActorIndex = LowResTemplateActor.Get() ? RepresentationSubsystem->FindOrAddTemplateActor(LowResTemplateActor.Get()) : INDEX_NONE;
	}

	bool bStaticMeshDescriptionValid = StaticMeshInstanceDesc.IsValid();
	if (bStaticMeshDescriptionValid)
	{
		if (bRegisterStaticMeshDesc && !BuildContext.IsInspectingData())
		{
			RepresentationFragment.StaticMeshDescHandle = RepresentationSubsystem->FindOrAddStaticMeshDesc(StaticMeshInstanceDesc);
			ensureMsgf(RepresentationFragment.StaticMeshDescHandle.IsValid()
				, TEXT("Expected to get a valid StaticMeshDescHandle since we already checked that StaticMeshInstanceDesc is valid"));
			// if the unexpected happens and StaticMeshDescHandle is not valid we're going to treat it as if StaticMeshInstanceDesc
			// was not valid in the first place and handle it accordingly in a moment
			bStaticMeshDescriptionValid = RepresentationFragment.StaticMeshDescHandle.IsValid();
		}
	}

	bool bSkinnedMeshDescriptionValid = SkinnedMeshInstanceDesc.IsValid();
	if (bSkinnedMeshDescriptionValid)
	{
		if (!BuildContext.IsInspectingData())
		{
			RepresentationFragment.SkinnedMeshDescHandle = RepresentationSubsystem->FindOrAddSkinnedMeshVisualizationDesc(SkinnedMeshInstanceDesc);
			ensureMsgf(RepresentationFragment.SkinnedMeshDescHandle.IsValid()
				, TEXT("Expected to get a valid SkinnedMeshDescHandle since we already checked that SkinnedMeshInstanceDesc is valid"));
			// if the unexpected happens and SkinnedMeshDescHandle is not valid we're going to treat it as if SkinnedMeshInstanceDesc
			// was not valid in the first place and handle it accordingly in a moment
			bSkinnedMeshDescriptionValid = RepresentationFragment.SkinnedMeshDescHandle.IsValid();
		}
	}



	FConstSharedStruct ParamsFragment;
	if (bStaticMeshDescriptionValid && bSkinnedMeshDescriptionValid)
	{
		ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(Params);
	}
	else
	{
		FMassRepresentationParameters ParamsCopy = Params;
		SanitizeMeshParams(ParamsCopy, !bStaticMeshDescriptionValid, !bSkinnedMeshDescriptionValid);
		ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(ParamsCopy);
	}
	ParamsFragment.Get<const FMassRepresentationParameters>().ComputeCachedValues();
	BuildContext.AddConstSharedFragment(ParamsFragment);

	FConstSharedStruct LODParamsFragment = EntityManager.GetOrCreateConstSharedFragment(LODParams);
	BuildContext.AddConstSharedFragment(LODParamsFragment);

	FSharedStruct LODSharedFragment = EntityManager.GetOrCreateSharedFragment<FMassVisualizationLODSharedFragment>(FConstStructView::Make(LODParams), LODParams);
	BuildContext.AddSharedFragment(LODSharedFragment);

	BuildContext.AddFragment<FMassRepresentationLODFragment>();
	BuildContext.AddTag<FMassVisibilityCulledByDistanceTag>();
	BuildContext.AddChunkFragment<FMassVisualizationChunkFragment>();

	BuildContext.AddTag<FMassVisualizationLODProcessorTag>();
	BuildContext.AddTag<FMassVisualizationProcessorTag>();
}

void UMassVisualizationTrait::SanitizeParams(FMassRepresentationParameters& InOutParams, const bool bStaticMeshDeterminedInvalid) const
{
	SanitizeMeshParams(InOutParams, bStaticMeshDeterminedInvalid);
}

void UMassVisualizationTrait::SanitizeMeshParams(FMassRepresentationParameters& InOutParams, const bool bStaticMeshDeterminedInvalid /*= false*/, const bool bSkinnedMeshDeterminedInvalid /*= false*/) const
{
	if (bStaticMeshDeterminedInvalid)
	{
		for (int32 LODIndex = 0; LODIndex < EMassLOD::Max; ++LODIndex)
		{
			if (InOutParams.LODRepresentation[LODIndex] == EMassRepresentationType::StaticMeshInstance)
			{
				InOutParams.LODRepresentation[LODIndex] = EMassRepresentationType::None;
			}
		}
	}

	if (bSkinnedMeshDeterminedInvalid)
	{
		for (int32 LODIndex = 0; LODIndex < EMassLOD::Max; ++LODIndex)
		{
			if (InOutParams.LODRepresentation[LODIndex] == EMassRepresentationType::SkinnedMeshInstance)
			{
				InOutParams.LODRepresentation[LODIndex] = EMassRepresentationType::None;
			}
		}
	}
}

void UMassVisualizationTrait::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (GEditor && (Ar.IsLoading() || Ar.IsSaving()))
	{
		ValidateParams();
	}
#endif // WITH_EDITOR
}

bool UMassVisualizationTrait::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	Super::ValidateTemplate(BuildContext, World, OutTraitRequirements);

#if WITH_EDITOR
	return ValidateParams();
#else
	return true;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
bool UMassVisualizationTrait::ValidateParams() const
{
	bool bIssuesFound = false;

	// if this test is called on any of the CDOs we don't care, we're never going to utilize those in practice.
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// the SM config provided is not valid. We need to check if EMassRepresentationType::StaticMeshInstance
		// is being used as any of the LODRepresentations. If so then we need to clear those out and report an error
		if (bRequireValidStaticMeshInstanceDesc == true
			&& StaticMeshInstanceDesc.IsValid() == false)
		{
			for (int32 LODIndex = 0; LODIndex < EMassLOD::Max; ++LODIndex)
			{
				if (Params.LODRepresentation[LODIndex] == EMassRepresentationType::StaticMeshInstance)
				{
					bIssuesFound = true;

					UE_LOGF(LogMassRepresentation, Error, "Trait %ls is using StaticMeshInstance representation type for "
						"LODRepresentation[%ls] while the trait's StaticMeshInstanceDesc is not valid (has no Meshes). Entities "
						"won't be visible at this LOD level."
						, *GetPathName(), *UEnum::GetValueAsString(EMassLOD::Type(LODIndex)));
				}
			}
		}
	}

	return !bIssuesFound;
}

void UMassVisualizationTrait::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName ParamsName = GET_MEMBER_NAME_CHECKED(UMassVisualizationTrait, Params);
	static const FName StaticMeshDescriptionName = GET_MEMBER_NAME_CHECKED(UMassVisualizationTrait, StaticMeshInstanceDesc);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropName == ParamsName || PropName == StaticMeshDescriptionName)
		{
			ValidateParams();
		}
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE 
