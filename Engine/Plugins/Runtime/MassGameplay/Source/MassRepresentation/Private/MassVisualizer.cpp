// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassVisualizer.h"
#include "MassVisualizationComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassVisualizer)

AMassVisualizer::AMassVisualizer()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent->SetMobility(EComponentMobility::Static);
	VisComponent = CreateDefaultSubobject<UMassVisualizationComponent>(TEXT("VisualizerComponent"));

	// MassVisualizers are created by the subsystems and should not be exposed to users or allowed to be deleted.
#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
#endif
}

