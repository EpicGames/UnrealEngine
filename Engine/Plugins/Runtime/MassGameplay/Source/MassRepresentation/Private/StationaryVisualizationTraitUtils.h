// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UMassEntityTraitBase;
struct FMassEntityTemplateBuildContext;
struct FStaticMeshInstanceVisualizationDesc;

namespace UE::Mass::LOD::Private
{
	void SetUpStationaryVisualizationTrait(const UMassEntityTraitBase& Trait, FMassEntityTemplateBuildContext& BuildContext, FStaticMeshInstanceVisualizationDesc& StaticMeshInstanceDesc);
}