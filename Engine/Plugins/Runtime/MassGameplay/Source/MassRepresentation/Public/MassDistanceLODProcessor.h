// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRepresentationFragments.h"
#include "MassCommonFragments.h"
#include "MassLODCalculator.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassDistanceLODProcessor.generated.h"


/** 
 * Tag required by Distance LOD Processor to update LOD information. Removing the tag allows to support temporary 
 * disabling of processing for individual entities.
 */
USTRUCT()
struct FMassDistanceLODProcessorTag : public FMassTag
{
	GENERATED_BODY();
};

UCLASS(MinimalAPI)
class UMassDistanceLODProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSREPRESENTATION_API UMassDistanceLODProcessor();

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	MASSREPRESENTATION_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	/** 
	 * Execution method for this processor 
	 * @param EntitySubsystem is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 */
	MASSREPRESENTATION_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	
	/**
	 * Forces Off LOD on all calculation
	 * @param bForce to whether force or not Off LOD
	 */
	void ForceOffLOD(bool bForce) { bForceOFFLOD = bForce; }

protected:
	FMassEntityQuery CloseEntityQuery;
	FMassEntityQuery FarEntityQuery;
	FMassEntityQuery DebugEntityQuery;

	bool bForceOFFLOD = false;

	UPROPERTY(Transient)
	TObjectPtr<const UScriptStruct> FilterTag = nullptr;
};
