// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassRepresentationTypes.h"
#include "MassRepresentationFragments.h"
#include "GameFramework/Actor.h"
#include "MassDistanceVisualizationTrait.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMassRepresentationSubsystem;
class UMassRepresentationActorManagement;
class UMassProcessor;

/** This class has been soft-deprecated. Use MassStationaryVisualizationTrait or MassMovableVisualizationTrait */
UCLASS(MinimalAPI, meta=(DisplayName="DEPRECATED DistanceVisualization"))
class UMassDistanceVisualizationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()
public:
	UE_API UMassDistanceVisualizationTrait();

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	UE_API virtual void DestroyTemplate(const UWorld& World) const override;

	/** Instanced static mesh information for this agent */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	mutable FStaticMeshInstanceVisualizationDesc StaticMeshInstanceDesc;

	/** Actor class of this agent when spawned in high resolution*/
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<AActor> HighResTemplateActor;

	/** Actor class of this agent when spawned in low resolution*/
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<AActor> LowResTemplateActor;

	/** Allow subclasses to override the representation subsystem to use */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta = (EditCondition = "bCanModifyRepresentationSubsystemClass"))
	TSubclassOf<UMassRepresentationSubsystem> RepresentationSubsystemClass;

	/** Configuration parameters for the representation processor */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FMassRepresentationParameters Params;

	/** Configuration parameters for the Distance LOD processor */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FMassDistanceLODParameters LODParams;

	/** If set to true will result in the visualization-related fragments being added to server-size entities as well.
	 *  By default only the clients require visualization fragments */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bAllowServerSideVisualization = false;

#if WITH_EDITORONLY_DATA
	/** the property is marked like this to ensure it won't show up in UI */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|Visual")
	bool bCanModifyRepresentationSubsystemClass = true;
#endif // WITH_EDITORONLY_DATA

protected:
	/** 
	 * Controls whether StaticMeshInstanceDesc gets registered via FindOrAddStaticMeshDesc call. Setting it to `false` 
	 * can be useful for subclasses to avoid needlessly creating visualization data in RepresentationSubsystem, 
	 * data that will never be used.
	 */
	bool bRegisterStaticMeshDesc = true;
};

#undef UE_API
