// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassVisualizationTrait.h"
#include "MassMovableVisualizationTrait.generated.h"

#define UE_API MASSREPRESENTATION_API


UCLASS(MinimalAPI)
class UMassMovableVisualizationTrait : public UMassVisualizationTrait
{
	GENERATED_BODY()
public:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};
	

#undef UE_API
