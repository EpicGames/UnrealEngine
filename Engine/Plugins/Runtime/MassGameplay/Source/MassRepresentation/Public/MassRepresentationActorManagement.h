// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationActorManagement.generated.h"

#define UE_API MASSREPRESENTATION_API

struct FMassActorSpawnRequestHandle;
struct FMassCommandBuffer;
struct FMassEntityManager;
enum class EMassActorSpawnRequestAction : uint8;
enum class EMassActorEnabledType : uint8;
struct FConstStructView;
struct FMassEntityHandle;
struct FMassActorFragment;
struct FMassRepresentationLODFragment;
struct FMassRepresentationFragment;
class UMassRepresentationSubsystem;
class UMassActorSubsystem;

UCLASS(MinimalAPI)
class UMassRepresentationActorManagement : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Returns the spawn priority from the LOD
	 * @param Representation is the type of enabling to do on this actor
	 */
	UE_API virtual float GetSpawnPriority(const FMassRepresentationLODFragment& Representation) const;

	/**
	 * Returns an actor of the template type and setup fragments values from it
	 * @param RepresentationSubsystem to use to get or spawn the actor
	 * @param EntityManager associated to the mass agent
	 * @param MassAgent is the handle to the associated mass agent
	 * @param ActorInfo is the fragment where we are going to store the actor pointer
	 * @param Transform is the spatial information about where to spawn the actor
	 * @param TemplateActorIndex is the index of the type fetched with UMassRepresentationSubsystem::FindOrAddTemplateActor()
	 * @param SpawnRequestHandle (in/out) In: previously requested spawn Out: newly requested spawn
	 * @param Priority of this spawn request in comparison with the others, lower value means higher priority
	 * @return the actor spawned
	 */
	UE_API virtual AActor* GetOrSpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem, FMassEntityManager& EntityManager
		, const FMassEntityHandle MassAgent, const FTransform& Transform, const int16 TemplateActorIndex
		, FMassActorSpawnRequestHandle& InOutSpawnRequestHandle, const float Priority) const;

	/**
	 * Enable/disable a spawned actor for a mass entity
	 * @param EnabledType is the type of enabling to do on this actor
	 * @param Actor is the actual actor to perform enabling type on
	 * @param EntityIdx is the entity index currently processing
	 * @param CommandBuffer to queue up anything that is thread sensitive
	 */
	UE_API virtual void SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx, FMassCommandBuffer& CommandBuffer) const;

	/**
	 * Teleports the actor at the specified transform by preserving its velocity and without collision.
	 * The destination will be adjusted to fit an existing capsule.
	 * @param Transform is the new actor's transform
	 * @param Actor is the actual actor to teleport
	 * @param CommandBuffer to queue up anything that is thread sensitive
	 */
	UE_API virtual void TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const;

	/**
	 * Returns whether the given actor is ready to be used as the visual representation of a Mass entity.
	 * The default implementation always returns true, preserving the original behavior of switching
	 * the representation as soon as UWorld::SpawnActor returns. Game-side subclasses should override
	 * this to keep the entity on its non-actor representation (typically StaticMeshInstance) while
	 * an asynchronously streamed asset (SkeletalMesh, StaticMesh, ...) is still being loaded by the
	 * spawned actor.
	 *
	 * Called from UMassRepresentationProcessor::UpdateRepresentation: while this returns false the
	 * actor stays disabled and the ISM/SkinnedMesh instance remains visible (via bIsWaitingForActorVisualReadiness).
	 * The wait is bounded by the cvar ai.massrepresentation.MaxActorVisualReadinessWaitFrames; after that
	 * cap the processor force-enables and logs a warning, so a buggy override doesn't strand entities.
	 *
	 * The hook also fires for externally-owned actors (those hydrated by replication or other
	 * non-Mass spawners) on the frame Mass first commits CurrentRepresentation to an actor type;
	 * once committed those actors don't re-enter the spawn switch on subsequent frames, so
	 * post-commit streaming changes are not gated.
	 *
	 * Note on dedicated servers: signals like AActor::WasRecentlyRendered will never report true on
	 * a server. Implementations should short-circuit to true when IsNetMode(NM_DedicatedServer) is
	 * true, otherwise entities will never leave the default representation server-side.
	 *
	 * @param Actor the just-spawned (or pre-existing externally-owned) actor under consideration
	 * @return true once the actor is ready to represent the entity, false to keep waiting
	 */
	UE_API virtual bool IsActorReadyForRepresentation(const AActor& Actor) const;

	/**
	 * Method that will be bound to a delegate called before the spawning of an actor to let the requester prepare it
	 * @param SpawnRequestHandle the handle of the spawn request that is about to spawn
	 * @param SpawnRequest of the actor that is about to spawn
	 * @param EntityManager to use to retrieve the mass agent fragments
	 */
	UE_API virtual void OnPreActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const;

	/**
	 * Method that will be bound to a delegate used post-spawn to notify and let the requester configure the actor
	 * @param SpawnRequestHandle the handle of the spawn request that was just spawned
	 * @param SpawnRequest of the actor that just spawned
	 * @param EntityManager to use to retrieve the mass agent fragments
	 * @return The action to take on the spawn request, either keep it there or remove it.
	 */
	UE_API virtual EMassActorSpawnRequestAction OnPostActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const;

	/**
	 * Static methods to Release an actor or cancel its spawning (calls ReleaseAnyActorOrCancelAnySpawning)
	 * WARNING: This method will destroy the associated actor in any and by the same fact might also move the entity into a new archetype.
	 *          So any reference to fragment might become invalid.
	 * @param EntityManager to use to retrieve the mass agent fragments
	 * @param MassAgent is the handle to the associated mass agent
	 * @param bImmediate whether to perform the actor destruction immediately, otherwise it will be queued for processing later
	 * @return True if actor was release or spawning request was canceled
	 */
	static UE_API void ReleaseAnyActorOrCancelAnySpawning(FMassEntityManager& EntityManager, const FMassEntityHandle MassAgent, bool bImmediate = false);

	/**
	 * static Release an actor or cancel its spawning
	 * WARNING: This method will destroy the associated actor in any and by the same fact might also move the entity into a new archetype.
	 *          So any reference to fragment might become invalid if you are not within the pipe execution
	 * @param RepresentationSubsystem to use to release any actors or cancel spawning requests
	 * @param MassAgent is the handle to the associated mass agent
	 * @param ActorInfo is the fragment where we are going to store the actor pointer
	 * @param Representation fragment containing the current and previous visual state
	 * @param ActorSubsystem passed over to FMassActorFragment::ResetAndUpdateHandleMap, used to avoid fetching the subsystem
	 *	from UWorld every time.
	 * @param bImmediate whether to perform the actor destruction immediately, otherwise it will be queued for processing later
	 */
	static UE_API void ReleaseAnyActorOrCancelAnySpawning(UMassRepresentationSubsystem& RepresentationSubsystem, const FMassEntityHandle MassAgent
		, FMassActorFragment& ActorInfo, FMassRepresentationFragment& Representation, UMassActorSubsystem* ActorSubsystem = nullptr, bool bImmediate = false);
};

#undef UE_API
