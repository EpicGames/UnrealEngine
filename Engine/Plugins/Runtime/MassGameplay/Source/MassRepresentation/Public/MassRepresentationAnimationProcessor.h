// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassRepresentationAnimationProcessor.generated.h"

#define UE_API MASSREPRESENTATION_API

struct FMassInstancedSkinnedMeshInfo;
struct FAnimSequenceTrackAutoPlayData;

/**
 * This processor's responsibility is to consume Animation Data generated by end user processors and forward it onto the Skinned Mesh
 * this data will be batch updated in the Instanced Skinned Mesh Component
 */
UCLASS(MinimalAPI)
class UMassConsumeInstancedSkinnedMeshAnimationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassConsumeInstancedSkinnedMeshAnimationProcessor();

	static UE_API void UpdateMeshAnimation(FMassEntityHandle EntityHandle, FMassInstancedSkinnedMeshInfo& MeshInfo, FAnimSequenceTrackAutoPlayData AnimationData, const float LODSignificance, const float PrevLODSignificance = -1.0f);

protected:

	// UMassProcessor overrides begin
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	// UMassProcessor overrides end

	FMassEntityQuery EntityQuery;
};

#undef UE_API