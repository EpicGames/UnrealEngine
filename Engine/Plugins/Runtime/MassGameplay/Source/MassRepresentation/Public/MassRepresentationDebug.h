// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "Containers/ArrayView.h"

#if WITH_MASSGAMEPLAY_DEBUG

class UWorld;
struct FColor;
struct FMassExecutionContext;
struct FMassRepresentationFragment;
struct FMassRepresentationLODFragment;
struct FTransformFragment;

namespace UE::Mass::Representation::Debug
{
	extern MASSREPRESENTATION_API FColor RepresentationColors[];

	extern MASSREPRESENTATION_API int32 DebugRepresentation;
	extern MASSREPRESENTATION_API float DebugRepresentationMaxSignificance;

	extern MASSREPRESENTATION_API int32 DebugRepresentationLOD;
	extern MASSREPRESENTATION_API float DebugRepresentationLODMaxSignificance;

	void DebugDisplayRepresentation(FMassExecutionContext& Context, TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList, TConstArrayView<FMassRepresentationFragment> RepresentationList, TConstArrayView<FTransformFragment> LocationList, UWorld* World);

	void VisLogRepresentation(FMassExecutionContext& Context, TConstArrayView<FMassRepresentationLODFragment> RepresentationLODList, TConstArrayView<FMassRepresentationFragment> RepresentationList, TConstArrayView<FTransformFragment> LocationList, UObject* LogOwner);
}

#endif // WITH_MASSGAMEPLAY_DEBUG
