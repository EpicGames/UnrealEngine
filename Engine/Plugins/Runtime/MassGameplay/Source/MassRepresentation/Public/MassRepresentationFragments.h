// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODTypes.h"
#include "MassEntityTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassActorSpawnerSubsystem.h"
#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "MassRepresentationActorManagement.h"
#include "MassLODCalculator.h"
#include "MassRepresentationFragments.generated.h"


class UMassRepresentationSubsystem;
class UMassRepresentationActorManagement;

USTRUCT()
struct FMassStaticRepresentationTag : public FMassTag
{
	GENERATED_BODY(); 
};

USTRUCT()
struct FMassRepresentationLODFragment : public FMassFragment
{
	GENERATED_BODY()

	/** LOD information */
	UPROPERTY()
	TEnumAsByte<EMassLOD::Type> LOD = EMassLOD::Max;

	UPROPERTY()
	TEnumAsByte<EMassLOD::Type> PrevLOD = EMassLOD::Max;

	/** Visibility Info */
	UPROPERTY()
	EMassVisibility Visibility = EMassVisibility::Max;

	UPROPERTY()
	EMassVisibility PrevVisibility = EMassVisibility::Max;

	/** Value scaling from 0 to 3, 0 highest LOD we support and 3 being completely off LOD */
	UPROPERTY()
	float LODSignificance = 0.0f;
};

USTRUCT()
struct FMassRepresentationFragment : public FMassFragment
{
    GENERATED_BODY()

	static constexpr int32 MaxActorVisualReadinessWaitFrames = static_cast<int32>(TNumericLimits<uint8>::Max());

    UPROPERTY()
    EMassRepresentationType CurrentRepresentation = EMassRepresentationType::None;

    UPROPERTY()
    EMassRepresentationType PrevRepresentation = EMassRepresentationType::None;

    UPROPERTY()
    int16 HighResTemplateActorIndex = INDEX_NONE;

    UPROPERTY()
    int16 LowResTemplateActorIndex = INDEX_NONE;

    UPROPERTY()
    FStaticMeshInstanceVisualizationDescHandle StaticMeshDescHandle;

	UPROPERTY()
    FSkinnedMeshInstanceVisualizationDescHandle SkinnedMeshDescHandle;

    /** Value scaling from 0 to 3, 0 highest LOD we support and 3 being completely off LOD */
    UPROPERTY()
    float PrevLODSignificance = -1.0f;

    UPROPERTY()
    uint8 bIsPendingDestruction : 1  = false;

private:
	/**
	 * When true, the actor has been spawned (or re-used externally) but is kept disabled while the prior mesh
	 * representation (ISM/SkinnedMesh) covers the visual gap. Cleared once
	 * UMassRepresentationActorManagement::IsActorReadyForRepresentation reports true (or after the
	 * bDoWaitForActorVisualReadiness 1-frame delay). May span multiple frames when an override delays readiness for
	 * asset streaming.
	 */
	UPROPERTY()
	uint8 bIsWaitingForActorVisualReadiness : 1  = false;

	/**
	 * Counts consecutive frames bIsWaitingForActorVisualReadiness has been set, used by UpdateRepresentation to detect runaway
	 * readiness gates (e.g. an override that never returns true) and force-enable the actor after a configurable
	 * cap. Reset to 0 whenever bIsWaitingForActorVisualReadiness transitions to false. Saturates at 255.
	 */
	UPROPERTY()
	uint8 ActorVisualReadinessWaitFrames = 0;

public:
    UPROPERTY()
    FMassActorSpawnRequestHandle ActorSpawnRequestHandle;

    UPROPERTY()
    FTransform PrevTransform;

	bool IsWaitingForActorVisualReadiness() const
	{
		return bIsWaitingForActorVisualReadiness;
	}

	/** sets bIsWaitingForActorVisualReadiness and resets ActorVisualReadinessWaitFrames */
	void SetIsWaitingForActorVisualReadiness(const bool bNewValue)
	{
		bIsWaitingForActorVisualReadiness = bNewValue;
		ActorVisualReadinessWaitFrames = 0;
	}

	void IncreaseActorVisualReadinessWaitFrames()
	{
		ActorVisualReadinessWaitFrames = static_cast<uint8>(FMath::Min(static_cast<int32>(ActorVisualReadinessWaitFrames + 1), MaxActorVisualReadinessWaitFrames));
	}

	bool IsActorVisualRedinessTimedOut(const int32 Limit) const
	{
		return static_cast<int32>(ActorVisualReadinessWaitFrames >= Limit);
	}
};

USTRUCT()
struct FMassRepresentationAnimationFragment : public FMassFragment
{
    GENERATED_BODY()

	UPROPERTY(Transient)
	FAnimSequenceTrackAutoPlayData AnimData;
};

USTRUCT()
struct FMassRepresentationSubsystemSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TObjectPtr<UMassRepresentationSubsystem> RepresentationSubsystem = nullptr;
};

template<>
struct TMassSharedFragmentTraits<FMassRepresentationSubsystemSharedFragment> final
{
	enum
	{
		GameThreadOnly = true
	};
};


USTRUCT()
struct FMassRepresentationParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassRepresentationParameters() = default;

	/** Allow subclasses to override the representation actor management behavior */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta = (EditCondition = "bCanModifyRepresentationActorManagementClass"))
	TSubclassOf<UMassRepresentationActorManagement> RepresentationActorManagementClass;

	/** What should be the representation of this entity for each specific LOD */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	EMassRepresentationType LODRepresentation[EMassLOD::Max] = { EMassRepresentationType::HighResSpawnedActor, EMassRepresentationType::LowResSpawnedActor, EMassRepresentationType::StaticMeshInstance, EMassRepresentationType::None };

	/** World Partition grid name to test collision against, default None will be the main grid */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	FName WorldPartitionGridNameContainingCollision;

	/** At what rate should the not visible entity be updated in seconds */
	UPROPERTY(EditAnywhere, Category = "Mass|Visualization", config)
	float NotVisibleUpdateRate = 0.5f;

	// -- all propeties comparable with Memcpy go above this line. 
	// -- all other properties need to be manually handled in Identical implementation (as long as they are not "transient")

	/** 
	 * If true, forces UMassRepresentationProcessor to override the WantedRepresentationType to actor representation whenever an external (non Mass owned)
	 * actor is set on an entitie's FMassActorFragment fragment. If / when the actor fragment is reset, WantedRepresentationType resumes selecting the 
	 * appropriate representation for the current representation LOD.
	 *
	 * Useful for server-authoritative actor spawning to force actor representation on clients for replicated actors. 
	 */ 
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	uint8 bForceActorRepresentationForExternalActors : 1 = false;

	/** If true, LowRes actors will be kept around, disabled, whilst StaticMeshInstance representation is active */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	uint8 bKeepLowResActors : 1  = true;

	/** When switching to ISM keep the actor an extra frame, helps cover rendering glitches (i.e. occlusion query being one frame late) */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	uint8 bKeepActorExtraFrame : 1  = false;

	/**
	 * When switching from ISM/SkinnedMesh to Actor, keep the mesh visible while the spawned actor reports
	 * not-ready (UMassRepresentationActorManagement::IsActorReadyForRepresentation) - typically because a
	 * soft-pointered SkeletalMesh / StaticMesh is still streaming in, or downstream init has not run yet
	 * on the freshly-spawned actor. Bounded by ai.massrepresentation.MaxActorVisualReadinessWaitFrames.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	uint8 bWaitForActorVisualReadiness : 1  = false;

	/** If true, will spread the first visualization update over the period specified in NotVisibleUpdateRate member */
	UPROPERTY(EditAnywhere, Category = "Mass|Representation", config)
	uint8 bSpreadFirstVisualizationUpdate : 1  = false;

#if WITH_EDITORONLY_DATA
	/** the property is marked like this to ensure it won't show up in UI */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|Visual")
	uint8 bCanModifyRepresentationActorManagementClass : 1 = true;
#endif // WITH_EDITORONLY_DATA

	inline void ComputeCachedValues() const;

	/** Default representation when unable to spawn an actor, gets calculated at initialization */
	UPROPERTY(Transient)
	mutable EMassRepresentationType CachedDefaultRepresentationType = EMassRepresentationType::None;

	UPROPERTY(Transient)
	mutable TObjectPtr<UMassRepresentationActorManagement> CachedRepresentationActorManagement = nullptr;

	/** 
	 * The operation by design ignores CachedDefaultRepresentationType and CachedRepresentationActorManagement,
	 * since those two properties are fully rutime and are derived from the others.
	 */
	bool Identical(const FMassRepresentationParameters* InOtherValue, uint32 /*PortFlags*/) const
	{
		constexpr size_t NumBytesToCompare = offsetof(FMassRepresentationParameters, NotVisibleUpdateRate) + sizeof(NotVisibleUpdateRate);
		const int32 Result = FMemory::Memcmp(this, InOtherValue, NumBytesToCompare);

		return Result == 0
			// comparing bitfields separately since the padding might contain garbage
			&& bForceActorRepresentationForExternalActors == InOtherValue->bForceActorRepresentationForExternalActors
			&& bKeepLowResActors == InOtherValue->bKeepLowResActors
			&& bKeepActorExtraFrame == InOtherValue->bKeepActorExtraFrame
			&& bWaitForActorVisualReadiness == InOtherValue->bWaitForActorVisualReadiness
			&& bSpreadFirstVisualizationUpdate == InOtherValue->bSpreadFirstVisualizationUpdate
#if WITH_EDITORONLY_DATA
			&& bCanModifyRepresentationActorManagementClass == InOtherValue->bCanModifyRepresentationActorManagementClass
#endif // WITH_EDITORONLY_DATA
		;
	}
};

template<>
struct TStructOpsTypeTraits<FMassRepresentationParameters> : public TStructOpsTypeTraitsBase2<FMassRepresentationParameters>
{
	enum 
	{
		WithIdentical = true,
	};
};

template<>
struct TMassSharedFragmentTraits<FMassRepresentationParameters> final
{
	enum
	{
		GameThreadOnly = true
	};
};


inline void FMassRepresentationParameters::ComputeCachedValues() const
{
	// Calculate the default representation when actor isn't spawned yet.
	for (int32 LOD = EMassLOD::High; LOD < EMassLOD::Max; LOD++)
	{
		// Find the first representation type after any actors
		if (LODRepresentation[LOD] == EMassRepresentationType::HighResSpawnedActor ||
			LODRepresentation[LOD] == EMassRepresentationType::LowResSpawnedActor)
		{
			continue;
		}

		CachedDefaultRepresentationType = LODRepresentation[LOD];
		break;
	}

	CachedRepresentationActorManagement = RepresentationActorManagementClass.GetDefaultObject();
	if (CachedRepresentationActorManagement == nullptr)
	{
		// We should have warn about it in the traits.
		CachedRepresentationActorManagement = UMassRepresentationActorManagement::StaticClass()->GetDefaultObject<UMassRepresentationActorManagement>();
	}
}

USTRUCT()
struct FMassVisualizationLODParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** Distances where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float BaseLODDistance[EMassLOD::Max] = { 0.f, 1000.f, 2500.f, 10000.f };
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float VisibleLODDistance[EMassLOD::Max] = { 0.f, 2000.f, 4000.f, 15000.f };
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float BufferHysteresisOnDistancePercentage = 10.0f;

	/** Maximum limit for each entity per LOD */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	int32 LODMaxCount[EMassLOD::Max] = {50, 100, 500, MAX_int32};

	/** Entities within this distance from frustum will be considered visible. Expressed in Unreal Units. */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float DistanceToFrustum = 0.0f;

	/** Once visible how much further than DistanceToFrustum does the entities need to be before being cull again */
	/** 
	 * Once an entity is visible how far away from frustum does it need to get to lose "visible" state. 
	 * Expressed in Unreal Units and is added to DistanceToFrustum to arrive at the final value to be used for testing.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float DistanceToFrustumHysteresis = 0.0f;

	/** Filter these settings with specified tag */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (BaseStruct = "/Script/MassEntity.MassTag"))
	TObjectPtr<UScriptStruct> FilterTag = nullptr;
};

USTRUCT()
struct FMassVisualizationLODSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	FMassVisualizationLODSharedFragment() = default;
	FMassVisualizationLODSharedFragment(const FMassVisualizationLODParameters& LODParams);

	TMassLODCalculator<FMassRepresentationLODLogic> LODCalculator;
	bool bHasAdjustedDistancesFromCount = false;

	UPROPERTY(Transient)
	TObjectPtr<const UScriptStruct> FilterTag = nullptr;
};

/** Simplest version of LOD Calculation based strictly on Distance parameters 
 *	Compared to FMassVisualizationLODParameters, we:
 *	* Only include a single set of LOD Distances (radial distance from viewer)
 *	* we do not care about distance to Frustum
 *	* we do not care about Max Count
 */
USTRUCT()
struct FMassDistanceLODParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** Distances where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float LODDistance[EMassLOD::Max] = { 0.f, 1000.f, 2500.f, 10000.f };

	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float BufferHysteresisOnDistancePercentage = 10.0f;

	/** Filter these settings with specified tag */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (BaseStruct = "/Script/MassEntity.MassTag"))
	TObjectPtr<UScriptStruct> FilterTag = nullptr;
};

/** Simplest version of LOD Calculation based strictly on Distance parameters 
 *	Compared to FMassVisualizationLODSharedFragment, we:
 *	* Cannot Adjust the Distance from count
 *	* We care about a MassLODCalculator with a new LOD logic that excludes Visibility computation
 */
USTRUCT()
struct FMassDistanceLODSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	FMassDistanceLODSharedFragment() = default;
	FMassDistanceLODSharedFragment(const FMassDistanceLODParameters& LODParams);

	TMassLODCalculator<FMassDistanceLODLogic> LODCalculator;

	UPROPERTY(Transient)
	TObjectPtr<const UScriptStruct> FilterTag = nullptr;
};
