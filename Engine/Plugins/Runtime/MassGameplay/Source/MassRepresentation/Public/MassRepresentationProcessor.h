// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODFragments.h"
#include "MassRepresentationFragments.h"
#include "MassActorSpawnerSubsystem.h"
#include "MassObserverProcessor.h"
#include "MassRepresentationProcessor.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMassRepresentationSubsystem;
class UMassActorSubsystem;
struct FMassActorFragment;


namespace UE::Mass::Representation
{
	extern MASSREPRESENTATION_API int32 bAllowKeepActorExtraFrame;
	extern MASSREPRESENTATION_API int32 bAllowWaitForActorVisualReadiness;

	/**
	 * Computes whether to keep the actor visible an extra frame across an actor->ISM transition.
	 * Combines the cvar kill-switch with the per-archetype params.
	 */
	inline bool ShouldKeepActorExtraFrame(const FMassRepresentationParameters& Params)
	{
		return bAllowKeepActorExtraFrame != 0 && Params.bKeepLowResActors && Params.bKeepActorExtraFrame;
	}
}

USTRUCT()
struct FMassRepresentationUpdateParams
{
	GENERATED_BODY()

	/** 
	 * Controls whether UpdateRepresentation run will ask the RepresentationSubsystem whether the relevant world 
	 * collision has been already loaded while considering switching an entity to a actor-level representation. Note that
	 * the test is unnecessary for static nor stationary entities.
	 */
	UPROPERTY(config, EditDefaultsOnly, Category = "Mass")
	bool bTestCollisionAvailibilityForActorVisualization = true;
};

UCLASS(MinimalAPI, Abstract)
class UMassRepresentationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassRepresentationProcessor();

	/*
	 * Update representation type for each entity, must be called within a ForEachEntityChunk
	 * @param Context of the execution from the entity sub system
	 */
	static UE_API void UpdateRepresentation(FMassExecutionContext& Context, const FMassRepresentationUpdateParams& Params);

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	/** 
	 * Execution method for this processor 
	 * @param EntityManager is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/**
	 * Release the actor to the subsystem, will only release it the actor or spawn request matches the template actor
	 * @param RepresentationSubsystem to use to release the actor or cancel the spawning
	 * @param MassAgent is the handle to the associated mass agent
	 * @param ActorInfo is the fragment where we are going to store the actor pointer
	 * @param TemplateActorIndex is the index of the type to release
	 * @param SpawnRequestHandle (in/out) In: previously requested spawn to cancel if any
	 * @param CommandBuffer to queue up anything that is thread sensitive
	 * @param bCancelSpawningOnly tell to only cancel the existing spawning request and to not release the associated actor it any.
	 * @return if the actor was release or the spawning was canceled.
	 */
	static UE_API bool ReleaseActorOrCancelSpawning(UMassRepresentationSubsystem& RepresentationSubsystem, UMassActorSubsystem* MassActorSubsystem
		, const FMassEntityHandle MassAgent, FMassActorFragment& ActorInfo, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle
		, FMassCommandBuffer& CommandBuffer, const bool bCancelSpawningOnly = false);

	FMassEntityQuery EntityQuery;

	UPROPERTY(config, EditDefaultsOnly, Category = "Mass")
	FMassRepresentationUpdateParams UpdateParams;
};

	
/**
 * Tag required by Visualization Processor to process given archetype. Removing the tag allows to support temporary
 * disabling of processing for individual entities of given archetype.
 */
USTRUCT()
struct FMassVisualizationProcessorTag : public FMassTag
{
	GENERATED_BODY();
};

UCLASS(MinimalAPI)
class UMassVisualizationProcessor : public UMassRepresentationProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassVisualizationProcessor();

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	/**
	 * Execution method for this processor
	 * @param EntityManager is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/**
	 * Updates chunk visibility info for later chunk logic optimization
	 * @param Context of the execution from the entity sub system
	 * @return The visualization chunk fragment
	 */
	UE_API FMassVisualizationChunkFragment& UpdateChunkVisibility(FMassExecutionContext& Context) const;

	/**
	 * Updates entity visibility tag for later chunk logic optimization
	 * @param Entity of the entity to update visibility on
	 * @param Representation fragment containing the current and previous visual state
	 * @param RepresentationLOD fragment containing the visibility information
	 * @param ChunkData is the visualization chunk fragment
	 * @param CommandBuffer to queue up anything that is thread sensitive
	 */
	static UE_API void UpdateEntityVisibility(const FMassEntityHandle Entity, const FMassRepresentationFragment& Representation, const FMassRepresentationLODFragment& RepresentationLOD, FMassVisualizationChunkFragment& ChunkData, FMassCommandBuffer& CommandBuffer);

	/**
	 * Update representation and visibility for each entity, must be called within a ForEachEntityChunk
	 * @param Context of the execution from the entity sub system
	 */
	UE_API void UpdateVisualization(FMassExecutionContext& Context);
};


UCLASS(MinimalAPI)
class UMassRepresentationFragmentDestructor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassRepresentationFragmentDestructor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
