// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Misc/MTAccessDetector.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "MassActorSpawnerSubsystem.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassRepresentationSubsystem.generated.h"

class UMassVisualizationComponent;
class AMassVisualizer;
struct FStaticMeshInstanceVisualizationDesc;
struct FMassInstancedStaticMeshInfo;
struct FMassActorSpawnRequestHandle;
class UMassActorSpawnerSubsystem;
class UMassAgentComponent;
struct FMassEntityManager;
enum class EMassProcessingPhase : uint8;
class UWorldPartitionSubsystem;

/**
 * Subsystem responsible for all visual of mass agents, will handle actors spawning and static mesh instances
 */
UCLASS(MinimalAPI)
class UMassRepresentationSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:
	/** 
	 * Get the index of the static mesh visual type, will add a new one if does not exist  
	 * @param Desc is the information for the static mesh that will be instantiated later via AddStaticMeshInstance()
	 * @return The index of the static mesh type 
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle FindOrAddStaticMeshDesc(const FStaticMeshInstanceVisualizationDesc& Desc);

	/**
	 * Creates a dedicated visual type described by host Desc and ties ISMComponent to it.
	 * @note this is a helper function for a common "single ISMComponent" case. Calls AddVisualDescWithISMComponents under the hood.
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle AddVisualDescWithISMComponent(const FStaticMeshInstanceVisualizationDesc& Desc, UInstancedStaticMeshComponent& ISMComponent);

	/**
	 * Creates a dedicated visual type described by host Desc and ties given ISMComponents to it.
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle AddVisualDescWithISMComponents(const FStaticMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedStaticMeshComponent>> ISMComponents);

	/**
	 * Fetches FMassISMCSharedData indicated by DescriptionIndex, or nullptr if it's not a valid index
	 */
	MASSREPRESENTATION_API const FMassISMCSharedData* GetISMCSharedDataForDescriptionIndex(const int32 DescriptionIndex) const;

	/**
	 * Fetches FMassISMCSharedData indicated by an ISMC, or nullptr if the ISMC is not represented by any shared data.
	 */
	MASSREPRESENTATION_API const FMassISMCSharedData* GetISMCSharedDataForInstancedStaticMesh(const UInstancedStaticMeshComponent* ISMC) const;

	/** 
	 * Removes all data associated with a given VisualizationIndex. Note that this is safe to do only if there are no
	 * entities relying on this index. No entity data patching will take place.
	 */
	MASSREPRESENTATION_API void RemoveVisualDesc(const FStaticMeshInstanceVisualizationDescHandle VisualizationHandle);

	/** 
	 * @return the array of all the static mesh instance component information
	 */
	MASSREPRESENTATION_API FMassInstancedStaticMeshInfoArrayView GetMutableInstancedStaticMeshInfos();

	/** Mark render state of the static mesh instances dirty */
	MASSREPRESENTATION_API void DirtyStaticMeshInstances();

	/** 
	 * Store the template actor uniquely and return an index to it 
	 * @param ActorClass is a template actor class we will need to spawn for an agent 
	 * @return The index of the template actor type
	 */
	MASSREPRESENTATION_API int16 FindOrAddTemplateActor(const TSubclassOf<AActor>& ActorClass);

	/** 
	 * Get or spawn an actor from the TemplateActorIndex
	 * @param MassAgent is the handle to the associated mass agent
	 * @param Transform where to create this actor
	 * @param TemplateActorIndex is the index of the type fetched with FindOrAddTemplateActor()
	 * @param SpawnRequestHandle [IN/OUT] IN: previously requested spawn OUT: newly requested spawn
	 * @param Priority of this spawn request in comparison with the others, lower value means higher priority (optional)
	 * @param ActorPreSpawnDelegate is an optional delegate called before the spawning of an actor
	 * @param ActorPostSpawnDelegate is an optional delegate called once the actor is spawned
	 * @return The spawned actor from the template actor type if ready
	 */
	MASSREPRESENTATION_API AActor* GetOrRequestSpawnActorFromTemplate(const FMassEntityHandle MassAgent, const FTransform& Transform, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& InOutSpawnRequestHandle, float Priority = MAX_FLT,
		FMassActorPreSpawnDelegate ActorPreSpawnDelegate = FMassActorPreSpawnDelegate(), FMassActorPostSpawnDelegate ActorPostSpawnDelegate = FMassActorPostSpawnDelegate());

	/**
	 * Cancel spawning request that is matching the TemplateActorIndex
	 * @param MassAgent is the handle to the associated mass agent
	 * @param TemplateActorIndex is the template type of the actor to release in case it was successfully spawned
	 * @param SpawnRequestHandle [IN/OUT] previously requested spawn, gets invalidated as a result of this call.
	 * @return True if spawning request was canceled
	 */
	MASSREPRESENTATION_API bool CancelSpawning(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle & SpawnRequestHandle);

	/**
	 * Release an actor that is matching the TemplateActorIndex
	 * @param MassAgent is the handle to the associated mass agent
	 * @param TemplateActorIndex is the template type of the actor to release in case it was successfully spawned
	 * @param ActorToRelease is the actual actor to release if any
	 * @param bImmediate means it needs to be done immediately and not queue for later
	 * @return True if actor was released
	 */
	MASSREPRESENTATION_API bool ReleaseTemplateActor(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, AActor* ActorToRelease, bool bImmediate);

	/**
	 * Release an actor or cancel its spawning if it is matching the TemplateActorIndex
	 * @param MassAgent is the handle to the associated mass agent
	 * @param TemplateActorIndex is the template type of the actor to release in case it was successfully spawned
	 * @param ActorToRelease is the actual actor to release if any
	 * @param SpawnRequestHandle [IN/OUT] previously requested spawn, gets invalidated as a result of this call.
	 * @param bImmediate whether to perform the actor destruction immediately, otherwise it will be queued for processing later
	 * @return True if actor was released or spawning request was canceled
	 */
	MASSREPRESENTATION_API bool ReleaseTemplateActorOrCancelSpawning(const FMassEntityHandle MassAgent, const int16 TemplateActorIndex, AActor* ActorToRelease, FMassActorSpawnRequestHandle& SpawnRequestHandle, bool bImmediate = false);


	/**
	 * Compare if an actor matches the registered template actor
	 * @param Actor to compare its class against the template
	 * @param TemplateActorIndex is the template type of the actor to compare against
	 * @return True if actor matches the template
	 */
	MASSREPRESENTATION_API bool DoesActorMatchTemplate(const AActor& Actor, const int16 TemplateActorIndex) const;

	MASSREPRESENTATION_API TSubclassOf<AActor> GetTemplateActorClass(const int16 TemplateActorIndex);

	MASSREPRESENTATION_API bool IsCollisionLoaded(const FName TargetGrid, const FTransform& Transform) const;

	/**
	 * Responds to the FMassEntityTemplate getting destroyed, and releases reference to corresponding Actor in TemplateActors
	 */
	MASSREPRESENTATION_API void ReleaseTemplate(const TSubclassOf<AActor>& ActorClass);

	/**
	 * Release all references to static meshes and template actors
	 * Use with caution, all entities using this representation subsystem must be destroy otherwise they will point to invalid resources */
	MASSREPRESENTATION_API void ReleaseAllResources();

	UMassActorSpawnerSubsystem* GetActorSpawnerSubsystem() const { return ActorSpawnerSubsystem; }

	/** Access to the visualization component for animation state readback */
	UMassVisualizationComponent* GetVisualizationComponent() const { return VisualizationComponent; }

protected:
	// USubsystem BEGIN
	MASSREPRESENTATION_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSREPRESENTATION_API virtual void Deinitialize() override;
	// USubsystem END

	MASSREPRESENTATION_API bool SpawnVisualizer(TNotNull<UWorld*> World);

	/** Needed for batching the update of static mesh transform */
	MASSREPRESENTATION_API void OnProcessingPhaseStarted(const float DeltaSeconds, const EMassProcessingPhase Phase) const;

	MASSREPRESENTATION_API void OnMassAgentComponentEntityAssociated(const UMassAgentComponent& AgentComponent);
	MASSREPRESENTATION_API void OnMassAgentComponentEntityDetaching(const UMassAgentComponent& AgentComponent);

	MASSREPRESENTATION_API bool ReleaseTemplateActorInternal(const int16 TemplateActorIndex, AActor* ActorToRelease, bool bImmediate);
	MASSREPRESENTATION_API bool CancelSpawningInternal(const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle, bool bImmediateActorRelease = false);

	static MASSREPRESENTATION_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:

	/**
	 * @return the array of all the skinned mesh instance component information
	 */
	MASSREPRESENTATION_API FMassInstancedSkinnedMeshInfoArrayView GetMutableInstancedSkinnedMeshInfos();

	/** 
	 * Get the index of the skinned mesh visual type, will add a new one if does not exist
	 * @param Desc is the information for the mesh that will be instantiated later when adding a skinned mesh instance
	 * @return The index of the mesh type
	 */
	MASSREPRESENTATION_API FSkinnedMeshInstanceVisualizationDescHandle FindOrAddSkinnedMeshVisualizationDesc(const FSkinnedMeshInstanceVisualizationDesc& Desc);

protected:

	struct FTemplateActorData
	{
		TSubclassOf<AActor> Actor;
		uint32 RefCount{0u};
	};
	
	struct FTemplateActorEqualsPredicate
	{
		const TSubclassOf<AActor>& ActorClass;

		FTemplateActorEqualsPredicate(const TSubclassOf<AActor>& ActorClass) : ActorClass(ActorClass) {}

		bool operator()(const FTemplateActorData& ActorData) const
		{
			return ActorData.Actor == ActorClass;
		}
	};

	/** The array of all the template actors */
	TSparseArray<FTemplateActorData> TemplateActors;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(TemplateActorsMTAccessDetector);

	/** The component that handles all the static mesh instances */
	UPROPERTY(Transient)
	TObjectPtr<UMassVisualizationComponent> VisualizationComponent;

	/** The actor owning the above visualization component */
	UPROPERTY(Transient)
	TObjectPtr<AMassVisualizer> Visualizer;

	UPROPERTY(Transient)
	TObjectPtr<UMassActorSpawnerSubsystem> ActorSpawnerSubsystem;

	TSharedPtr<FMassEntityManager> EntityManager;

	UPROPERTY(Transient)
	TObjectPtr<UWorldPartitionSubsystem> WorldPartitionSubsystem;

	/** The time to wait before retrying a to spawn actor that failed */
	float RetryMovedDistanceSq = 1000000.0f;

	/** The distance a failed spawned actor needs to move before we retry */
	float RetryTimeInterval = 10.0f;

	/** Keeping track of all the mass agent this subsystem is responsible for spawning actors */
	TMap<FMassEntityHandle, int32> HandledMassAgents;

	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
public:
	UE_DEPRECATED(5.8, "Use GetOrRequestSpawnActorFromTemplate instead.")
	AActor* GetOrSpawnActorFromTemplate(const FMassEntityHandle MassAgent, const FTransform& Transform, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& InOutSpawnRequestHandle, float Priority = MAX_FLT,
		FMassActorPreSpawnDelegate ActorPreSpawnDelegate = FMassActorPreSpawnDelegate(), FMassActorPostSpawnDelegate ActorPostSpawnDelegate = FMassActorPostSpawnDelegate())
	{
		return GetOrRequestSpawnActorFromTemplate(MassAgent, Transform, TemplateActorIndex, InOutSpawnRequestHandle, Priority, MoveTemp(ActorPreSpawnDelegate), MoveTemp(ActorPostSpawnDelegate));
	}

private:
	UFUNCTION()
	MASSREPRESENTATION_API void HandleVisualizerEndPlay(AActor* Actor, const EEndPlayReason::Type EndPlayReason);

	/** Used for default return value from GetMutableInstancedStaticMeshInfos */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstancedStaticMeshInfosDetector);

	/** Used for default return value from GetMutableInstancedSkinedMeshInfos */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstancedSkinnedMeshInfosDetector);
};
