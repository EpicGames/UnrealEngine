// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODTypes.h"
#include "Mass/EntityHandle.h"
#include "Engine/DataTable.h"
#include "Misc/MTAccessDetector.h"
#include "InstanceDataTypes.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "UObject/ObjectKey.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Math/MathFwd.h"
#include "MassRepresentationTypes.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMaterialInterface;
class UStaticMesh;
struct FMassLODSignificanceRange;
class UMassVisualizationComponent;

DECLARE_LOG_CATEGORY_EXTERN(LogMassRepresentation, Log, All);

namespace UE::Mass::ProcessorGroupNames
{
	const FName Representation = FName(TEXT("Representation"));
	// Sub-group of Representation containing every UMassVisualizationProcessor subclass.
	// Consumers that depend on CurrentRepresentation being finalised should ExecuteAfter this.
	const FName VisualizationProcessing = FName(TEXT("Representation.VisualizationProcessing"));
}

using FISMCSharedDataKey = TObjectKey<UInstancedStaticMeshComponent>;

UENUM()
enum class EMassRepresentationType : uint8
{
	HighResSpawnedActor,
	LowResSpawnedActor,
	SkinnedMeshInstance,
	StaticMeshInstance,
	None,
};

namespace UE::Mass::Representation
{
	UE_API inline bool IsValidMeshRepresentation(EMassRepresentationType RepresentationType)
	{
		return RepresentationType == EMassRepresentationType::SkinnedMeshInstance || RepresentationType == EMassRepresentationType::StaticMeshInstance;
	}

	UE_API inline bool IsValidActorRepresentation(EMassRepresentationType RepresentationType)
	{
		return RepresentationType == EMassRepresentationType::HighResSpawnedActor || RepresentationType == EMassRepresentationType::LowResSpawnedActor;
	}

	/**
	 * Debug-draw color per EMassRepresentationType, shared across Mass visualization debug paths.
	 * The array is sized to cover every enum value (including None);
	 * a static_assert in MassRepresentationTypes.cpp keeps it in sync with the enum.
	 */
	extern UE_API const FColor RepresentationTypeColors[];
}

enum class EMassActorEnabledType : uint8
{
	Disabled,
	LowRes,
	HighRes,
};

USTRUCT()
struct FMassStaticMeshInstanceVisualizationMeshDesc
{
	GENERATED_BODY()

	UE_API FMassStaticMeshInstanceVisualizationMeshDesc();

	/** A relative Transform from the parent, passed along to FMassISMCSharedData::LocalTransform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FTransform LocalTransform = FTransform::Identity;
	
	/** The static mesh visual representation */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TObjectPtr<UStaticMesh> Mesh = nullptr;

	/**
	 * Material overrides for the static mesh visual representation. 
	 * 
	 * Array indices correspond to material slot indices on the static mesh.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TArray<TObjectPtr<UMaterialInterface>> MaterialOverrides;

	/** The minimum inclusive LOD significance to start using this static mesh */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	float MinLODSignificance = float(EMassLOD::High);

	/** The maximum exclusive LOD significance to stop using this static mesh */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	float MaxLODSignificance = float(EMassLOD::Max);

	/** Controls whether the ISM can cast shadow or not */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bCastShadows = false;

	/**
	 * Determines the value of corresponding FMassISMCSharedData.bRequiresExternalInstanceIDTracking.
	 * Recommended enabled for stationary meshes.
	 * @see FMassISMCSharedData.bRequiresExternalInstanceIDTracking for details.
	 */
	bool bRequiresExternalInstanceIDTracking = false;
	
	/** Controls the mobility of the ISM */
	EComponentMobility::Type Mobility = EComponentMobility::Movable;

	/** InstancedStaticMeshComponent class to use to manage instances described by this struct instance */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<UInstancedStaticMeshComponent> ISMComponentClass;

	bool operator==(const FMassStaticMeshInstanceVisualizationMeshDesc& Other) const
	{		
		return Mesh == Other.Mesh && 
			MaterialOverrides == Other.MaterialOverrides && 
			FMath::IsNearlyEqual(MinLODSignificance, Other.MinLODSignificance, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(MaxLODSignificance, Other.MaxLODSignificance, KINDA_SMALL_NUMBER) &&
			bCastShadows == Other.bCastShadows && 
			Mobility == Other.Mobility &&
			LocalTransform.Equals(Other.LocalTransform);
	}
	
	friend inline uint32 GetTypeHash(const FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc)
	{
		uint32 Hash = 0x0;
		Hash = PointerHash(MeshDesc.Mesh, Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.bCastShadows), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.Mobility), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.LocalTransform), Hash);
		for (UMaterialInterface* MaterialOverride : MeshDesc.MaterialOverrides)
		{
			if (MaterialOverride)
			{
				Hash = PointerHash(MaterialOverride, Hash);
			}
		}
		return Hash;
	}

	// convenience function for setting MinLODSinificance and MaxLODSinificance based on EMassLOD values
	void SetSignificanceRange(const EMassLOD::Type MinLOD, const EMassLOD::Type MaxLOD)
	{
		checkSlow(MinLOD <= MaxLOD);
		MinLODSignificance = float(MinLOD);
		MaxLODSignificance = float(MaxLOD);
	}
};

USTRUCT()
struct FStaticMeshInstanceVisualizationDesc : public FTableRowBase
{
	GENERATED_BODY()

	/** 
	 * Mesh descriptions. These will be instanced together using the same transform for each, to 
	 * visualize the given agent.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta=(TitleProperty="{Mesh}"))
	TArray<FMassStaticMeshInstanceVisualizationMeshDesc> Meshes;

	/** Boolean to enable code to transform the static meshes if not align the mass agent transform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bUseTransformOffset = false;

	/** Transform to offset the static meshes if not align the mass agent transform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta=(EditCondition="bUseTransformOffset"))
	FTransform TransformOffset = FTransform::Identity;

	/** Custom data that can eventually be propagated to UInstancedStaticMeshComponent::PerInstanceSMCustomData */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TArray<float> CustomDataFloats;

	bool operator==(const FStaticMeshInstanceVisualizationDesc& Other) const
	{
		return Meshes == Other.Meshes && bUseTransformOffset == Other.bUseTransformOffset && TransformOffset.Equals(Other.TransformOffset);
	}

	void Reset()
	{
		Meshes.Empty();
		bUseTransformOffset = false;
		TransformOffset = FTransform::Identity;
		CustomDataFloats.Empty();
	}

	/** @return whether any of descriptions in Meshes is valid. This implies that empty Meshes will be treated as not valid.*/
	UE_API bool IsValid() const;
};

/** Handle for FStaticMeshInstanceVisualizationDesc's registered with UMassRepresentationSubsystem */
USTRUCT()
struct alignas(2) FStaticMeshInstanceVisualizationDescHandle
{
	GENERATED_BODY()

	static constexpr uint16 InvalidIndex = TNumericLimits<uint16>::Max();

	FStaticMeshInstanceVisualizationDescHandle() = default;

	explicit FStaticMeshInstanceVisualizationDescHandle(uint16 InIndex)
	: Index(InIndex)
	{}

	explicit FStaticMeshInstanceVisualizationDescHandle(int32 InIndex) 
	{
		// Handle special case INDEX_NONE = InvalidIndex
		if (InIndex == INDEX_NONE)
		{
			Index = InvalidIndex;
		}
		else
		{
			checkf(InIndex < static_cast<int32>(InvalidIndex), TEXT("Visualization description index InIndex %d is out of expected bounds (< %u)"), InIndex, InvalidIndex);
			Index = static_cast<uint16>(InIndex);
		}
	}

	inline int32 ToIndex() const
	{
		return IsValid() ? Index : INDEX_NONE;
	}

	bool IsValid() const
	{
		return Index != InvalidIndex;
	}

	bool operator==(const FStaticMeshInstanceVisualizationDescHandle& Other) const = default;

private:

	UPROPERTY()
	uint16 Index = InvalidIndex;

	// @todo: Add a version / serial number to protect against recycled handle reuse. Leaving this out for now to keep size down due to 
	// prevalent use in FMassRepresentationFragment. Perhaps serial number could be formed from the referenced 
	// FStaticMeshInstanceVisualizationDesc's hash.
};
static_assert(sizeof(FStaticMeshInstanceVisualizationDescHandle) == sizeof(uint16), "FStaticMeshInstanceVisualizationDescHandle must be uint16 sized to ensure FMassRepresentationFragment memory isn't unexpectedly bloated");

class UInstancedStaticMeshComponent;


struct FMassISMCSharedData
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMassISMCSharedData() 
		: bRequiresExternalInstanceIDTracking(false)
	{		
	}

	explicit FMassISMCSharedData(UInstancedStaticMeshComponent* InISMC, bool bInRequiresExternalInstanceIDTracking = false, const FTransform& InTransformOffset = FTransform::Identity)
		: LocalTransform(InTransformOffset), ISMC(InISMC), bRequiresExternalInstanceIDTracking(bInRequiresExternalInstanceIDTracking)
	{
	}

	FMassISMCSharedData(const FMassISMCSharedData& Other) = default;
	FMassISMCSharedData& operator=(const FMassISMCSharedData& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void SetISMComponent(UInstancedStaticMeshComponent& InISMC)
	{
		check(ISMC == nullptr && ISMComponentReferencesCount == 0);
		ISMC = &InISMC;
	}

	UInstancedStaticMeshComponent* GetMutableISMComponent() { return ISMC; }
	const UInstancedStaticMeshComponent* GetISMComponent() const { return ISMC; }
	int32 OnISMComponentReferenceStored() { return ++ISMComponentReferencesCount; }
	int32 OnISMComponentReferenceReleased() { ensure(ISMComponentReferencesCount >= 0); return --ISMComponentReferencesCount; }

	void ResetAccumulatedData()
	{
		UpdateInstanceIds.Reset();
		StaticMeshInstanceCustomFloats.Reset();
		StaticMeshInstanceTransforms.Reset();
		StaticMeshInstancePrevTransforms.Reset();
		RemoveInstanceIds.Reset();
		WriteIterator = 0;
	}

	void RemoveUpdatedInstanceIdsAtSwap(const int32 InstanceIDIndex)
	{
		UpdateInstanceIds.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		StaticMeshInstanceTransforms.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		StaticMeshInstancePrevTransforms.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		if (StaticMeshInstanceCustomFloats.Num())
		{
			StaticMeshInstanceCustomFloats.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		}
	}

	bool HasUpdatesToApply() const { return UpdateInstanceIds.Num() || RemoveInstanceIds.Num(); }
	TConstArrayView<FMassEntityHandle> GetUpdateInstanceIds() const { return UpdateInstanceIds; }
	TConstArrayView<FTransform> GetStaticMeshInstanceTransforms() const { return StaticMeshInstanceTransforms; }
	/** 
	 * this function is a flavor we need to interact with older engine API that's using TArray references. 
	 * Use GetStaticMeshInstanceTransforms instead whenever possible. 
	 */
	const TArray<FTransform>& GetStaticMeshInstanceTransformsArray() const { return StaticMeshInstanceTransforms; }
	TConstArrayView<FTransform> GetStaticMeshInstancePrevTransforms() const { return StaticMeshInstancePrevTransforms; }
	TConstArrayView<FMassEntityHandle> GetRemoveInstanceIds() const { return RemoveInstanceIds; }
	TConstArrayView<float> GetStaticMeshInstanceCustomFloats() const { return StaticMeshInstanceCustomFloats; }
	
	bool RequiresExternalInstanceIDTracking() const { return bRequiresExternalInstanceIDTracking; }

	void Reset() 
	{
		*this = FMassISMCSharedData();
	}

	using FEntityToPrimitiveIdMap = Experimental::TRobinHoodHashMap<FMassEntityHandle, FPrimitiveInstanceId>;

	FEntityToPrimitiveIdMap& GetMutableEntityPrimitiveToIdMap() { return EntityHandleToPrimitiveIdMap; }
	const FEntityToPrimitiveIdMap& GetEntityPrimitiveToIdMap() const { return EntityHandleToPrimitiveIdMap; }

	int16 GetComponentInstanceIdTouchCounter() const { return ComponentInstanceIdTouchCounter; }

protected:
	friend FMassLODSignificanceRange;
	friend UMassVisualizationComponent;

	FTransform LocalTransform = FTransform::Identity;

	/** Buffer holding current frame transforms for the static mesh instances, used to batch update the transforms */
	TArray<FMassEntityHandle> UpdateInstanceIds;
	TArray<FTransform> StaticMeshInstanceTransforms;
	TArray<FTransform> StaticMeshInstancePrevTransforms;
	TArray<FMassEntityHandle> RemoveInstanceIds;

	/** Buffer holding current frame custom floats for the static mesh instances, used to batch update the ISMs custom data */
	TArray<float> StaticMeshInstanceCustomFloats;

	// When initially adding to StaticMeshInstanceCustomFloats, can use the size as the write iterator, but on subsequent processors, we need to know where to start writing
	int32 WriteIterator = 0;

	UInstancedStaticMeshComponent* ISMC = nullptr;
	int32 ISMComponentReferencesCount = 0;

	/** 
	 * When set to true will result in MassVisualizationComponent manually perform Instance ID-related operations 
	 * instead of relying on ISMComponent's internal ID operations. 
	 * @note this mechanism has been added in preparation of changes to ISM component to change access to its internal 
	 *	instance ID logic. WIP as of Jun 17th 2023 
	 */
	uint8 bRequiresExternalInstanceIDTracking : 1;
	
private:
	/** Indicates that mutating changes, that can affect MassInstanceIdToComponentInstanceIdMap, have been performed.
	 *	Can be used to validate whether cached data stored in other placed needs to be re-cached. */
	uint16 ComponentInstanceIdTouchCounter = 0;

protected:
	FEntityToPrimitiveIdMap EntityHandleToPrimitiveIdMap;
};


/** 
 * The container type hosting FMassISMCSharedData instances and supplying functionality of marking entries that require 
 * instance-related operations (adding, removing). 
 * 
 * To get a FMassISMCSharedData instance to add operations to it call GetAndMarkDirty.
 * 
 * Use FDirtyIterator to iterate over just the data that needs processing. 
 * 
 * @see UMassVisualizationComponent::EndVisualChanges for iteration
 * @see FMassLODSignificanceRange methods for performing dirtying operations
 */
struct FMassISMCSharedDataMap
{
	struct FDirtyIterator
	{
		friend FMassISMCSharedDataMap;
		explicit FDirtyIterator(FMassISMCSharedDataMap& InContainer)
			: Container(InContainer), It(InContainer.GetDirtyArray())
		{
			if (It && It.GetValue() != bValueToCheck)
			{
				// will result in either setting IT to the first bInValue, or making bool(It) == false
				++(*this);
			}
		}
	public:
		operator bool() const { return bool(It); }

		FDirtyIterator& operator++()
		{
			while (++It)
			{
				if (It.GetValue() == bValueToCheck)
				{
					break;
				}
			}
			return *this;
		}

		FMassISMCSharedData& operator*() const
		{
			return Container.GetAtIndex(It.GetIndex());
		}

		void ClearDirtyFlag()
		{
			It.GetValue() = false;
		}

	private:
		FMassISMCSharedDataMap& Container;
		TBitArray<>::FIterator It;
		static constexpr bool bValueToCheck = true;
	};

	FMassISMCSharedData& GetAndMarkDirtyChecked(const FISMCSharedDataKey OwnerKey)
	{
		const int32 DataIndex = Map[OwnerKey];
		DirtyData[DataIndex] = true;
		return Data[DataIndex];
	}

	FMassISMCSharedData* GetAndMarkDirty(const FISMCSharedDataKey OwnerKey)
	{
		const int32* DataIndex = Map.Find(OwnerKey);
		if (ensureMsgf(DataIndex, TEXT("%hs Failed to find OwnerKey %s"), __FUNCTION__, *GetNameSafe(OwnerKey.ResolveObjectPtrEvenIfGarbage())))
		{
			DirtyData[*DataIndex] = true;
			return &Data[*DataIndex];
		}
		return nullptr;
	}
	
	template<typename... TArgs>
	FMassISMCSharedData& FindOrAdd(const FISMCSharedDataKey OwnerKey, TArgs&&... InNewInstanceArgs)
	{
		const int32* DataIndex = Map.Find(OwnerKey);
		if (DataIndex == nullptr)
		{
			return Add(OwnerKey, Forward<TArgs>(InNewInstanceArgs)...);
		}
		check(Data.IsValidIndex(*DataIndex));
		return Data[*DataIndex];
	}

	FMassISMCSharedData* Find(const FISMCSharedDataKey OwnerKey)
	{
		int32* DataIndex = Map.Find(OwnerKey);
		return (DataIndex == nullptr || *DataIndex == INDEX_NONE) ? (FMassISMCSharedData*)nullptr : &Data[*DataIndex];
	}

	template<typename... TArgs>
	FMassISMCSharedData& Add(const FISMCSharedDataKey OwnerKey, TArgs&&... InNewInstanceArgs)
	{
		const int32 DataIndex = FreeIndices.Num() ? FreeIndices.Pop() : Data.Num();
		Map.Add(OwnerKey, DataIndex);

		if (DataIndex == Data.Num())
		{
			DirtyData.Add(false, DataIndex - DirtyData.Num() + 1);
			DirtyData[DataIndex] = true;
			return Data.Add_GetRef(FMassISMCSharedData(Forward<TArgs>(InNewInstanceArgs)...));
		}
		else
		{
			DirtyData[DataIndex] = true;
			Data[DataIndex] = FMassISMCSharedData(Forward<TArgs>(InNewInstanceArgs)...);
			return Data[DataIndex];
		}
	}

	void Remove(const FISMCSharedDataKey OwnerKey)
	{
		int32 DataIndex = INDEX_NONE;
		if (ensure(Map.RemoveAndCopyValue(OwnerKey, DataIndex)))
		{
			DirtyData[DataIndex] = false;
			Data[DataIndex].Reset();
			FreeIndices.Add(DataIndex);
		}
	}

	FMassISMCSharedData& GetAtIndex(const int32 DataIndex)
	{
		return Data[DataIndex];
	}
	
	TBitArray<>& GetDirtyArray()
	{ 
		return DirtyData;
	}

	/** @return total number of entries in Data array. Note that some or all entries could be empty (i.e. already freed) */
	int32 Num() const
	{
		return Data.Num();
	}

	/** @return number of non-empty entries in Data. */
	int32 NumValid() const
	{
		return Data.Num() - FreeIndices.Num();
	}

	bool IsDirty(const int32 DataIndex) const
	{
		return DirtyData[DataIndex];
	}

	bool IsEmpty() const
	{
		return NumValid() == 0;
	}

	void Reset()
	{
		*this = FMassISMCSharedDataMap();
	}

	const FMassISMCSharedData* GetDataForIndex(const int32 Index) const
	{
		return Data.IsValidIndex(Index) ? &Data[Index] : nullptr;
	}

	const FMassISMCSharedData* GetDataForKey(const FISMCSharedDataKey Key) const
	{
		const int32* Index = Map.Find(Key);
		return (Index && Data.IsValidIndex(*Index))
			? &Data[*Index]
			: nullptr;
	}

protected:
	TArray<FMassISMCSharedData> Data;
	/** Mapping from Owner (as FObjectKey) of data represented by FMassISMCSharedData to an index to Data */
	TMap<FISMCSharedDataKey, int32> Map;
	/** Indicates whether corresponding Data entry has any instance work assigned to it (instance addition or removal) */
	TBitArray<> DirtyData;
	/** Indices to Data that are available for reuse */
	TArray<int32> FreeIndices;
};


USTRUCT()
struct FMassLODSignificanceRange
{
	GENERATED_BODY()
public:

	UE_API void AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, TConstArrayView<FISMCSharedDataKey> ExcludeStaticMeshRefs);

	// Adds the specified struct reinterpreted as custom floats to our custom data. Individual members of the specified struct should always fit into a float.
	// When adding any custom data, the custom data must be added for every instance.
	template<typename InCustomDataType>
	void AddBatchedCustomData(InCustomDataType InCustomData, const TArray<FISMCSharedDataKey>& ExcludeStaticMeshRefs, int32 NumFloatsToPad = 0)
	{
		check(ISMCSharedDataPtr);
		static_assert((sizeof(InCustomDataType) % sizeof(float)) == 0, "AddBatchedCustomData: InCustomDataType should have a total size multiple of sizeof(float), and have members that fit in a float's boundaries");
		const size_t StructSize = sizeof(InCustomDataType);
		const size_t StructSizeInFloats = StructSize / sizeof(float);
		for (int i = 0; i < StaticMeshRefs.Num(); i++)
		{
			if (ExcludeStaticMeshRefs.Contains(StaticMeshRefs[i]))
			{
				continue;
			}

			FMassISMCSharedData& SharedData = (*ISMCSharedDataPtr).GetAndMarkDirtyChecked(StaticMeshRefs[i]);
			const int32 StartIndex = SharedData.StaticMeshInstanceCustomFloats.AddDefaulted(StructSizeInFloats + NumFloatsToPad);
			InCustomDataType* CustomData = reinterpret_cast<InCustomDataType*>(&SharedData.StaticMeshInstanceCustomFloats[StartIndex]);
			*CustomData = InCustomData;
		}
	}

	UE_API void AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const TArray<FISMCSharedDataKey>& ExcludeStaticMeshRefs);

	/** Single-instance version of AddBatchedCustomData when called to add entities (as opposed to modify existing ones).*/
	UE_API void AddInstance(const FMassEntityHandle EntityHandle, const FTransform& Transform);

	UE_API void RemoveInstance(const FMassEntityHandle EntityHandle);

	UE_API void WriteCustomDataFloatsAtStartIndex(int32 StaticMeshIndex, const TArrayView<float>& CustomFloats, const int32 FloatsPerInstance, const int32 StartIndex, const TArray<FISMCSharedDataKey>& ExcludeStaticMeshRefs);

	/** LOD Significance range */
	float MinSignificance = 0.0f;
	float MaxSignificance = 0.0f;

	/** The component handling these instances */
	TArray<FISMCSharedDataKey> StaticMeshRefs;

	FMassISMCSharedDataMap* ISMCSharedDataPtr = nullptr;
};

USTRUCT()
struct FMassInstancedStaticMeshInfo
{
	GENERATED_BODY()
public:

	FMassInstancedStaticMeshInfo() = default;

	explicit FMassInstancedStaticMeshInfo(const FStaticMeshInstanceVisualizationDesc& InDesc)
		: Desc(InDesc)
	{
	}

	/** Clears out contents so that a given FMassInstancedStaticMeshInfo instance can be reused */
	UE_API void Reset();

	const FStaticMeshInstanceVisualizationDesc& GetDesc() const
	{
		return Desc;
	}

	/** Whether or not to transform the static meshes if not align the mass agent transform */
	bool ShouldUseTransformOffset() const { return Desc.bUseTransformOffset; }
	FTransform GetTransformOffset() const { return Desc.TransformOffset; }

	inline FMassLODSignificanceRange* GetLODSignificanceRange(float LODSignificance)
	{
		for (FMassLODSignificanceRange& Range : LODSignificanceRanges)
		{
			if (LODSignificance >= Range.MinSignificance && LODSignificance < Range.MaxSignificance)
			{
				return &Range;
			}
		}
		return nullptr;
	}

	void AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedTransform(EntityHandle, Transform, PrevTransform, {});
			if(PrevLODSignificance >= 0.0f)
			{
				FMassLODSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedTransform(EntityHandle, Transform, PrevTransform, Range->StaticMeshRefs);
				}
			}
		}
	}

	inline void RemoveInstance(const FMassEntityHandle EntityHandle, const float LODSignificance)
	{
		if (FMassLODSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->RemoveInstance(EntityHandle);
		}
	}

	// Adds the specified struct reinterpreted as custom floats to our custom data. Individual members of the specified struct should always fit into a float.
	// When adding any custom data, the custom data must be added for every instance.
	template<typename InCustomDataType>
	void AddBatchedCustomData(InCustomDataType InCustomData, const float LODSignificance, const float PrevLODSignificance = -1.0f, int32 NumFloatsToPad = 0)
	{
		if (FMassLODSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedCustomData(InCustomData, {}, NumFloatsToPad);
			if(PrevLODSignificance >= 0.0f)
			{
				FMassLODSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedCustomData(InCustomData, Range->StaticMeshRefs, NumFloatsToPad);
				}
			}
		}
	}

	inline void AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedCustomDataFloats(CustomFloats, {});
			if(PrevLODSignificance >= 0.0f)
			{
				FMassLODSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedCustomDataFloats(CustomFloats, Range->StaticMeshRefs);
				}
			}
		}
	}

	void WriteCustomDataFloatsAtStartIndex(int32 StaticMeshIndex, const TArrayView<float>& CustomFloats, const float LODSignificance, const int32 FloatsPerInstance, const int32 FloatStartIndex, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->WriteCustomDataFloatsAtStartIndex(StaticMeshIndex, CustomFloats, FloatsPerInstance, FloatStartIndex, {});
			if(PrevLODSignificance >= 0.0f)
			{
				FMassLODSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->WriteCustomDataFloatsAtStartIndex(StaticMeshIndex, CustomFloats, FloatsPerInstance, FloatStartIndex, Range->StaticMeshRefs);
				}
			}
		}
	}

	void AddISMComponent(FMassISMCSharedData& SharedData)
	{
		if (ensure(SharedData.GetISMComponent()))
		{
			InstancedStaticMeshComponents.Add(SharedData.GetMutableISMComponent());
			SharedData.OnISMComponentReferenceStored();
		}
	}

	int32 GetLODSignificanceRangesNum() const { return LODSignificanceRanges.Num(); }

	bool IsValid() const
	{
		return Desc.Meshes.Num() && InstancedStaticMeshComponents.Num() && LODSignificanceRanges.Num();
	}

protected:

	/** Destroy the visual instance */
	UE_API void ClearVisualInstance(UInstancedStaticMeshComponent& ISMComponent);

	/** Information about this static mesh which will represent all instances */
	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	FStaticMeshInstanceVisualizationDesc Desc;

	/** The components handling these instances */
	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> InstancedStaticMeshComponents;

	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	TArray<FMassLODSignificanceRange> LODSignificanceRanges;

	friend class UMassVisualizationComponent;
};

#if ENABLE_MT_DETECTOR

#define MAKE_MASS_INSTANCED_STATIC_MESH_INFO_ARRAY_VIEW(ArrayView, AccessDetector) FMassInstancedStaticMeshInfoArrayView(ArrayView, AccessDetector)
struct FMassInstancedStaticMeshInfoArrayViewAccessDetector
{
	FMassInstancedStaticMeshInfoArrayViewAccessDetector(TArrayView<FMassInstancedStaticMeshInfo> InInstancedStaticMeshInfos, const FRWAccessDetector& InAccessDetector)
		: InstancedStaticMeshInfos(InInstancedStaticMeshInfos)
		, AccessDetector(&InAccessDetector)
	{
		UE_MT_ACQUIRE_WRITE_ACCESS(*AccessDetector);
	}

	FMassInstancedStaticMeshInfoArrayViewAccessDetector(FMassInstancedStaticMeshInfoArrayViewAccessDetector&& Other)
		: InstancedStaticMeshInfos(Other.InstancedStaticMeshInfos)
		, AccessDetector(Other.AccessDetector)
	{
		Other.AccessDetector = nullptr;
	}
	FMassInstancedStaticMeshInfoArrayViewAccessDetector(const FMassInstancedStaticMeshInfoArrayViewAccessDetector& Other) = delete;
	void operator=(const FMassInstancedStaticMeshInfoArrayViewAccessDetector& Other) = delete;

	~FMassInstancedStaticMeshInfoArrayViewAccessDetector()
	{
		if(AccessDetector)
		{
			UE_MT_RELEASE_WRITE_ACCESS(*AccessDetector);
		}
	}

	inline FMassInstancedStaticMeshInfo& operator[](int32 Index) const
	{
		return InstancedStaticMeshInfos[Index];
	}

	bool IsValidIndex(const int32 Index) const
	{
		return InstancedStaticMeshInfos.IsValidIndex(Index);
	}

	int32 Num() const
	{
		return InstancedStaticMeshInfos.Num();
	}

private:
	TArrayView<FMassInstancedStaticMeshInfo> InstancedStaticMeshInfos;
	const FRWAccessDetector* AccessDetector;
};

using FMassInstancedStaticMeshInfoArrayView = FMassInstancedStaticMeshInfoArrayViewAccessDetector;

#else // ENABLE_MT_DETECTOR

#define MAKE_MASS_INSTANCED_STATIC_MESH_INFO_ARRAY_VIEW(ArrayView, AccessDetector) ArrayView

using FMassInstancedStaticMeshInfoArrayView = TArrayView<FMassInstancedStaticMeshInfo>;

#endif // ENABLE_MT_DETECTOR

#undef UE_API
