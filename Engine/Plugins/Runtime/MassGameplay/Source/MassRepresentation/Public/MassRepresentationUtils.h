// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassLODFragments.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#include "MassCommonTypes.h"
#include "MassRepresentationTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6


struct FMassCommandBuffer;

namespace UE::Mass::Representation
{

inline EMassVisibility GetVisibilityFromArchetype(const FMassExecutionContext& Context)
{
	if (Context.DoesArchetypeHaveTag<FMassVisibilityCanBeSeenTag>())
	{
		return EMassVisibility::CanBeSeen;
	}
	if (Context.DoesArchetypeHaveTag<FMassVisibilityCulledByFrustumTag>())
	{
		return EMassVisibility::CulledByFrustum;
	}
	if (Context.DoesArchetypeHaveTag<FMassVisibilityCulledByDistanceTag>())
	{
		return EMassVisibility::CulledByDistance;
	}
	return EMassVisibility::Max;
}

template <EMassVisibility Level>
struct TMassVisibilityTagForLevel
{
	typedef FMassTag FTag;
};

template<>
struct TMassVisibilityTagForLevel<EMassVisibility::CanBeSeen>
{
	typedef FMassVisibilityCanBeSeenTag FTag;
};

template<>
struct TMassVisibilityTagForLevel<EMassVisibility::CulledByFrustum>
{
	typedef FMassVisibilityCulledByFrustumTag FTag;
};

template<>
struct TMassVisibilityTagForLevel<EMassVisibility::CulledByDistance>
{
	typedef FMassVisibilityCulledByDistanceTag FTag;
};

inline const UScriptStruct* GetTagFromVisibility(EMassVisibility Visibility)
{
	switch (Visibility)
	{
		case EMassVisibility::CanBeSeen:
			return TMassVisibilityTagForLevel<EMassVisibility::CanBeSeen>::FTag::StaticStruct();
		case EMassVisibility::CulledByFrustum:
			return TMassVisibilityTagForLevel<EMassVisibility::CulledByFrustum>::FTag::StaticStruct();
		case EMassVisibility::CulledByDistance:
			return TMassVisibilityTagForLevel<EMassVisibility::CulledByDistance>::FTag::StaticStruct();
		default:
			checkf(false, TEXT("Unsupported visibility Type"));
		case EMassVisibility::Max:
			return nullptr;
	}
}

inline bool IsVisibilityTagSet(const FMassExecutionContext& Context, EMassVisibility Visibility)
{
	switch (Visibility)
	{
		case EMassVisibility::CanBeSeen:
			return Context.DoesArchetypeHaveTag<FMassVisibilityCanBeSeenTag>();
		case EMassVisibility::CulledByFrustum:
			return Context.DoesArchetypeHaveTag<FMassVisibilityCulledByFrustumTag>();
		case EMassVisibility::CulledByDistance:
			return Context.DoesArchetypeHaveTag<FMassVisibilityCulledByDistanceTag>();
		default:
			checkf(false, TEXT("Unsupported visibility Type"));
		case EMassVisibility::Max:
			return false;
	}
}

void PushSwapTagsCommand(FMassCommandBuffer& CommandBuffer, const FMassEntityHandle Entity, const EMassVisibility PrevVisibility, const EMassVisibility NewVisibility);

} // UE::Mass::Representation