// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequence.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Engine/DataTable.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "InstanceDataTypes.h"
#include "Mass/EntityHandle.h"
#include "MassLODTypes.h"
#include "MassRepresentationTypes.h"
#include "Math/MathFwd.h"
#include "Misc/MTAccessDetector.h"
#include "UObject/ObjectKey.h"

#include "MassSkinnedMeshRepresentationTypes.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMaterialInterface;
struct FMassLODSignificanceRange;
class UMassVisualizationComponent;
class USkinnedAsset;
class UTransformProviderData;

/**
// * Resolved animation state read back from ISKM tracks.
// * Populated during EndVisualChanges() where the ASTPD track data is already hot in cache.
// * Consumed by processors that need to push animation state to UAF.
// */
struct FMassSkinnedMeshResolvedAnimState
{
	int32 SequenceIndex = INDEX_NONE;
	float Position = 0.0f;
	float PlayRate = 1.0f;
	TWeakObjectPtr<UAnimSequence> AnimSequence = nullptr;
};

using FInstancedSkinnedMeshComponentSharedDataKey = TObjectKey<UInstancedSkinnedMeshComponent>;

USTRUCT()
struct FMassSkinnedMeshInstanceVisualizationMeshDesc
{
	GENERATED_BODY()

	UE_API FMassSkinnedMeshInstanceVisualizationMeshDesc();

	/** A relative Transform from the parent, passed along to FMassInstancedSkinnedMeshComponentSharedData::LocalTransform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FTransform LocalTransform = FTransform::Identity;
	
	/** The mesh visual representation */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TObjectPtr<USkinnedAsset> Asset = nullptr;

	/** The Animation Transform Provider when using this mesh*/
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TObjectPtr<UTransformProviderData> TransformProvider = nullptr;

	/**
	 * Material overrides for the mesh visual representation. 
	 * 
	 * Array indices correspond to material slot indices on the mesh.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TArray<TObjectPtr<UMaterialInterface>> MaterialOverrides;

	/** The minimum inclusive LOD significance to start using this mesh */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	float MinLODSignificance = float(EMassLOD::High);

	/** The maximum exclusive LOD significance to stop using this mesh */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	float MaxLODSignificance = float(EMassLOD::Max);

	/** Controls whether the ISM can cast shadow or not */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bCastShadows = false;

	/**
	 * Determines the value of corresponding FMassISMCSharedData.bRequiresExternalInstanceIDTracking.
	 * Recommended enabled for stationary meshes.
	 * @see FMassISMCSharedData.bRequiresExternalInstanceIDTracking for details.
	 */
	bool bRequiresExternalInstanceIDTracking = false;
	
	/** Controls the mobility of the ISM */
	EComponentMobility::Type Mobility = EComponentMobility::Movable;

	/** InstancedSkinnedMeshComponent class to use to manage instances described by this struct instance */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<UInstancedSkinnedMeshComponent> InstancedSkinnedMeshComponentClass;

	/**
	 * Passed to UInstancedSkinnedMeshComponent::SetAnimationMinScreenSize.
	 * Values: < 0 = always animate, 0 = use r.Skinning.DefaultAnimationMinScreenSize, > 0 = custom threshold.
	 * @see UInstancedSkinnedMeshComponent::SetAnimationMinScreenSize
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	float AnimationMinScreenSize = 0.0f;

	bool operator==(const FMassSkinnedMeshInstanceVisualizationMeshDesc& Other) const
	{		
		return Asset == Other.Asset && 
			TransformProvider == Other.TransformProvider &&
			InstancedSkinnedMeshComponentClass == Other.InstancedSkinnedMeshComponentClass &&
			MaterialOverrides == Other.MaterialOverrides && 
			FMath::IsNearlyEqual(MinLODSignificance, Other.MinLODSignificance, KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(MaxLODSignificance, Other.MaxLODSignificance, KINDA_SMALL_NUMBER) &&
			bCastShadows == Other.bCastShadows && 
			Mobility == Other.Mobility &&
			bRequiresExternalInstanceIDTracking == Other.bRequiresExternalInstanceIDTracking &&
			LocalTransform.Equals(Other.LocalTransform) &&
			AnimationMinScreenSize == Other.AnimationMinScreenSize;
	}
	
	friend inline uint32 GetTypeHash(const FMassSkinnedMeshInstanceVisualizationMeshDesc& MeshDesc)
	{
		uint32 Hash = 0x0;
		Hash = PointerHash(MeshDesc.Asset, Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.TransformProvider), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.InstancedSkinnedMeshComponentClass), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.bCastShadows), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.Mobility), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.LocalTransform), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.bRequiresExternalInstanceIDTracking), Hash);
		Hash = HashCombine(GetTypeHash(MeshDesc.AnimationMinScreenSize), Hash);
		for (UMaterialInterface* MaterialOverride : MeshDesc.MaterialOverrides)
		{
			if (MaterialOverride)
			{
				Hash = PointerHash(MaterialOverride, Hash);
			}
		}
		return Hash;
	}

	// convenience function for setting MinLODSinificance and MaxLODSinificance based on EMassLOD values
	void SetSignificanceRange(const EMassLOD::Type MinLOD, const EMassLOD::Type MaxLOD)
	{
		checkSlow(MinLOD <= MaxLOD);
		MinLODSignificance = float(MinLOD);
		MaxLODSignificance = float(MaxLOD);
	}
};

USTRUCT()
struct FSkinnedMeshInstanceVisualizationDesc : public FTableRowBase
{
	GENERATED_BODY()

	/** 
	 * Mesh descriptions. These will be instanced together using the same transform for each, to 
	 * visualize the given agent.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta=(TitleProperty="{Mesh}"))
	TArray<FMassSkinnedMeshInstanceVisualizationMeshDesc> Meshes;

	/** Boolean to enable code to transform the meshes if not align the mass agent transform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bUseTransformOffset = false;

	/** Transform to offset the meshes if not align the mass agent transform */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta=(EditCondition="bUseTransformOffset"))
	FTransform TransformOffset = FTransform::Identity;

	/** Custom data that can eventually be propagated to UInstancedSkinnedMeshComponent::PerInstanceCustomData */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TArray<float> CustomDataFloats;

	bool operator==(const FSkinnedMeshInstanceVisualizationDesc& Other) const
	{
		return Meshes == Other.Meshes && bUseTransformOffset == Other.bUseTransformOffset && TransformOffset.Equals(Other.TransformOffset);	//Intentionally not comparing CustomDataFloats
	}

	void Reset()
	{
		Meshes.Empty();
		bUseTransformOffset = false;
		TransformOffset = FTransform::Identity;
		CustomDataFloats.Empty();
	}

	/** @return whether any of descriptions in Meshes is valid. This implies that empty Meshes will be treated as not valid.*/
	UE_API bool IsValid() const;
};

/** Handle for FSkinnedMeshInstanceVisualizationDesc's registered with UMassRepresentationSubsystem */
USTRUCT()
struct alignas(2) FSkinnedMeshInstanceVisualizationDescHandle 
{
	GENERATED_BODY()

	static constexpr uint16 InvalidIndex = TNumericLimits<uint16>::Max();

	FSkinnedMeshInstanceVisualizationDescHandle () = default;

	explicit FSkinnedMeshInstanceVisualizationDescHandle (uint16 InIndex)
	: Index(InIndex)
	{}

	explicit FSkinnedMeshInstanceVisualizationDescHandle (int32 InIndex) 
	{
		// Handle special case INDEX_NONE = InvalidIndex
		if (InIndex == INDEX_NONE)
		{
			Index = InvalidIndex;
		}
		else
		{
			checkf(InIndex < static_cast<int32>(InvalidIndex), TEXT("Visualization description index InIndex %d is out of expected bounds (< %u)"), InIndex, InvalidIndex);
			Index = static_cast<uint16>(InIndex);
		}
	}

	inline int32 ToIndex() const
	{
		return IsValid() ? Index : INDEX_NONE;
	}

	bool IsValid() const
	{
		return Index != InvalidIndex;
	}

	bool operator==(const FSkinnedMeshInstanceVisualizationDescHandle & Other) const = default;

private:

	UPROPERTY()
	uint16 Index = InvalidIndex;

	// @todo: Add a version / serial number to protect against recycled handle reuse. Leaving this out for now to keep size down due to 
	// prevalent use in FMassRepresentationFragment. Perhaps serial number could be formed from the referenced 
	// FStaticMeshInstanceVisualizationDesc's hash.
};
static_assert(sizeof(FSkinnedMeshInstanceVisualizationDescHandle ) == sizeof(uint16), "FSkinnedMeshInstanceVisualizationDescHandle must be uint16 sized to ensure FMassRepresentationFragment memory isn't unexpectedly bloated");

struct FMassInstancedSkinnedMeshComponentSharedData
{
	FMassInstancedSkinnedMeshComponentSharedData()
	: bRequiresExternalInstanceIDTracking(false)
	{
	}

	explicit FMassInstancedSkinnedMeshComponentSharedData(UInstancedSkinnedMeshComponent* InInstancedSkinnedMeshComponent, bool bInRequiresExternalInstanceIDTracking = false, const FTransform& InTransformOffset = FTransform::Identity)
		: LocalTransform(InTransformOffset), InstancedSkinnedMeshComponent(InInstancedSkinnedMeshComponent), bRequiresExternalInstanceIDTracking(bInRequiresExternalInstanceIDTracking)
	{
	}

	FMassInstancedSkinnedMeshComponentSharedData(const FMassInstancedSkinnedMeshComponentSharedData& Other) = default;
	FMassInstancedSkinnedMeshComponentSharedData& operator=(const FMassInstancedSkinnedMeshComponentSharedData& Other) = default;


	void SetInstancedSkinnedMeshComponent(UInstancedSkinnedMeshComponent& InInstancedSkinnedMeshComponent)
	{
		check(InstancedSkinnedMeshComponent == nullptr && InstancedSkinnedMeshComponentReferencesCount == 0);
		InstancedSkinnedMeshComponent = &InInstancedSkinnedMeshComponent;
	}


	UInstancedSkinnedMeshComponent* GetMutableInstancedSkinnedMeshComponent()	{ return InstancedSkinnedMeshComponent; }
	const UInstancedSkinnedMeshComponent* GetInstancedSkinnedMeshComponent() const	{ return InstancedSkinnedMeshComponent; }
	int32 OnInstancedSkinnedMeshComponentReferenceStored() { return ++InstancedSkinnedMeshComponentReferencesCount; }
	int32 OnInstancedSkinnedMeshComponentReferenceReleased() 
	{ 
		ensure(InstancedSkinnedMeshComponentReferencesCount >= 0); 
		return --InstancedSkinnedMeshComponentReferencesCount;
	}

	void ResetAccumulatedData()
	{
		EntitiesRequiringUpdate.Reset();
		MeshInstanceCustomFloats.Reset();
		MeshInstanceTransforms.Reset();
		MeshInstancePrevTransforms.Reset();
		MeshInstanceAnimationData.Reset();
		EntitiesRequiringRemoval.Reset();
		WriteIterator = 0;
	}

	void RemoveUpdatedInstanceIdsAtSwap(const int32 InstanceIDIndex)
	{
		EntitiesRequiringUpdate.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		MeshInstanceTransforms.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		MeshInstancePrevTransforms.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		MeshInstanceAnimationData.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		if (MeshInstanceCustomFloats.Num())
		{
			MeshInstanceCustomFloats.RemoveAtSwap(InstanceIDIndex, EAllowShrinking::No);
		}
	}

	bool HasUpdatesToApply() const { return EntitiesRequiringUpdate.Num() || EntitiesRequiringRemoval.Num(); }
	TConstArrayView<FMassEntityHandle> GetEntitiesRequiringUpdate() const { return EntitiesRequiringUpdate; }
	TConstArrayView<FTransform> GetMeshInstanceTransforms() const { return MeshInstanceTransforms; }
	TConstArrayView<FAnimSequenceTrackAutoPlayData> GetMeshInstanceAnimationData() const { return MeshInstanceAnimationData; }
	/**
	 * this function is a flavor we need to interact with older engine API that's using TArray references.
	 * Use GetMeshInstanceTransforms instead whenever possible.
	 */
	const TArray<FTransform>& GetMeshInstanceTransformsArray() const { return MeshInstanceTransforms; }
	const TArray<FAnimSequenceTrackAutoPlayData>& GetMeshInstanceAnimatioDataArray() const { return MeshInstanceAnimationData; }

	TConstArrayView<FTransform> GetMeshInstancePrevTransforms() const { return MeshInstancePrevTransforms; }
	TConstArrayView<FMassEntityHandle> GetEntitiesRequiringRemoval() const { return EntitiesRequiringRemoval; }
	TConstArrayView<float> GetMeshInstanceCustomFloats() const { return MeshInstanceCustomFloats; }

	bool RequiresExternalInstanceIDTracking() const { return bRequiresExternalInstanceIDTracking; }

	void Reset()
	{
		*this = FMassInstancedSkinnedMeshComponentSharedData();
	}

	using FEntityToPrimitiveIdMap = Experimental::TRobinHoodHashMap<FMassEntityHandle, FPrimitiveInstanceId>;
	using FEntityToTrackIdMap = Experimental::TRobinHoodHashMap<FMassEntityHandle, int32>;

	FEntityToPrimitiveIdMap& GetMutableEntityPrimitiveToIdMap() { return EntityHandleToPrimitiveIdMap; }
	const FEntityToPrimitiveIdMap& GetEntityPrimitiveToIdMap() const { return EntityHandleToPrimitiveIdMap; }

	FEntityToTrackIdMap& GetMutableEntityToTrackMap() { return EntityHandleToTrackIdMap; }
	const FEntityToTrackIdMap& GetEntityToTrackMap() const { return EntityHandleToTrackIdMap; }

	FEntityToPrimitiveIdMap& GetMutableEntityToPrimitiveIdMap() { return EntityHandleToPrimitiveIdMap; }
	const FEntityToPrimitiveIdMap& GetEntityToPrimitiveIdMap() const { return EntityHandleToPrimitiveIdMap; }

	int16 GetComponentInstanceIdTouchCounter() const { return ComponentInstanceIdTouchCounter; }

protected:
	friend FMassLODInstancedSkinnedMeshSignificanceRange;
	friend UMassVisualizationComponent;

	FTransform LocalTransform = FTransform::Identity;

	/** Buffer holding current frame transforms for the mesh instances, used to batch update the transforms */
	TArray<FMassEntityHandle> EntitiesRequiringUpdate;
	TArray<FTransform> MeshInstanceTransforms;
	TArray<FTransform> MeshInstancePrevTransforms;
	/** Buffer holding animation data */
	TArray<FAnimSequenceTrackAutoPlayData> MeshInstanceAnimationData;
	TArray<FMassEntityHandle> EntitiesRequiringRemoval;

	/** Buffer holding current frame custom floats for the mesh instances, used to batch update the ISMs custom data */
	TArray<float> MeshInstanceCustomFloats;

	// When initially adding to MeshInstanceCustomFloats, can use the size as the write iterator, but on subsequent processors, we need to know where to start writing
	int32 WriteIterator = 0;

	UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent = nullptr;

	int32 InstancedSkinnedMeshComponentReferencesCount = 0;

	/**
	 * When set to true will result in MassVisualizationComponent manually perform Instance ID-related operations
	 * instead of relying on ISMComponent's internal ID operations.
	 * @note this mechanism has been added in preparation of changes to ISM component to change access to its internal
	 *	instance ID logic. WIP as of Jun 17th 2023
	 */
	uint8 bRequiresExternalInstanceIDTracking : 1;

private:
	/** Indicates that mutating changes, that can affect MassInstanceIdToComponentInstanceIdMap, have been performed.
	 *	Can be used to validate whether cached data stored in other placed needs to be re-cached. */
	uint16 ComponentInstanceIdTouchCounter = 0;

protected:
	FEntityToPrimitiveIdMap EntityHandleToPrimitiveIdMap;
	FEntityToTrackIdMap EntityHandleToTrackIdMap;
};


/**
 * The container type hosting FMassInstancedSkinnedMeshSharedData instances and supplying functionality of marking entries that require
 * instance-related operations (adding, removing).
 *
 * To get a FMassInstance instance to add operations to it call GetAndMarkDirty.
 *
 * Use FDirtyIterator to iterate over just the data that needs processing.
 *
 * @see UMassVisualizationComponent::EndVisualChanges for iteration
 * @see FMassLODSignificanceRange methods for performing dirtying operations
 */
struct FMassInstancedSkinnedMeshComponentSharedDataMap
{
	struct FDirtyIterator
	{
		friend FMassInstancedSkinnedMeshComponentSharedDataMap;
		explicit FDirtyIterator(FMassInstancedSkinnedMeshComponentSharedDataMap& InContainer)
			: Container(InContainer), It(InContainer.GetDirtyArray())
		{
			if (It && It.GetValue() != bValueToCheck)
			{
				// will result in either setting IT to the first bInValue, or making bool(It) == false
				++(*this);
			}
		}
	public:
		operator bool() const { return bool(It); }

		FDirtyIterator& operator++()
		{
			while (++It)
			{
				if (It.GetValue() == bValueToCheck)
				{
					break;
				}
			}
			return *this;
		}

		FMassInstancedSkinnedMeshComponentSharedData& operator*() const
		{
			return Container.GetAtIndex(It.GetIndex());
		}

		void ClearDirtyFlag()
		{
			It.GetValue() = false;
		}

	private:
		FMassInstancedSkinnedMeshComponentSharedDataMap& Container;
		TBitArray<>::FIterator It;
		static constexpr bool bValueToCheck = true;
	};

	FMassInstancedSkinnedMeshComponentSharedData& GetAndMarkDirtyChecked(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey)
	{
		const int32 DataIndex = Map[OwnerKey];
		DirtyData[DataIndex] = true;
		return Data[DataIndex];
	}

	FMassInstancedSkinnedMeshComponentSharedData* GetAndMarkDirty(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey)
	{
		const int32* DataIndex = Map.Find(OwnerKey);
		if (ensureMsgf(DataIndex, TEXT("%hs Failed to find OwnerKey %s"), __FUNCTION__, *GetNameSafe(OwnerKey.ResolveObjectPtrEvenIfGarbage())))
		{
			DirtyData[*DataIndex] = true;
			return &Data[*DataIndex];
		}
		return nullptr;
	}

	template<typename... TArgs>
	FMassInstancedSkinnedMeshComponentSharedData& FindOrAdd(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey, TArgs&&... InNewInstanceArgs)
	{
		const int32* DataIndex = Map.Find(OwnerKey);
		if (DataIndex == nullptr)
		{
			return Add(OwnerKey, Forward<TArgs>(InNewInstanceArgs)...);
		}
		check(Data.IsValidIndex(*DataIndex));
		return Data[*DataIndex];
	}

	FMassInstancedSkinnedMeshComponentSharedData* Find(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey)
	{
		int32* DataIndex = Map.Find(OwnerKey);
		return (DataIndex == nullptr || *DataIndex == INDEX_NONE) ? (FMassInstancedSkinnedMeshComponentSharedData*)nullptr : &Data[*DataIndex];
	}

	template<typename... TArgs>
	FMassInstancedSkinnedMeshComponentSharedData& Add(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey, TArgs&&... InNewInstanceArgs)
	{
		const int32 DataIndex = FreeIndices.Num() ? FreeIndices.Pop() : Data.Num();
		Map.Add(OwnerKey, DataIndex);

		if (DataIndex == Data.Num())
		{
			DirtyData.Add(false, DataIndex - DirtyData.Num() + 1);
			DirtyData[DataIndex] = true;
			return Data.Add_GetRef(FMassInstancedSkinnedMeshComponentSharedData(Forward<TArgs>(InNewInstanceArgs)...));
		}
		else
		{
			DirtyData[DataIndex] = true;
			Data[DataIndex] = FMassInstancedSkinnedMeshComponentSharedData(Forward<TArgs>(InNewInstanceArgs)...);
			return Data[DataIndex];
		}
	}

	void Remove(const FInstancedSkinnedMeshComponentSharedDataKey OwnerKey)
	{
		int32 DataIndex = INDEX_NONE;
		if (ensure(Map.RemoveAndCopyValue(OwnerKey, DataIndex)))
		{
			DirtyData[DataIndex] = false;
			Data[DataIndex].Reset();
			FreeIndices.Add(DataIndex);
		}
	}

	FMassInstancedSkinnedMeshComponentSharedData& GetAtIndex(const int32 DataIndex)
	{
		return Data[DataIndex];
	}

	TBitArray<>& GetDirtyArray()
	{
		return DirtyData;
	}

	/** @return total number of entries in Data array. Note that some or all entries could be empty (i.e. already freed) */
	int32 Num() const
	{
		return Data.Num();
	}

	/** @return number of non-empty entries in Data. */
	int32 NumValid() const
	{
		return Data.Num() - FreeIndices.Num();
	}

	bool IsDirty(const int32 DataIndex) const
	{
		return DirtyData[DataIndex];
	}

	bool IsEmpty() const
	{
		return NumValid() == 0;
	}

	void Reset()
	{
		*this = FMassInstancedSkinnedMeshComponentSharedDataMap();
	}

	const FMassInstancedSkinnedMeshComponentSharedData* GetDataForIndex(const int32 Index) const
	{
		return Data.IsValidIndex(Index) ? &Data[Index] : nullptr;
	}

	const FMassInstancedSkinnedMeshComponentSharedData* GetDataForKey(const FInstancedSkinnedMeshComponentSharedDataKey Key) const
	{
		const int32* Index = Map.Find(Key);
		return (Index && Data.IsValidIndex(*Index))
			? &Data[*Index]
			: nullptr;
	}

protected:
	TArray<FMassInstancedSkinnedMeshComponentSharedData> Data;
	/** Mapping from Owner (as FObjectKey) of data represented by FMassISMCSharedData to an index to Data */
	TMap<FInstancedSkinnedMeshComponentSharedDataKey, int32> Map;
	/** Indicates whether corresponding Data entry has any instance work assigned to it (instance addition or removal) */
	TBitArray<> DirtyData;
	/** Indices to Data that are available for reuse */
	TArray<int32> FreeIndices;
};



USTRUCT()
struct FMassLODInstancedSkinnedMeshSignificanceRange
{
	GENERATED_BODY()
public:

	UE_API void AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, TConstArrayView<FInstancedSkinnedMeshComponentSharedDataKey> ExcludeMeshRefs);
	UE_API void AddBatchedAnimationData(const FMassEntityHandle EntityHandle, FAnimSequenceTrackAutoPlayData AnimationData, TConstArrayView<FInstancedSkinnedMeshComponentSharedDataKey> ExcludeMeshRefs);

	// Adds the specified struct reinterpreted as custom floats to our custom data. Individual members of the specified struct should always fit into a float.
	// When adding any custom data, the custom data must be added for every instance.
	template<typename InCustomDataType>
	void AddBatchedCustomData(InCustomDataType InCustomData, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs, int32 NumFloatsToPad = 0)
	{
		check(InstancedSkinnedMeshSharedDataPtr);
		static_assert((sizeof(InCustomDataType) % sizeof(float)) == 0, "AddBatchedCustomData: InCustomDataType should have a total size multiple of sizeof(float), and have members that fit in a float's boundaries");
		const size_t StructSize = sizeof(InCustomDataType);
		const size_t StructSizeInFloats = StructSize / sizeof(float);
		for (int i = 0; i < SkinnedMeshComponentRefs.Num(); i++)
		{
			if (ExcludeMeshRefs.Contains(SkinnedMeshComponentRefs[i]))
			{
				continue;
			}

			FMassInstancedSkinnedMeshComponentSharedData& SharedData = (*InstancedSkinnedMeshSharedDataPtr).GetAndMarkDirtyChecked(SkinnedMeshComponentRefs[i]);
			const int32 StartIndex = SharedData.MeshInstanceCustomFloats.AddDefaulted(StructSizeInFloats + NumFloatsToPad);
			InCustomDataType* CustomData = reinterpret_cast<InCustomDataType*>(&SharedData.MeshInstanceCustomFloats[StartIndex]);
			*CustomData = InCustomData;
		}
	}

	/**
	 * Adds uniform custom data, CustomFloats is applied to all meshes in the SignificanceRange.
	 *
	 * Only safe to call from a Mass Processor.
	 * Requires: UMassRepresentationSubsystem (ReadWrite)
	 */
	UE_API void AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs);

	/**
	 * Adds per-mesh custom data, PerMeshFloats is indexed in source Desc.Meshes order.
	 *
	 * Only safe to call from a Mass Processor.
	 * Requires: UMassRepresentationSubsystem (ReadWrite)
	 */
	UE_API void AddBatchedCustomDataFloats(TConstArrayView<TArray<float>> PerMeshFloats, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs);

	/** Single-instance version of AddBatchedCustomData when called to add entities (as opposed to modify existing ones).*/
	UE_API void AddInstance(const FMassEntityHandle EntityHandle, const FTransform& Transform);

	UE_API void RemoveInstance(const FMassEntityHandle EntityHandle);

	UE_API void WriteCustomDataFloatsAtStartIndex(int32 MeshIndex, const TArrayView<float>& CustomFloats, const int32 FloatsPerInstance, const int32 StartIndex, const TArray<FInstancedSkinnedMeshComponentSharedDataKey>& ExcludeMeshRefs);

	/** LOD Significance range */
	float MinSignificance = 0.0f;
	float MaxSignificance = 0.0f;

	/** The component handling these instances */
	TArray<FInstancedSkinnedMeshComponentSharedDataKey> SkinnedMeshComponentRefs;

	/** Source Desc.Meshes index per SkinnedMeshComponentRefs entry. */
	TArray<int32> SourceDescMeshIndices;

	FMassInstancedSkinnedMeshComponentSharedDataMap* InstancedSkinnedMeshSharedDataPtr = nullptr;
};


USTRUCT()
struct FMassInstancedSkinnedMeshInfo
{
	GENERATED_BODY()
public:

	FMassInstancedSkinnedMeshInfo() = default;

	explicit FMassInstancedSkinnedMeshInfo(const FSkinnedMeshInstanceVisualizationDesc& InDesc)
		: Desc(InDesc)
	{
	}

	/** Clears out contents so that a given FMassInstancedSkinnedMeshInfo instance can be reused */
	UE_API void Reset();

	const FSkinnedMeshInstanceVisualizationDesc& GetDesc() const
	{
		return Desc;
	}

	/** Whether or not to transform the  meshes if not align the mass agent transform */
	bool ShouldUseTransformOffset() const { return Desc.bUseTransformOffset; }
	FTransform GetTransformOffset() const { return Desc.TransformOffset; }

	inline FMassLODInstancedSkinnedMeshSignificanceRange* GetLODSignificanceRange(float LODSignificance)
	{
		for (FMassLODInstancedSkinnedMeshSignificanceRange& Range : LODSignificanceRanges)
		{
			if (LODSignificance >= Range.MinSignificance && LODSignificance < Range.MaxSignificance)
			{
				return &Range;
			}
		}
		return nullptr;
	}

	void AddBatchedTransform(const FMassEntityHandle EntityHandle, const FTransform& Transform, const FTransform& PrevTransform, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedTransform(EntityHandle, Transform, PrevTransform, {});
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedTransform(EntityHandle, Transform, PrevTransform, Range->SkinnedMeshComponentRefs);
				}
			}
		}
	}

	void AddBatchedAnimationData(const FMassEntityHandle EntityHandle, FAnimSequenceTrackAutoPlayData AnimationData, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedAnimationData(EntityHandle, AnimationData, {});
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedAnimationData(EntityHandle, AnimationData, Range->SkinnedMeshComponentRefs);
				}
			}
		}
	}


	inline void RemoveInstance(const FMassEntityHandle EntityHandle, const float LODSignificance)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->RemoveInstance(EntityHandle);
		}
	}

	// Adds the specified struct reinterpreted as custom floats to our custom data. Individual members of the specified struct should always fit into a float.
	// When adding any custom data, the custom data must be added for every instance.
	template<typename InCustomDataType>
	void AddBatchedCustomData(InCustomDataType InCustomData, const float LODSignificance, const float PrevLODSignificance = -1.0f, int32 NumFloatsToPad = 0)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedCustomData(InCustomData, {}, NumFloatsToPad);
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedCustomData(InCustomData, Range->SkinnedMeshComponentRefs, NumFloatsToPad);
				}
			}
		}
	}

	/**
	 * Adds uniform custom data, CustomFloats is applied to all meshes of the given LODSignificance.
	 *
	 * Only safe to call from a Mass Processor.
	 * Requires: UMassRepresentationSubsystem (ReadWrite)
	 */
	inline void AddBatchedCustomDataFloats(const TArray<float>& CustomFloats, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedCustomDataFloats(CustomFloats, {});
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedCustomDataFloats(CustomFloats, Range->SkinnedMeshComponentRefs);
				}
			}
		}
	}

	/**
	 * Adds per-mesh custom data, PerMeshFloats is indexed in source Desc.Meshes order.
	 *
	 * Only safe to call from a Mass Processor.
	 * Requires: UMassRepresentationSubsystem (ReadWrite)
	 */
	inline void AddBatchedCustomDataFloats(TConstArrayView<TArray<float>> PerMeshFloats, const float LODSignificance, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->AddBatchedCustomDataFloats(PerMeshFloats, {});
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->AddBatchedCustomDataFloats(PerMeshFloats, Range->SkinnedMeshComponentRefs);
				}
			}
		}
	}

	void WriteCustomDataFloatsAtStartIndex(int32 MeshIndex, const TArrayView<float>& CustomFloats, const float LODSignificance, const int32 FloatsPerInstance, const int32 FloatStartIndex, const float PrevLODSignificance = -1.0f)
	{
		if (FMassLODInstancedSkinnedMeshSignificanceRange* Range = GetLODSignificanceRange(LODSignificance))
		{
			Range->WriteCustomDataFloatsAtStartIndex(MeshIndex, CustomFloats, FloatsPerInstance, FloatStartIndex, {});
			if (PrevLODSignificance >= 0.0f)
			{
				FMassLODInstancedSkinnedMeshSignificanceRange* PrevRange = GetLODSignificanceRange(PrevLODSignificance);
				if (ensureMsgf(PrevRange, TEXT("Couldn't find a valid LODSignificanceRange for PrevLODSignificance %f"), PrevLODSignificance)
					&& PrevRange != Range)
				{
					PrevRange->WriteCustomDataFloatsAtStartIndex(MeshIndex, CustomFloats, FloatsPerInstance, FloatStartIndex, Range->SkinnedMeshComponentRefs);
				}
			}
		}
	}

	void AddInstancedSkinnedMeshComponent(FMassInstancedSkinnedMeshComponentSharedData& SharedData)
	{
		if (ensure(SharedData.GetInstancedSkinnedMeshComponent()))
		{
			InstancedSkinnedMeshComponents.Add(SharedData.GetMutableInstancedSkinnedMeshComponent());
			SharedData.OnInstancedSkinnedMeshComponentReferenceStored();
		}
	}

	int32 GetLODSignificanceRangesNum() const { return LODSignificanceRanges.Num(); }

	bool IsValid() const
	{
		return Desc.Meshes.Num() && InstancedSkinnedMeshComponents.Num() && LODSignificanceRanges.Num();
	}

protected:

	/** Destroy the visual instance */
	UE_API void ClearVisualInstance(UInstancedSkinnedMeshComponent& ISMComponent);

	/** Information about this  mesh which will represent all instances */
	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	FSkinnedMeshInstanceVisualizationDesc Desc;

	/** The components handling these instances */
	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	TArray<TObjectPtr<UInstancedSkinnedMeshComponent>> InstancedSkinnedMeshComponents;

	UPROPERTY(VisibleAnywhere, Category = "Mass/Debug")
	TArray<FMassLODInstancedSkinnedMeshSignificanceRange> LODSignificanceRanges;

	friend class UMassVisualizationComponent;
};


#if ENABLE_MT_DETECTOR

#define MAKE_MASS_INSTANCED_SKINNED_MESH_INFO_ARRAY_VIEW(ArrayView, AccessDetector) FMassInstancedSkinnedMeshInfoArrayView(ArrayView, AccessDetector)
struct FMassInstancedSkinnedMeshInfoArrayViewAccessDetector
{
	FMassInstancedSkinnedMeshInfoArrayViewAccessDetector(TArrayView<FMassInstancedSkinnedMeshInfo> InInstancedSkinnedMeshInfos, const FRWAccessDetector& InAccessDetector)
		: InstancedSkinnedMeshInfos(InInstancedSkinnedMeshInfos)
		, AccessDetector(&InAccessDetector)
	{
		UE_MT_ACQUIRE_WRITE_ACCESS(*AccessDetector);
	}

	FMassInstancedSkinnedMeshInfoArrayViewAccessDetector(FMassInstancedSkinnedMeshInfoArrayViewAccessDetector&& Other)
		: InstancedSkinnedMeshInfos(Other.InstancedSkinnedMeshInfos)
		, AccessDetector(Other.AccessDetector)
	{
		Other.AccessDetector = nullptr;
	}
	FMassInstancedSkinnedMeshInfoArrayViewAccessDetector(const FMassInstancedSkinnedMeshInfoArrayViewAccessDetector& Other) = delete;
	void operator=(const FMassInstancedSkinnedMeshInfoArrayViewAccessDetector& Other) = delete;

	~FMassInstancedSkinnedMeshInfoArrayViewAccessDetector()
	{
		if (AccessDetector)
		{
			UE_MT_RELEASE_WRITE_ACCESS(*AccessDetector);
		}
	}

	inline FMassInstancedSkinnedMeshInfo& operator[](int32 Index) const
	{
		return InstancedSkinnedMeshInfos[Index];
	}

	bool IsValidIndex(const int32 Index) const
	{
		return InstancedSkinnedMeshInfos.IsValidIndex(Index);
	}

	int32 Num() const
	{
		return InstancedSkinnedMeshInfos.Num();
	}

private:
	TArrayView<FMassInstancedSkinnedMeshInfo> InstancedSkinnedMeshInfos;
	const FRWAccessDetector* AccessDetector;
};

using FMassInstancedSkinnedMeshInfoArrayView = FMassInstancedSkinnedMeshInfoArrayViewAccessDetector;


#else // ENABLE_MT_DETECTOR

#define MAKE_MASS_INSTANCED_SKINNED_MESH_INFO_ARRAY_VIEW(ArrayView, AccessDetector) ArrayView
using FMassInstancedSkinnedMeshInfoArrayView = TArrayView<FMassInstancedSkinnedMeshInfo>;

#endif // ENABLE_MT_DETECTOR

#undef UE_API