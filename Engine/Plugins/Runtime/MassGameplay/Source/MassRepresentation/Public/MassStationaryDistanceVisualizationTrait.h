// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassDistanceVisualizationTrait.h"
#include "MassStationaryDistanceVisualizationTrait.generated.h"

#define UE_API MASSREPRESENTATION_API


UCLASS(MinimalAPI)
class UMassStationaryDistanceVisualizationTrait : public UMassDistanceVisualizationTrait
{
	GENERATED_BODY()
public:
	UE_API UMassStationaryDistanceVisualizationTrait(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

protected:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};

#undef UE_API
