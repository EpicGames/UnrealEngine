// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MassStationaryISMRepresentationFragmentDestructor.generated.h"

#define UE_API MASSREPRESENTATION_API


/** 
 * This class is responsible for cleaning up ISM instances visualizing stationary entities
 */
UCLASS(MinimalAPI)
class UMassStationaryISMRepresentationFragmentDestructor : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassStationaryISMRepresentationFragmentDestructor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
