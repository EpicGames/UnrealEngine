// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassStationaryISMSwitcherProcessor.generated.h"

#define UE_API MASSREPRESENTATION_API


struct FMassExecutionContext;

namespace UE::Mass::Signals
{
	const FName SwitchedToActor = FName(TEXT("SwitchedToActor"));
	const FName SwitchedToISM = FName(TEXT("SwitchedToISM"));
}

/**
 * Tag required by UMassStationaryISMSwitcherProcessor to process given archetype. Removing the tag allows to support temporary
 * disabling of processing for individual entities of given archetype.
 */
USTRUCT()
struct FMassStationaryISMSwitcherProcessorTag : public FMassTag
{
	GENERATED_BODY();
};

/** 
 * This processor's sole responsibility is to process all entities tagged with FMassStaticRepresentationTag
 * and check if they've switched to or away from EMassRepresentationType::StaticMeshInstance or EMassRepresentationType::SkinnedMeshInstance ; and acordingly add or remove 
 * the entity from the appropriate ArrayView.
 */
UCLASS(MinimalAPI)
class UMassStationaryISMSwitcherProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassStationaryISMSwitcherProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static UE_API void ProcessContext(FMassExecutionContext& Context);

protected:
	// UMassProcessor overrides begin
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	// UMassProcessor overrides end
	// 
	// UMassRepresentationProcessor overrides begin
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	// UMassRepresentationProcessor overrides end

	FMassEntityQuery EntityQuery;
};

#undef UE_API
