// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassUpdateSkinnedMeshProcessor.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMassRepresentationSubsystem;
struct FMassInstancedSkinnedMeshInfo;

UCLASS(MinimalAPI)
class UMassUpdateInstancedSkinnedMeshProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassUpdateInstancedSkinnedMeshProcessor();

	static UE_API void UpdateMeshTransform(FMassEntityHandle EntityHandle, FMassInstancedSkinnedMeshInfo& MeshInfo, const FTransform& Transform, const FTransform& PrevTransform, const float LODSignificance, const float PrevLODSignificance = -1.0f);

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	/**
	 * Execution method for this processor
	 * @param EntitySubsystem is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API