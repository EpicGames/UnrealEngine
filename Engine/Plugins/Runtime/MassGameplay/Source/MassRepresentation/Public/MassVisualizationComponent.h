// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "Components/ActorComponent.h"
#include "Misc/MTAccessDetector.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassVisualizationComponent.generated.h"

class UInstancedStaticMeshComponent;

/** 
 * This component handles all the static mesh instances for a MassRepresentationProcessor and is an actor component off a MassVisualizer actor.
 * Meant to be created at runtime and owned by an MassVisualizer actor. Will ensure if placed on a different type of actor. 
 */
UCLASS(MinimalAPI)
class UMassVisualizationComponent : public UActorComponent
{
	GENERATED_BODY()
public:

	/** 
	 * Get the index of the visual type, will add a new one if does not exist
	 * @param Desc is the information for the visual that will be instantiated later via AddVisualInstance()
	 * @return The index of the visual type 
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle FindOrAddVisualDesc(const FStaticMeshInstanceVisualizationDesc& Desc);

	/** 
	 * Creates a dedicated visual type described by host Desc and ties ISMComponent to it.
	 * @note this is a helper function for a common "single ISMComponent" case. Calls AddVisualDescWithISMComponents under the hood.
	 * @return The index of the visual type 
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle AddVisualDescWithISMComponent(const FStaticMeshInstanceVisualizationDesc& Desc, UInstancedStaticMeshComponent& ISMComponent);

	/**
	 * Creates a dedicated visual type described by host Desc and ties given ISMComponents to it.
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle AddVisualDescWithISMComponents(const FStaticMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedStaticMeshComponent>> ISMComponents);

	/**
	 * Fetches FMassISMCSharedData indicated by DescriptionIndex, or nullptr if it's not a valid index
	 */
	MASSREPRESENTATION_API const FMassISMCSharedData* GetISMCSharedDataForDescriptionIndex(const int32 DescriptionIndex) const;

	/**
	 * Fetches FMassISMCSharedData indicated by an ISMC, or nullptr if the ISMC is not represented by any shared data.
	 */
	MASSREPRESENTATION_API const FMassISMCSharedData* GetISMCSharedDataForInstancedStaticMesh(const UInstancedStaticMeshComponent* ISMC) const;

	/**
	 * Removes all data associated with a given VisualizationIndex. Note that this is safe to do only if there are no
	 * entities relying on this index. No entity data patching will take place.
	 */
	MASSREPRESENTATION_API void RemoveVisualDesc(const FStaticMeshInstanceVisualizationDescHandle VisualizationHandle);

	/** Get the array of all visual instance informations */
	FMassInstancedStaticMeshInfoArrayView GetMutableVisualInfos()
	{
		FMassInstancedStaticMeshInfoArrayView View = MAKE_MASS_INSTANCED_STATIC_MESH_INFO_ARRAY_VIEW(MakeArrayView(InstancedStaticMeshInfos), InstancedStaticMeshInfosDetector);
		return MoveTemp(View);
	}

	/** Destroy all visual instances */
 	MASSREPRESENTATION_API void ClearAllVisualInstances();

	/** Dirty render state on all static mesh components */
 	MASSREPRESENTATION_API void DirtyVisuals();

	/** Signal the beginning of the static mesh instance changes, used to prepare the batching update of the static mesh instance transforms*/
	MASSREPRESENTATION_API void BeginVisualChanges();

	/** Signal the end of the static mesh instance changes, used to batch apply the transforms on the static mesh instances*/
	MASSREPRESENTATION_API void EndVisualChanges();

protected:
	/**
	 * Process all removed IDs in FMassISMCSharedData and apply to the ISM component.
	 */
	MASSREPRESENTATION_API void ProcessRemoves(UInstancedStaticMeshComponent& ISMComponent, FMassISMCSharedData& SharedData, bool bUpdateNavigation = true);
	
	/** 
	 * Applies changes accumulated in SharedData while manually updating the Instance ID mapping. This approach is done in preparation 
	 * to upcoming ISM changes to keep the mapping management more secure (by making mapping private and fully component-owned).
	 */
	MASSREPRESENTATION_API void HandleChangesWithExternalIDTracking(UInstancedStaticMeshComponent& ISMComponent, FMassISMCSharedData& SharedData);

	/** Recreate all the static mesh components from the InstancedStaticMeshInfos */
	MASSREPRESENTATION_API void ConstructStaticMeshComponents();

	/** Overridden to make sure this component is only added to a MassVisualizer actor */
	MASSREPRESENTATION_API virtual void PostInitProperties() override;

	/**
	 * Creates LODSignificance ranges for all the meshes indicated by Info
	 * @param ForcedStaticMeshRefKeys if not empty will be used when adding individual FMassStaticMeshInstanceVisualizationMeshDesc
	 *	instances to LOD significance ranges.
	 */	
	MASSREPRESENTATION_API void BuildLODSignificanceForInfo(FMassInstancedStaticMeshInfo& Info, TConstArrayView<UInstancedStaticMeshComponent*> StaticMeshRefKeys);

	/** Either adds an element to InstancedStaticMeshInfos or reuses an existing entry based on InstancedStaticMeshInfosFreeIndices*/
	MASSREPRESENTATION_API FStaticMeshInstanceVisualizationDescHandle AddInstancedStaticMeshInfo(const FStaticMeshInstanceVisualizationDesc& Desc);

	/** The information of all the instanced static meshes. Make sure to use AddInstancedStaticMeshInfo to add elements to it */
	UPROPERTY(Transient)
	TArray<FMassInstancedStaticMeshInfo> InstancedStaticMeshInfos;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstancedStaticMeshInfosDetector);

	/** Indices to InstancedStaticMeshInfos that have been released and can be reused */
	TArray<FStaticMeshInstanceVisualizationDescHandle> InstancedStaticMeshInfosFreeIndices;

	/** Mapping from ISMComponent (indicated by FISMCSharedDataKey) to corresponding VisualDescHandle */
	TMap<FISMCSharedDataKey, FStaticMeshInstanceVisualizationDescHandle> ISMComponentMap;

	FMassISMCSharedDataMap ISMCSharedData;

	/** 
	 * Mapping FMassStaticMeshInstanceVisualizationMeshDesc hash to FMassISMCSharedData entries for all FMassStaticMeshInstanceVisualizationMeshDesc
	 * that didn't come with ISMC explicitly provided. Used only for initialization.
	 * Note that FMassStaticMeshInstanceVisualizationMeshDesc that were added with ISMComponents provided directly
	 * (via AddVisualDescWithISMComponents call) will never make it to this map.
	 */
	TMap<uint32, FISMCSharedDataKey> MeshDescToISMCMap;

	/** Indices to InstancedStaticMeshInfos that need their SMComponent constructed */
	TArray<FStaticMeshInstanceVisualizationDescHandle> InstancedSMComponentsRequiringConstructing;

public:
	/**
	 * Get the index of the visual type, will add a new one if does not exist
	 * @param Desc is the information for the visual that will be instantiated later via AddVisualInstance()
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FSkinnedMeshInstanceVisualizationDescHandle FindOrAddSkinnedMeshInstanceVisualDesc(const FSkinnedMeshInstanceVisualizationDesc& Desc);

	/**
	 * Creates a dedicated visual type described by host Desc and ties InstancedSkinnedMeshComponent to it.
	 * @note this is a helper function for a common "single ISMComponent" case. Calls AddVisualDescWithComponents under the hood.
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FSkinnedMeshInstanceVisualizationDescHandle AddSkinnedMeshInstanceVisualDescWithComponent(const FSkinnedMeshInstanceVisualizationDesc& Desc, UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent);

	/**
	 * Creates a dedicated visual type described by host Desc and ties given InstancedSkinnedMeshComponents to it.
	 * @return The index of the visual type
	 */
	MASSREPRESENTATION_API FSkinnedMeshInstanceVisualizationDescHandle AddSkinnedMeshInstanceVisualDescWithComponents(const FSkinnedMeshInstanceVisualizationDesc& Desc, TArrayView<TObjectPtr<UInstancedSkinnedMeshComponent>> InstancedSkinnedMeshComponents);
	
	/**
	 * Fetches FMassInstancedSkinnedMeshComponentSharedData indicated by DescriptionIndex, or nullptr if it's not a valid index
	 */
	MASSREPRESENTATION_API const FMassInstancedSkinnedMeshComponentSharedData* GetSharedDataForDescriptionIndex(const int32 DescriptionIndex) const;

	/**
	 * Fetches FMassInstancedSkinnedMeshComponentSharedData indicated by an InstancedSkinnedMeshComponent, or nullptr if the InstancedSkinnedMeshComponent is not represented by any shared data.
	 */
	MASSREPRESENTATION_API const FMassInstancedSkinnedMeshComponentSharedData* GetSharedDataForInstancedSkinnedMeshComponent(const UInstancedSkinnedMeshComponent* InstancedSkinnedMeshComponent) const;

	/**
	 * Removes all data associated with a given VisualizationIndex. Note that this is safe to do only if there are no
	 * entities relying on this index. No entity data patching will take place.
	 */
	MASSREPRESENTATION_API void RemoveSkinnedMeshInstanceVisualDesc(const FSkinnedMeshInstanceVisualizationDescHandle VisualizationHandle);

	/** Get the array of all visual instance informations */
	FMassInstancedSkinnedMeshInfoArrayView GetMutableInstanceVisualInfos()
	{
		FMassInstancedSkinnedMeshInfoArrayView View = MAKE_MASS_INSTANCED_SKINNED_MESH_INFO_ARRAY_VIEW(MakeArrayView(InstancedSkinnedMeshInfos), InstancedSkinnedMeshInfosDetector);
		return MoveTemp(View);
	}

	const FMassSkinnedMeshResolvedAnimState* FindResolvedAnimState(const FMassEntityHandle& EntityHandle) const
	{
		return ResolvedAnimStates.Find(EntityHandle);
	}

protected:
	/**
	 * Process all removed IDs in FMassInstancedSkinnedMeshComponentSharedData and apply to the Instanced Skinned Mesh component.
	 */
	MASSREPRESENTATION_API void ProcessRemovesForComponent(UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent, FMassInstancedSkinnedMeshComponentSharedData& SharedData);

	/**
	 * Applies changes accumulated in SharedData while manually updating the Instance ID mapping. This approach is done in preparation
	 * to upcoming Instanced Mesh changes to keep the mapping management more secure (by making mapping private and fully component-owned).
	 */
	MASSREPRESENTATION_API void HandleChangesWithExternalIDTrackingForComponent(UInstancedSkinnedMeshComponent& InstancedSkinnedMeshComponent, FMassInstancedSkinnedMeshComponentSharedData& SharedData);

	/** Recreate all the instanced skinned mesh components from the InstancedSkinnedMeshInfos */
	MASSREPRESENTATION_API void ConstructSkinnedMeshComponents();

	/**
	 * Creates LODSignificance ranges for all the meshes indicated by Info
	 * @param MeshRefKeys if not empty will be used when adding individual FMassSkinnedMeshInstanceVisualizationMeshDesc
	 *	instances to LOD significance ranges.
	 */
	MASSREPRESENTATION_API void BuildLODSignificanceForInstancedSkinnedMeshInfo(FMassInstancedSkinnedMeshInfo& Info, TConstArrayView<UInstancedSkinnedMeshComponent*> MeshRefKeys);

	/** Either adds an element to InstancedSkinnedMeshInfos or reuses an existing entry based on InstancedSkinnedMeshInfosFreeIndices*/
	MASSREPRESENTATION_API FSkinnedMeshInstanceVisualizationDescHandle AddInstancedSkinnedMeshInfo(const FSkinnedMeshInstanceVisualizationDesc& Desc);

	/** The information of all the instanced skinned meshes. Make sure to use AddInstancedSkinnedMeshInfo to add elements to it */
	UPROPERTY(Transient)
	TArray<FMassInstancedSkinnedMeshInfo> InstancedSkinnedMeshInfos;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstancedSkinnedMeshInfosDetector);

	/** Indices to InstancedSkinnedMeshInfos that have been released and can be reused */
	TArray<FSkinnedMeshInstanceVisualizationDescHandle> InstancedSkinnedMeshInfosFreeIndices;

	/** Mapping from InstancedSkinnedMeshComponent (indicated by FInstancedSkinnedMeshComponentSharedDataKey) to corresponding SkinnedMeshInstanceVisualizationDescHandle */
	TMap<FInstancedSkinnedMeshComponentSharedDataKey, FSkinnedMeshInstanceVisualizationDescHandle> InstancedSkinnedMeshComponentMap;

	FMassInstancedSkinnedMeshComponentSharedDataMap InstancedSkinnedMeshComponentSharedData;

	/** 
	 * Mapping FMassSkinnedMeshInstanceVisualizationMeshDesc hash to FMassInstancedSkinnedMeshSharedData entries for all FMassSkinnedMeshInstanceVisualizationMeshDesc
	 * that didn't come with Instanced Skinned Mesh Component explicitly provided. Used only for initialization.
	 * Note that FMassSkinnedMeshInstanceVisualizationMeshDesc that were added with InstancedSkinnedMeshComponents provided directly
	 * (via AddVisualDescWithComponents call) will never make it to this map.
	 */
	TMap<uint32, FInstancedSkinnedMeshComponentSharedDataKey> MeshDescToInstancedSkinnedMeshComponentMap;

	/** Indices to InstancedSkinnedMeshInfos that need their Skinned Mesh Component constructed */
	TArray<FSkinnedMeshInstanceVisualizationDescHandle> InstancedSkinnedMeshComponentsRequiringConstructing;

	/** Resolved animation state per entity, populated during EndVisualChanges().
	 * Keyed by entity handle — one entry per entity regardless of how many ISKMs it has.
	 * Read by processors that need to push animation state to UAF during representation handoff. */
	Experimental::TRobinHoodHashMap<FMassEntityHandle, FMassSkinnedMeshResolvedAnimState> ResolvedAnimStates;

};