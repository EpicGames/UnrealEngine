// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "MassRepresentationFragments.h"
#include "GameFramework/Actor.h"
#include "MassVisualizationTrait.generated.h"

#define UE_API MASSREPRESENTATION_API

class UMassRepresentationSubsystem;
class UMassRepresentationActorManagement;
class UMassProcessor;

/** This class has been soft-deprecated. Use MassStationaryVisualizationTrait or MassMovableVisualizationTrait */
UCLASS(MinimalAPI, meta=(DisplayName="DEPRECATED Visualization"))
class UMassVisualizationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()
public:
	UE_API UMassVisualizationTrait();

	/** Instanced static mesh information for this agent */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	mutable FStaticMeshInstanceVisualizationDesc StaticMeshInstanceDesc;

	/** Instanced skinned mesh information for this agent */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	mutable FSkinnedMeshInstanceVisualizationDesc SkinnedMeshInstanceDesc;

	/** Actor class of this agent when spawned in high resolution*/
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<AActor> HighResTemplateActor;

	/** Actor class of this agent when spawned in low resolution*/
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<AActor> LowResTemplateActor;

	/** Allow subclasses to override the representation subsystem to use */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual", meta = (EditCondition = "bCanModifyRepresentationSubsystemClass"))
	TSubclassOf<UMassRepresentationSubsystem> RepresentationSubsystemClass;

	/** Configuration parameters for the representation processor */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FMassRepresentationParameters Params;

	/** Configuration parameters for the visualization LOD processor */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FMassVisualizationLODParameters LODParams;

	/** If set to true will result in the visualization-related fragments being added to server-size entities as well.
	 *  By default only the clients require visualization fragments */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	bool bAllowServerSideVisualization = false;

#if WITH_EDITORONLY_DATA
	/** the property is marked like this to ensure it won't show up in UI */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|Visual")
	bool bCanModifyRepresentationSubsystemClass = true;

	/**
	 * When True, ValidateParams() will require a valid StaticMeshInstanceDesc.
	 *
	 * When False, ValidateParams() skips checking the StaticMeshInstanceDesc.
	 *
	 * Use False when you expect a given Trait to have invalid StaticMeshInstanceDesc
	 * settings, and you don't want errors logged when it occurs.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Mass|Visual", AdvancedDisplay)
	bool bRequireValidStaticMeshInstanceDesc = true;
#endif // WITH_EDITORONLY_DATA

protected:
	/** 
	 * Controls whether StaticMeshInstanceDesc gets registered via FindOrAddStaticMeshDesc call. Setting it to `false` 
	 * can be useful for subclasses to avoid needlessly creating visualization data in RepresentationSubsystem, 
	 * data that will never be used.
	 */
	bool bRegisterStaticMeshDesc = true;

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	UE_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const override;

	/** 
	 * Tests whether StaticMeshInstanceDesc is valid and if not cleans up InOutParamss of EMassRepresentationType::StaticMeshInstance
	 * occurrences. 
	 * @param bStaticMeshDeterminedInvalid if StaticMeshInstanceDesc has already been determined invalid then bStaticMeshDeterminedInvalid
	 *	can be set to `true` to skip the redundant check.
	 */
	UE_DEPRECATED(5.8, "Please use SanitizeMeshParams version for ISKM Support")
	UE_API virtual void SanitizeParams(FMassRepresentationParameters& InOutParams, const bool bStaticMeshDeterminedInvalid = false) const;

	/**
	 * Tests whether StaticMeshInstanceDesc and SkinnedMeshInstanceDesc are valid and if not cleans up InOutParams for those occurrences.
	 * @param bStaticMeshDeterminedInvalid if StaticMeshInstanceDesc has already been determined invalid then bStaticMeshDeterminedInvalid
	 *	can be set to `true` to skip the redundant check.
	 * @param bSkinnedMeshDeterminedInvalid if SkinnedMeshInstanceDesc has already been determined invalid then bSkinnedMeshDeterminedInvalid
	 *	can be set to `true` to skip the redundant check.
	 */
	UE_API virtual void SanitizeMeshParams(FMassRepresentationParameters& InOutParams, const bool bStaticMeshDeterminedInvalid = false, const bool bSkinnedMeshDeterminedInvalid = false) const;

	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool ValidateParams() const;
#endif // WITH_EDITOR
};

#undef UE_API
