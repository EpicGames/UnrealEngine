// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "MassVisualizer.generated.h"

#define UE_API MASSREPRESENTATION_API


class UMassVisualizationComponent;

/**
 * Actor holding the mass visual component responsible to handle the representation of the mass agent as the static mesh instances 
 * There may be a separate instance of these for different types of Agents (Cars, NPC's etc)
 */
UCLASS(MinimalAPI, NotPlaceable, Transient)
class AMassVisualizer : public AActor
{
	GENERATED_BODY()
public:
	UE_API AMassVisualizer();

	/** Visualization component is guaranteed to exist if this class is created */
	UMassVisualizationComponent& GetVisualizationComponent() const { return *VisComponent; }

protected:
	UPROPERTY()
	TObjectPtr<UMassVisualizationComponent> VisComponent;
};

#undef UE_API
