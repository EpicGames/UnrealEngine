// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSimulationSubsystem.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassSimulationSettings.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntitySettings.h"
#include "Engine/Engine.h"
#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSimulationSubsystem)

DEFINE_LOG_CATEGORY(LogMassSim);

UMassSimulationSubsystem::FOnSimulationStarted UMassSimulationSubsystem::OnSimulationStarted;

namespace UE::Mass::Simulation
{
	bool bDoEntityCompaction = true;
	bool bSimulationTickingEnabled = true;

	namespace Private
	{
	FAutoConsoleVariableRef CVars[] = {
		{TEXT("mass.EntityCompaction"), bDoEntityCompaction, TEXT("Maximize the number of entities per chunk"), ECVF_Cheat}
		, {TEXT("mass.SimulationTickingEnabled"), bSimulationTickingEnabled, TEXT("Controls whether Mass simulation ticking is allowed in game worlds. Upon changing the value it also stops/starts Mass simulation ticking, if needed"), 
			FConsoleVariableDelegate::CreateStatic(UMassSimulationSubsystem::HandleSimulationTickingEnabledCVarChange)}
		};
	}
}

//----------------------------------------------------------------------//
// UMassSimulationSubsystem
//----------------------------------------------------------------------//
UMassSimulationSubsystem::UMassSimulationSubsystem(const FObjectInitializer&)
	: PhaseManager(new FMassProcessingPhaseManager())
{
}

void UMassSimulationSubsystem::BeginDestroy()
{
	ReleaseEventHandles();

	if (bSimulationStarted)
	{
		check(EntityManager);
		StopSimulation();
	}

	Super::BeginDestroy();
}

FMassProcessingPhase::FOnPhaseEvent& UMassSimulationSubsystem::GetOnProcessingPhaseStarted(const EMassProcessingPhase Phase)
{
	return PhaseManager->GetOnPhaseStart(Phase);
}

FMassProcessingPhase::FOnPhaseEvent& UMassSimulationSubsystem::GetOnProcessingPhaseFinished(const EMassProcessingPhase Phase)
{
	return PhaseManager->GetOnPhaseEnd(Phase);
}

UMassSimulationSubsystem::FOnSimulationStarted& UMassSimulationSubsystem::GetOnSimulationStarted()
{
	return OnSimulationStarted;
}

bool UMassSimulationSubsystem::IsDuringMassProcessing() const 
{
	return EntityManager.IsValid() && EntityManager->IsProcessing();
}

void UMassSimulationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UMassEntitySubsystem* EntitySubsystem = Collection.InitializeDependency<UMassEntitySubsystem>();
	check(EntitySubsystem);
	EntityManager = EntitySubsystem->GetMutableEntityManager().AsShared();
	
	GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).AddUObject(this, &UMassSimulationSubsystem::OnProcessingPhaseStarted, EMassProcessingPhase::PrePhysics);

	HandleLateCreation();
}

void UMassSimulationSubsystem::Deinitialize()
{
	GetOnProcessingPhaseStarted(EMassProcessingPhase::PrePhysics).RemoveAll(this);
	StopSimulation();

	PhaseManager->Deinitialize();
	EntityManager.Reset();

	ReleaseEventHandles();

	Super::Deinitialize();
}

void UMassSimulationSubsystem::PostInitialize()
{
	Super::PostInitialize();

#if WITH_EDITOR
	UWorld* World = GetWorld();
	if (GEditor && World != nullptr && !World->IsGameWorld())
	{
		// in editor worlds we need to rebuild the pipeline at this point since OnWorldBeginPlay won't be called
		RebuildTickPipeline();

		PieBeginEventHandle = FEditorDelegates::BeginPIE.AddUObject(this, &UMassSimulationSubsystem::OnPieBegin);
		PieEndedEventHandle = FEditorDelegates::PrePIEEnded.AddUObject(this, &UMassSimulationSubsystem::OnPieEnded);

		UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
		check(Settings);
		MassEntitySettingsChangeHandle = Settings->GetOnSettingsChange().AddUObject(this, &UMassSimulationSubsystem::OnMassEntitySettingsChange);

		// note that this starts ticking for the editor world
		StartSimulation(*World);
	}
#endif // WITH_EDITOR
}

void UMassSimulationSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	// To evaluate the effective processors execution mode, we need to wait on OnWorldBeginPlay before calling
	// RebuildTickPipeline as we are sure by this time the network is setup correctly.
	RebuildTickPipeline();
	// note that since we're in this function we're tied to a game world. This means the StartSimulation in 
	// PostInitialize haven't been called.
	StartSimulation(InWorld);
}

void UMassSimulationSubsystem::RegisterDynamicProcessor(UMassProcessor& Processor)
{
	PhaseManager->RegisterDynamicProcessor(Processor);
}

void UMassSimulationSubsystem::UnregisterDynamicProcessor(UMassProcessor& Processor)
{
	PhaseManager->UnregisterDynamicProcessor(Processor);
}

void UMassSimulationSubsystem::HandleSimulationTickingEnabledCVarChange(IConsoleVariable*)
{
	if (GEngine == nullptr)
	{
		return;
	}

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType != EWorldType::Inactive)
		{
			UWorld* World = Context.World();
			
			// we only want to affect game worlds
			if (World == nullptr || World->IsGameWorld() == false)
			{
				continue;
			}

			if (UMassSimulationSubsystem* MassSimulationSubsystem = UWorld::GetSubsystem<UMassSimulationSubsystem>(World))
			{
				if (UE::Mass::Simulation::bSimulationTickingEnabled)
				{
					// if enabling call StartSimulation ony if the world has already begun play. Otherwise the 
					// simulation ticking will be started at the right time automatically
					if (MassSimulationSubsystem->HasCalledBeginPlay())
					{
						MassSimulationSubsystem->StartSimulation(*World);
					}
				}
				else
				{
					MassSimulationSubsystem->StopSimulation();
				}
			}
		}
	}
}

void UMassSimulationSubsystem::RebuildTickPipeline()
{
	UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
	check(Settings);
	TConstArrayView<FMassProcessingPhaseConfig> ProcessingPhasesConfig = Settings->GetProcessingPhasesConfig();
	FString DependencyGraphFileName;

#if WITH_EDITOR
	const UWorld* World = GetWorld();
	if (World != nullptr && !Settings->DumpDependencyGraphFileName.IsEmpty())
	{
		DependencyGraphFileName = FString::Printf(TEXT("%s_%s"), *Settings->DumpDependencyGraphFileName, *ToString(World->GetNetMode()));
	}
#endif // WITH_EDITOR

	PhaseManager->Initialize(*this, ProcessingPhasesConfig, DependencyGraphFileName);
}

void UMassSimulationSubsystem::StartSimulation(UWorld& InWorld)
{
	if (bSimulationStarted)
	{
		return;
	}

	if (UE::Mass::Simulation::bSimulationTickingEnabled)
	{
		PhaseManager->Start(InWorld);

		bSimulationStarted = true;

		OnSimulationStarted.Broadcast(&InWorld);
	}
}

void UMassSimulationSubsystem::StopSimulation()
{
	if (bSimulationStarted == false)
	{
		return;
	}

	PhaseManager->Stop();

	bSimulationStarted = false;
}

void UMassSimulationSubsystem::PauseSimulation()
{
	if (bSimulationPaused == true)
	{
		return;
	}

	bSimulationPaused = true;

	PhaseManager->Pause();

	OnSimulationPaused.Broadcast(this);
}

void UMassSimulationSubsystem::ResumeSimulation()
{
	if (bSimulationPaused == false)
	{
		return;
	}

	bSimulationPaused = false;

	PhaseManager->Resume();

	OnSimulationResumed.Broadcast(this);
}

void UMassSimulationSubsystem::OnProcessingPhaseStarted(const float DeltaSeconds, const EMassProcessingPhase Phase) const
{
	switch (Phase)
	{
		case EMassProcessingPhase::PrePhysics:
			{
				const UMassSimulationSettings* SimSettings = GetDefault<UMassSimulationSettings>();
				check(SimSettings);
				if (UE::Mass::Simulation::bDoEntityCompaction && SimSettings->bEntityCompactionEnabled)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DoEntityCompaction);
					check(EntityManager);
					EntityManager->DoEntityCompaction(SimSettings->DesiredEntityCompactionTimeSlicePerTick);
				}
			}
			break;
		default:
			// unhandled phases, by design, not every phase needs to be handled by the Actor subsystem
			break;
	}
}

#if WITH_EDITOR
void UMassSimulationSubsystem::OnPieBegin(const bool bIsSimulation)
{
	// called so that we're not processing phases for the editor world while PIE/SIE is running
	StopSimulation();
}

void UMassSimulationSubsystem::OnPieEnded(const bool bIsSimulation)
{
	UWorld* World = GetWorld();
	if (World && !bSimulationStarted)
	{
		// Resume processing phases in the editor world.
		StartSimulation(*World);
	}
}

void UMassSimulationSubsystem::OnMassEntitySettingsChange(const FPropertyChangedEvent& PropertyChangedEvent)
{
	RebuildTickPipeline();
}
#endif // WITH_EDITOR

void UMassSimulationSubsystem::ReleaseEventHandles()
{
#if WITH_EDITOR
	if (PieBeginEventHandle.IsValid())
	{
		FEditorDelegates::BeginPIE.Remove(PieBeginEventHandle);
		PieBeginEventHandle.Reset();
	}
	if (PieEndedEventHandle.IsValid())
	{
		FEditorDelegates::PrePIEEnded.Remove(PieEndedEventHandle);
		PieEndedEventHandle.Reset();
	}
	if (MassEntitySettingsChangeHandle.IsValid())
	{
		if (UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>())
		{
			Settings->GetOnSettingsChange().Remove(MassEntitySettingsChangeHandle);
		}
		MassEntitySettingsChangeHandle.Reset();
	}
#endif // WITH_EDITOR

	OnReleasingEventHandles();
}
