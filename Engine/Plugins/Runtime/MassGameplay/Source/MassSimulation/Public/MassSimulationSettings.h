// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassSimulationSettings.generated.h"

#define GET_MASSSIMULATION_CONFIG_VALUE(a) \
	UE_DEPRECATED_MACRO(5.8, "Use GetDefault<UMassSimulationSettings>() (or GetMutableDefault) directly.") \
	(GetMutableDefault<UMassSimulationSettings>()->a)

/**
 * Implements the settings for MassSimulation
 */
UCLASS(MinimalAPI, config = Mass, defaultconfig, DisplayName = "Mass Simulation")
class UMassSimulationSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	/** The desired budget in seconds allowed to do actor spawning per frame */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	double DesiredActorSpawningTimeSlicePerTick = 0.0015;

	/** The desired budget in seconds allowed to do actor destruction per frame */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	double DesiredActorDestructionTimeSlicePerTick = 0.0005;

	/** 
	 * Whether the Entity Compaction is allowed. When enabled all the archetypes will be processed on a regular basis
	 * and the entities will be moved around to optimize archetype chunk memory utilization. Note that this might not be
	 * desired if your set up results in individual chunks being often not compatible due to the number and value of chunk 
	 * and shared fragments.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	bool bEntityCompactionEnabled = true;

	/** The desired budget in seconds allowed to do entity compaction per frame */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	double DesiredEntityCompactionTimeSlicePerTick = 0.005;

	/** The time to wait before retrying to spawn an actor that previously failed, default 10 seconds */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	float DesiredActorFailedSpawningRetryTimeInterval = 5.0f;

	/** The distance a failed spawned actor needs to move before we retry, default 10 meters */
	UPROPERTY(EditDefaultsOnly, Category = "Runtime", config)
	float DesiredActorFailedSpawningRetryMoveDistance = 500.0f;
};
