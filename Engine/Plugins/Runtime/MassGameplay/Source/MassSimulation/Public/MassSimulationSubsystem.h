// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "MassProcessingTypes.h"
#include "MassProcessingPhaseManager.h"
#include "MassSubsystemBase.h"
#include "MassSimulationSubsystem.generated.h"


struct FMassEntityManager;
class IConsoleVariable;

DECLARE_LOG_CATEGORY_EXTERN(LogMassSim, Log, All);

UCLASS(config = Game, defaultconfig, MinimalAPI)
class UMassSimulationSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSimulationStarted, UWorld* /*World*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSimulationPauseEvent, TNotNull<UMassSimulationSubsystem*> /*this*/);
	
	MASSSIMULATION_API UMassSimulationSubsystem(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	const FMassProcessingPhaseManager& GetPhaseManager() const;
	FMassProcessingPhaseManager& GetMutablePhaseManager();

	MASSSIMULATION_API FMassProcessingPhase::FOnPhaseEvent& GetOnProcessingPhaseStarted(const EMassProcessingPhase Phase);
	MASSSIMULATION_API FMassProcessingPhase::FOnPhaseEvent& GetOnProcessingPhaseFinished(const EMassProcessingPhase Phase);
	MASSSIMULATION_API static FOnSimulationStarted& GetOnSimulationStarted();

	FOnSimulationPauseEvent& GetOnSimulationPaused();
	FOnSimulationPauseEvent& GetOnSimulationResumed();

	MASSSIMULATION_API void RegisterDynamicProcessor(UMassProcessor& Processor);
	MASSSIMULATION_API void UnregisterDynamicProcessor(UMassProcessor& Processor);

	bool IsSimulationStarted() const;

	/** @return whether hosted EntityManager is currently, actively being used for processing purposes. Equivalent to calling FMassEntityManager.IsProcessing() */
	MASSSIMULATION_API bool IsDuringMassProcessing() const;

	/** Starts/stops simulation ticking for all worlds, based on new `mass.SimulationTickingEnabled` cvar value */
	MASSSIMULATION_API static void HandleSimulationTickingEnabledCVarChange(IConsoleVariable*);

	/**
	 * Determine if this Simulation is currently paused.
	 * 
	 * While paused, phases will transition as usual, but processors
	 * will not be executed.
	 * 
	 * @return True if this Simulation is currently paused; else False
	 */
	bool IsSimulationPaused() const;

	/** Pause the simulation from executing processors during phase ticks */
	MASSSIMULATION_API void PauseSimulation();

	/** Resume the simulation executing processors during phase ticks */
	MASSSIMULATION_API void ResumeSimulation();

protected:
	// UWorldSubsystem BEGIN
	MASSSIMULATION_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSSIMULATION_API virtual void PostInitialize() override;
	MASSSIMULATION_API virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	MASSSIMULATION_API virtual void Deinitialize() override;
	// UWorldSubsystem END
	MASSSIMULATION_API virtual void BeginDestroy() override;
	
	MASSSIMULATION_API void RebuildTickPipeline();

	MASSSIMULATION_API void StartSimulation(UWorld& InWorld);
	MASSSIMULATION_API void StopSimulation();

	MASSSIMULATION_API void OnProcessingPhaseStarted(const float DeltaSeconds, const EMassProcessingPhase Phase) const;

#if WITH_EDITOR
	MASSSIMULATION_API void OnPieBegin(const bool bIsSimulation);
	MASSSIMULATION_API void OnPieEnded(const bool bIsSimulation);
	MASSSIMULATION_API void OnMassEntitySettingsChange(const FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

	/** Called when it's time to clean up all the delegate handles. Override OnReleasingEventHandles to add more handles cleanup */
	MASSSIMULATION_API void ReleaseEventHandles();
	
	/** Override to add more handle cleanup. Will get automatically called by ReleaseEventHandles */
	virtual void OnReleasingEventHandles() {}

protected:

	TSharedPtr<FMassEntityManager> EntityManager;

	TSharedRef<FMassProcessingPhaseManager> PhaseManager;

	static MASSSIMULATION_API FOnSimulationStarted OnSimulationStarted;

	FOnSimulationPauseEvent OnSimulationPaused;
	FOnSimulationPauseEvent OnSimulationResumed;

	UPROPERTY()
	FMassRuntimePipeline RuntimePipeline;

	float CurrentDeltaSeconds = 0.f;
	bool bTickInProgress = false;
	bool bSimulationStarted = false;
	bool bSimulationPaused = false;

#if WITH_EDITOR
	FDelegateHandle PieBeginEventHandle;
	FDelegateHandle PieEndedEventHandle;

	FDelegateHandle MassEntitySettingsChangeHandle;
#endif // WITH_EDITOR
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline UMassSimulationSubsystem::FOnSimulationPauseEvent& UMassSimulationSubsystem::GetOnSimulationPaused()
{
	return OnSimulationPaused;
}

inline UMassSimulationSubsystem::FOnSimulationPauseEvent& UMassSimulationSubsystem::GetOnSimulationResumed()
{
	return OnSimulationResumed;
}

inline bool UMassSimulationSubsystem::IsSimulationPaused() const
{
	return bSimulationPaused;
}

inline const FMassProcessingPhaseManager& UMassSimulationSubsystem::GetPhaseManager() const
{
	return PhaseManager.Get();
}

inline FMassProcessingPhaseManager& UMassSimulationSubsystem::GetMutablePhaseManager()
{
	return PhaseManager.Get();
}

inline bool UMassSimulationSubsystem::IsSimulationStarted() const
{
	return bSimulationStarted;
}
