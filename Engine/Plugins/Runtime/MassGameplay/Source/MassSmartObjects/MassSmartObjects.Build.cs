// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassSmartObjects : ModuleRules
	{
		public MassSmartObjects(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;

			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MassCore",
					"MassEntity",
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"MassCommon",
					"MassLOD",
					"MassMovement",
					"MassSignals",
					"MassSimulation",
					"MassSpawner",
					"SmartObjectsModule",
					"ZoneGraph",
					"ZoneGraphAnnotations",
					"MassGameplayExternalTraits"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MassActors"
				}
			);
		}
	}
}
