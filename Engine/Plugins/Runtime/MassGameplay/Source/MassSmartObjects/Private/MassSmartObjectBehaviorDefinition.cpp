// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectBehaviorDefinition.h"

#include "MassCommandBuffer.h"
#include "MassSmartObjectFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectBehaviorDefinition)

void USmartObjectMassBehaviorDefinition::Activate(FMassCommandBuffer& CommandBuffer, const FMassBehaviorEntityContext& EntityContext) const
{
	FMassSmartObjectTimedBehaviorFragment TimedBehaviorFragment;
	TimedBehaviorFragment.UseTime = UseTime;
	CommandBuffer.PushCommand<FMassCommandAddFragmentInstances>(EntityContext.EntityView.GetEntity(), TimedBehaviorFragment);
}

void USmartObjectMassBehaviorDefinition::Deactivate(FMassCommandBuffer& CommandBuffer, const FMassBehaviorEntityContext& EntityContext) const
{
	CommandBuffer.RemoveFragment<FMassSmartObjectTimedBehaviorFragment>(EntityContext.EntityView.GetEntity());
}
