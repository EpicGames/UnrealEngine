// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectHandler.h"
#include "MassSmartObjectBehaviorDefinition.h"
#include "MassCommonTypes.h"
#include "MassExecutionContext.h"
#include "SmartObjectSubsystem.h"
#include "VisualLogger/VisualLogger.h"
#include "MassCommandBuffer.h"
#include "MassSignalSubsystem.h"
#include "MassSmartObjectFragments.h"
#include "Engine/World.h"
#include "MassDebugger.h"


namespace UE::Mass::SmartObject
{
	void StopInteraction(FMassCommandBuffer& CommandBuffer, const FMassBehaviorEntityContext& Context, const FSmartObjectClaimHandle& ClaimHandle, const EMassSmartObjectInteractionStatus NewStatus)
	{
		FMassSmartObjectUserFragment& User = Context.EntityView.GetFragmentData<FMassSmartObjectUserFragment>();
		if ( User.InteractionHandle == ClaimHandle)
		{
			if (User.InteractionStatus == EMassSmartObjectInteractionStatus::InProgress ||
				User.InteractionStatus == EMassSmartObjectInteractionStatus::BehaviorCompleted)
			{
				if (const USmartObjectMassBehaviorDefinition* BehaviorDefinition = Context.SmartObjectSubsystem.GetBehaviorDefinition<USmartObjectMassBehaviorDefinition>(User.InteractionHandle))
				{
					BehaviorDefinition->Deactivate(CommandBuffer, Context);
				}
			}

			User.InteractionStatus = NewStatus;
			User.InteractionHandle.Invalidate();
		}
	}

	struct FPayload
	{
		FMassEntityHandle Entity;
		TSharedPtr<FMassEntityManager> EntityManager;
		TWeakObjectPtr<USmartObjectSubsystem> SmartObjectSubsystem;
		TWeakObjectPtr<UMassSignalSubsystem> SignalSubsystem;
	};

	void OnSlotInvalidated(const FSmartObjectClaimHandle& ClaimHandle, const ESmartObjectSlotState State, FPayload Payload)
	{
		USmartObjectSubsystem* SmartObjectSubsystem = Payload.SmartObjectSubsystem.Get();
		FMassEntityManager* EntityManager = Payload.EntityManager.Get();
		UMassSignalSubsystem* SignalSubsystem = Payload.SignalSubsystem.Get();
		if (SmartObjectSubsystem != nullptr && EntityManager != nullptr && SignalSubsystem != nullptr && EntityManager->IsEntityActive(Payload.Entity))
		{
			const FMassBehaviorEntityContext Context(FMassEntityView(*EntityManager, Payload.Entity), *SmartObjectSubsystem);

			StopInteraction(EntityManager->Defer(), Context, ClaimHandle, EMassSmartObjectInteractionStatus::Aborted);
			if (EntityManager->IsProcessing() == false)
			{
				EntityManager->FlushCommands();
			}

			SignalSubsystem->SignalEntity(UE::Mass::Signals::SmartObjectInteractionAborted, Payload.Entity);
		}
	}
} // UE::Mass::SmartObject;

//----------------------------------------------------------------------//
// FMassSmartObjectHandler
//----------------------------------------------------------------------//
FMassSmartObjectRequestID FMassSmartObjectHandler::FindCandidatesAsync(const FMassEntityHandle RequestingEntity, const FGameplayTagContainer& UserTags, const FGameplayTagQuery& ActivityRequirements, const FVector& Location) const
{
	UE::Mass::SmartObject::FFindCandidatesParameters Parameters
	{
		.UserTags = UserTags,
		.ActivityRequirements = ActivityRequirements,
		.Location = Location
	};

	return FindCandidatesAsync(RequestingEntity, MoveTemp(Parameters));
}

FMassSmartObjectRequestID FMassSmartObjectHandler::FindCandidatesAsync(const FMassEntityHandle RequestingEntity, const FGameplayTagContainer& UserTags, const FGameplayTagQuery& ActivityRequirements, const FZoneGraphCompactLaneLocation& LaneLocation) const
{
	UE::Mass::SmartObject::FFindCandidatesParameters Parameters
	{
		.UserTags = UserTags,
		.ActivityRequirements = ActivityRequirements,
		.LaneLocation = LaneLocation
	};

	return FindCandidatesAsync(RequestingEntity, MoveTemp(Parameters));
}

FMassSmartObjectRequestID FMassSmartObjectHandler::FindCandidatesAsync(const FMassEntityHandle RequestingEntity, UE::Mass::SmartObject::FFindCandidatesParameters&& Parameters) const
{
	const FMassEntityHandle ReservedEntity = ExecutionContext.GetEntityManagerChecked().ReserveEntity();
	FMassSmartObjectRequestResultFragment ResultFragment;

	if (Parameters.LaneLocation.LaneHandle.IsValid())
	{
		FMassSmartObjectLaneLocationRequestFragment RequestFragment;
		RequestFragment.CompactLaneLocation = Parameters.LaneLocation;
		RequestFragment.RequestingEntity = RequestingEntity;
		RequestFragment.UserTags = Parameters.UserTags;
		RequestFragment.ActivityRequirements = Parameters.ActivityRequirements;

		if (Parameters.MRUSlots.NumSlots > 0)
		{
			UE::Mass::SmartObject::FMRUSlotsFragment MRUSlotsFragment{.Slots = Parameters.MRUSlots};
			ExecutionContext.Defer().PushCommand<FMassCommandAddFragmentInstances>(ReservedEntity, RequestFragment, MRUSlotsFragment, ResultFragment);
		}
		else
		{
			ExecutionContext.Defer().PushCommand<FMassCommandAddFragmentInstances>(ReservedEntity, RequestFragment, ResultFragment);
		}
	}
	else
	{
		FMassSmartObjectWorldLocationRequestFragment RequestFragment;
		RequestFragment.SearchOrigin = Parameters.Location;
		RequestFragment.RequestingEntity = RequestingEntity;
		RequestFragment.UserTags = Parameters.UserTags;
		RequestFragment.ActivityRequirements = Parameters.ActivityRequirements;

		if (Parameters.MRUSlots.NumSlots > 0)
		{
			UE::Mass::SmartObject::FMRUSlotsFragment MRUSlotsFragment{ .Slots = Parameters.MRUSlots };
			ExecutionContext.Defer().PushCommand<FMassCommandAddFragmentInstances>(ReservedEntity, RequestFragment, MRUSlotsFragment, ResultFragment);
		}
		else
		{
			ExecutionContext.Defer().PushCommand<FMassCommandAddFragmentInstances>(ReservedEntity, RequestFragment, ResultFragment);
		}
	}

	return FMassSmartObjectRequestID(ReservedEntity);
}

const FMassSmartObjectCandidateSlots* FMassSmartObjectHandler::GetRequestCandidates(const FMassSmartObjectRequestID& RequestID) const
{
	const FMassEntityHandle RequestEntity = static_cast<FMassEntityHandle>(RequestID);
	const FMassEntityManager& EntityManager = ExecutionContext.GetEntityManagerChecked();

	if (!ensureMsgf(EntityManager.IsEntityValid(RequestEntity), TEXT("Invalid request.")))
	{
		return nullptr;
	}

	// Check if entity is built by now.
	if (!EntityManager.IsEntityBuilt(RequestEntity))
	{
		return nullptr;
	}

	const FMassSmartObjectRequestResultFragment& RequestFragment = EntityManager.GetFragmentDataChecked<FMassSmartObjectRequestResultFragment>(RequestEntity);

	return RequestFragment.bProcessed ? &RequestFragment.Candidates : nullptr;
}

void FMassSmartObjectHandler::RemoveRequest(const FMassSmartObjectRequestID& RequestID) const
{
	const FMassEntityHandle RequestEntity = static_cast<FMassEntityHandle>(RequestID);
	ExecutionContext.Defer().DestroyEntity(RequestEntity);
}

FSmartObjectClaimHandle FMassSmartObjectHandler::ClaimCandidate(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FMassSmartObjectCandidateSlots& Candidates, ESmartObjectClaimPriority ClaimPriority) const
{
	checkf(!User.InteractionHandle.IsValid(), TEXT("User should not already have an interaction."));
	
	const TConstArrayView<FSmartObjectCandidateSlot> View = MakeArrayView(Candidates.Slots.GetData(), Candidates.NumSlots);

	FSmartObjectClaimHandle ClaimedSlot;
	
	for (const FSmartObjectCandidateSlot& CandidateSlot : View)
	{
		ClaimedSlot = ClaimSmartObject(Entity, User, CandidateSlot.Result, ClaimPriority);  
		if (ClaimedSlot.IsValid())
		{
#if WITH_MASSGAMEPLAY_DEBUG
			UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
				&SmartObjectSubsystem,
				LogSmartObject,
				Log,
				TEXT("[%s] claimed [%s]"),
				*Entity.DebugGetDescription(),
				*LexToString(CandidateSlot.Result));
#endif // WITH_MASSGAMEPLAY_DEBUG
			break;
		}
	}

	return ClaimedSlot;
}

FSmartObjectClaimHandle FMassSmartObjectHandler::ClaimSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FSmartObjectRequestResult& RequestResult, ESmartObjectClaimPriority ClaimPriority) const
{
	const FSmartObjectClaimHandle ClaimHandle = SmartObjectSubsystem.MarkSlotAsClaimed(RequestResult.SlotHandle, ClaimPriority, FConstStructView::Make(FSmartObjectMassEntityUserData(Entity)));

#if WITH_MASSGAMEPLAY_DEBUG
	UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
		&SmartObjectSubsystem,
		LogSmartObject,
		Log,
		TEXT("[%s] claim for [%s] %s"),
		*Entity.DebugGetDescription(),
		*LexToString(RequestResult),
		ClaimHandle.IsValid() ? TEXT("Succeeded") : TEXT("Failed"));
#endif // WITH_MASSGAMEPLAY_DEBUG

	// Register callback to abort interaction if slot gets invalidated.
	// Callback will be unregistered by UMassSmartObjectUserFragmentDeinitializer
	UE::Mass::SmartObject::FPayload Payload;
	Payload.Entity = Entity;
	Payload.EntityManager = ExecutionContext.GetEntityManagerChecked().AsShared();
	Payload.SmartObjectSubsystem = &SmartObjectSubsystem;
	Payload.SignalSubsystem = &SignalSubsystem;
	SmartObjectSubsystem.RegisterSlotInvalidationCallback(ClaimHandle, FOnSlotInvalidated::CreateStatic(&UE::Mass::SmartObject::OnSlotInvalidated, Payload));
	
	return ClaimHandle;
}

bool FMassSmartObjectHandler::StartUsingSmartObject(
	const FMassEntityHandle Entity,
	FMassSmartObjectUserFragment& User,
	const FSmartObjectClaimHandle ClaimHandle) const
{
#if WITH_MASSGAMEPLAY_DEBUG
	UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
		&SmartObjectSubsystem,
		LogSmartObject,
		Log,
		TEXT("[%s] starts using [%s]"),
		*Entity.DebugGetDescription(),
		*LexToString(User.InteractionHandle));
#endif // WITH_MASSGAMEPLAY_DEBUG

	const USmartObjectMassBehaviorDefinition* BehaviorDefinition = SmartObjectSubsystem.MarkSlotAsOccupied<USmartObjectMassBehaviorDefinition>(ClaimHandle);
	if (BehaviorDefinition == nullptr)
	{
		return false;
	}

	User.InteractionStatus = EMassSmartObjectInteractionStatus::InProgress;
	User.InteractionHandle = ClaimHandle;

	// Activate behavior
	const FMassBehaviorEntityContext Context(FMassEntityView(ExecutionContext.GetEntityManagerChecked(), Entity), SmartObjectSubsystem);
	BehaviorDefinition->Activate(ExecutionContext.Defer(), Context);

	return true;
}


void FMassSmartObjectHandler::StopUsingSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const EMassSmartObjectInteractionStatus NewStatus) const
{
#if WITH_MASSGAMEPLAY_DEBUG
	UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
		&SmartObjectSubsystem,
		LogSmartObject,
		Log,
		TEXT("[%s] releases handle [%s]"),
		*Entity.DebugGetDescription(),
		*LexToString(User.InteractionHandle));
#endif // WITH_MASSGAMEPLAY_DEBUG

#if DO_ENSURE
	const EMassSmartObjectInteractionStatus CurrentStatus = User.InteractionStatus;

	switch (CurrentStatus)
	{
	case EMassSmartObjectInteractionStatus::Unset:
		ensureMsgf(NewStatus == EMassSmartObjectInteractionStatus::Unset,
			TEXT("Not expecting new status for inactive interaction. Received %s"),
			*UEnum::GetValueAsString(NewStatus));
		break;

	case EMassSmartObjectInteractionStatus::InProgress:
		ensureMsgf(NewStatus == EMassSmartObjectInteractionStatus::BehaviorCompleted
			|| NewStatus == EMassSmartObjectInteractionStatus::TaskCompleted
			|| NewStatus == EMassSmartObjectInteractionStatus::Aborted,
			TEXT("Expecting status 'BehaviorCompleted', 'TaskCompleted' or 'Aborted' for in progress interaction. Received %s"),
			*UEnum::GetValueAsString(NewStatus));
		break;

	case EMassSmartObjectInteractionStatus::BehaviorCompleted:
		ensureMsgf(NewStatus == EMassSmartObjectInteractionStatus::TaskCompleted || NewStatus == EMassSmartObjectInteractionStatus::Aborted,
			TEXT("Expecting status 'TaskCompleted' or 'Aborted' for 'BehaviorCompleted' interaction. Received %s"),
			*UEnum::GetValueAsString(NewStatus));
		break;

	case EMassSmartObjectInteractionStatus::TaskCompleted:
	case EMassSmartObjectInteractionStatus::Aborted:
		ensureMsgf(false, TEXT("Not expecting status changes for'Completed' or 'Aborted' interaction. Current %s - Received %s"),
			*UEnum::GetValueAsString(CurrentStatus),
			*UEnum::GetValueAsString(NewStatus));
		break;

	default:
		ensureMsgf(false, TEXT("Unhandled state: %s"), *UEnum::GetValueAsString(NewStatus));
	}
#endif

	const FMassBehaviorEntityContext Context(FMassEntityView(ExecutionContext.GetEntityManagerChecked(), Entity), SmartObjectSubsystem);
	UE::Mass::SmartObject::StopInteraction(ExecutionContext.Defer(), Context, User.InteractionHandle, NewStatus);
}


void FMassSmartObjectHandler::ReleaseSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FSmartObjectClaimHandle ClaimHandle) const
{
#if WITH_MASSGAMEPLAY_DEBUG
	UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
		&SmartObjectSubsystem,
		LogSmartObject,
		Log,
		TEXT("[%s] releases handle [%s]"),
		*Entity.DebugGetDescription(),
		*LexToString(User.InteractionHandle));
#endif // WITH_MASSGAMEPLAY_DEBUG

#if DO_ENSURE
	if (User.InteractionStatus == EMassSmartObjectInteractionStatus::InProgress)
	{
		ensureMsgf(false, TEXT("Expecting interaction to be compled before releasing the SmartObject. Current state %s"), *UEnum::GetValueAsString(User.InteractionStatus));
	}
#endif

	const FMassBehaviorEntityContext Context(FMassEntityView(ExecutionContext.GetEntityManagerChecked(), Entity), SmartObjectSubsystem);

	SmartObjectSubsystem.UnregisterSlotInvalidationCallback(ClaimHandle);

	Context.SmartObjectSubsystem.MarkSlotAsFree(ClaimHandle);
}
