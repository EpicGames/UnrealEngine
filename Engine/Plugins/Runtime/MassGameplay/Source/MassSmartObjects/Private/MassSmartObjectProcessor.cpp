// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectProcessor.h"
#include "MassCommandBuffer.h"
#include "MassCommonTypes.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "MassSmartObjectBehaviorDefinition.h"
#include "MassSmartObjectFragments.h"
#include "MassSmartObjectRequest.h"
#include "MassSmartObjectSettings.h"
#include "MassSmartObjectTypes.h"
#include "Misc/ScopeExit.h"
#include "SmartObjectOctree.h"
#include "SmartObjectSubsystem.h"
#include "SmartObjectZoneAnnotations.h"
#include "VisualLogger/VisualLogger.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphSubsystem.h"


//----------------------------------------------------------------------//
// UMassSmartObjectCandidatesFinderProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectProcessor)
void UMassSmartObjectCandidatesFinderProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	WorldRequestQuery.AddRequirement<FMassSmartObjectWorldLocationRequestFragment>(EMassFragmentAccess::ReadOnly);
	WorldRequestQuery.AddRequirement<FMassSmartObjectRequestResultFragment>(EMassFragmentAccess::ReadWrite);
	WorldRequestQuery.AddRequirement<UE::Mass::SmartObject::FMRUSlotsFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	WorldRequestQuery.AddTagRequirement<FMassSmartObjectCompletedRequestTag>(EMassFragmentPresence::None);
	WorldRequestQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadOnly);
	WorldRequestQuery.RegisterWithProcessor(*this);

	LaneRequestQuery.AddRequirement<FMassSmartObjectLaneLocationRequestFragment>(EMassFragmentAccess::ReadOnly);
	LaneRequestQuery.AddRequirement<FMassSmartObjectRequestResultFragment>(EMassFragmentAccess::ReadWrite);
	LaneRequestQuery.AddRequirement<UE::Mass::SmartObject::FMRUSlotsFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	LaneRequestQuery.AddTagRequirement<FMassSmartObjectCompletedRequestTag>(EMassFragmentPresence::None);
	LaneRequestQuery.AddSubsystemRequirement<UZoneGraphSubsystem>(EMassFragmentAccess::ReadOnly);
	LaneRequestQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadOnly);
	LaneRequestQuery.RegisterWithProcessor(*this);

	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
	ProcessorRequirements.AddSubsystemRequirement<UZoneGraphAnnotationSubsystem>(EMassFragmentAccess::ReadOnly);
}

UMassSmartObjectCandidatesFinderProcessor::UMassSmartObjectCandidatesFinderProcessor()
	: WorldRequestQuery(*this)
	, LaneRequestQuery(*this)
{
	// 1. Frame T Behavior create a request(deferred entity creation)
	// 2. Frame T+1: Processor execute the request might mark it as done(deferred add tag flushed at the end of the frame)
	// 3. Frame T+1: Behavior could cancel request(deferred destroy entity)
	// If the processor does not run before the behaviors, step 2 and 3 are flipped and it will crash while flushing the deferred commands
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Behavior);
}

void UMassSmartObjectCandidatesFinderProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	UMassSignalSubsystem& SignalSubsystem = ExecutionContext.GetMutableSubsystemChecked<UMassSignalSubsystem>();
	const UZoneGraphAnnotationSubsystem& AnnotationSubsystem = ExecutionContext.GetSubsystemChecked<UZoneGraphAnnotationSubsystem>();
	
	// Create filter
	FSmartObjectRequestFilter Filter;
	Filter.BehaviorDefinitionClasses = { USmartObjectMassBehaviorDefinition::StaticClass() };

	// Build list of request owner entities to send a completion signal
	TArray<FMassEntityHandle> EntitiesToSignal;

	auto BeginRequestProcessing = [](const FMassEntityHandle Entity, FMassExecutionContext& Context, FMassSmartObjectRequestResultFragment& Result)
	{
		Context.Defer().AddTag<FMassSmartObjectCompletedRequestTag>(Entity);
		Result.Candidates.NumSlots = 0;
	};

	auto EndRequestProcessing = [](const UObject* LogOwner, const FMassEntityHandle Entity, FMassSmartObjectRequestResultFragment& Result)
	{
		if (Result.Candidates.NumSlots > 0)
		{
			TArrayView<FSmartObjectCandidateSlot> View = MakeArrayView(Result.Candidates.Slots.GetData(), Result.Candidates.NumSlots);
			Algo::Sort(View, [](const FSmartObjectCandidateSlot& LHS, const FSmartObjectCandidateSlot& RHS) { return LHS.Cost < RHS.Cost; });
		}
		Result.bProcessed = true;

#if WITH_MASSGAMEPLAY_DEBUG
		UE_VLOG(LogOwner, LogSmartObject, VeryVerbose, TEXT("[%s] search completed: found %d"), *Entity.DebugGetDescription(), Result.Candidates.NumSlots);
#endif // WITH_MASSGAMEPLAY_DEBUG
	};

	// Process world location based requests
	WorldRequestQuery.ForEachEntityChunk(ExecutionContext, [this, &Filter, &EntitiesToSignal, &BeginRequestProcessing, &EndRequestProcessing](FMassExecutionContext& Context)
	{
		const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetSubsystemChecked<USmartObjectSubsystem>();

		const int32 NumEntities = Context.GetNumEntities();
		EntitiesToSignal.Reserve(EntitiesToSignal.Num() + NumEntities);

		const TConstArrayView<FMassSmartObjectWorldLocationRequestFragment> RequestList = Context.GetFragmentView<FMassSmartObjectWorldLocationRequestFragment>();
		const TConstArrayView<UE::Mass::SmartObject::FMRUSlotsFragment> MRUSlotsList = Context.GetFragmentView<UE::Mass::SmartObject::FMRUSlotsFragment>();
		const TArrayView<FMassSmartObjectRequestResultFragment> ResultList = Context.GetMutableFragmentView<FMassSmartObjectRequestResultFragment>();

		TArray<FSmartObjectRequestResult> QueryResults;
		TArray<FSmartObjectCandidateSlot> SortedCandidateSlots;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassSmartObjectWorldLocationRequestFragment& RequestFragment = RequestList[EntityIt];
			FMassSmartObjectRequestResultFragment& Result = ResultList[EntityIt];
			
			EntitiesToSignal.Add(RequestFragment.RequestingEntity);

			const FVector& SearchOrigin = RequestFragment.SearchOrigin;
			const FBox& SearchBounds = FBox::BuildAABB(SearchOrigin, FVector(SearchExtents));

			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			bool bDisplayDebug = false;
			FColor DebugColor(FColor::White);

#if WITH_MASSGAMEPLAY_DEBUG
			bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity, &DebugColor);
#endif // WITH_MASSGAMEPLAY_DEBUG

			BeginRequestProcessing(Entity, Context, Result);
			ON_SCOPE_EXIT{ EndRequestProcessing(&SmartObjectSubsystem, Entity, Result);	};

			Filter.UserTags = RequestFragment.UserTags;
			Filter.ActivityRequirements = RequestFragment.ActivityRequirements;
			
			QueryResults.Reset();
			SmartObjectSubsystem.FindSmartObjects(FSmartObjectRequest(SearchBounds, Filter), QueryResults);

			SortedCandidateSlots.Reset(QueryResults.Num());

			// Gather MRU slots to ignore if the settings are set to use them
			TArray<FSmartObjectSlotHandle> SlotsToIgnore;
			if (MRUSlotsList.Num() > 0)
			{
				MRUSlotsList[EntityIt].Slots.AppendTo(SlotsToIgnore);

				const UE::Mass::SmartObject::FMRUSlots& Slots = MRUSlotsList[EntityIt].Slots;
				SlotsToIgnore.Reserve(Slots.NumSlots);

				for (int32 SlotIndex = Slots.NumSlots - 1; SlotIndex >= 0; SlotIndex--)
				{
					SlotsToIgnore.Push(Slots.Slots[SlotIndex].Slot);
				}
			}

			for (const FSmartObjectRequestResult& QueryResult : QueryResults)
			{
				if (SlotsToIgnore.Contains(QueryResult.SlotHandle))
				{
					continue;
				}

				const FVector SlotLocation = SmartObjectSubsystem.GetSlotLocation(QueryResult.SlotHandle).GetValue();
				SortedCandidateSlots.Emplace(QueryResult, UE_REAL_TO_FLOAT_CLAMPED_MAX(FVector::DistSquared(SearchOrigin, SlotLocation)));

#if WITH_MASSGAMEPLAY_DEBUG
				if (bDisplayDebug)
				{
					constexpr uint16 DebugRadius = 10;
					UE_VLOG_LOCATION(&SmartObjectSubsystem, LogSmartObject, Display, SlotLocation, DebugRadius, DebugColor, TEXT("%s"), *LexToString(QueryResult.SmartObjectHandle));
					UE_VLOG_SEGMENT(&SmartObjectSubsystem, LogSmartObject, Display, SearchOrigin, SlotLocation, DebugColor, TEXT(""));
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
			}
			SortedCandidateSlots.Sort([](const FSmartObjectCandidateSlot& First, const FSmartObjectCandidateSlot& Second){ return First.Cost < Second.Cost; });

			Result.Candidates.NumSlots = IntCastChecked<uint8>(FMath::Min(FMassSmartObjectCandidateSlots::MaxNumCandidates, (uint32)SortedCandidateSlots.Num()));
			for (int ResultIndex = 0; ResultIndex < Result.Candidates.NumSlots; ResultIndex++)
			{
				Result.Candidates.Slots[ResultIndex] = SortedCandidateSlots[ResultIndex];
			}
		}
	});

	// Process lane based requests
	const FZoneGraphTag SmartObjectTag = GetDefault<UMassSmartObjectSettings>()->SmartObjectTag;
	USmartObjectZoneAnnotations* Annotations = Cast<USmartObjectZoneAnnotations>(AnnotationSubsystem.GetFirstAnnotationForTag(SmartObjectTag));

	LaneRequestQuery.ForEachEntityChunk(ExecutionContext,
		[&AnnotationSubsystem, Annotations, &Filter, SmartObjectTag, &EntitiesToSignal, &BeginRequestProcessing, &EndRequestProcessing](FMassExecutionContext& Context)
		{
#if WITH_MASSGAMEPLAY_DEBUG
			const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetSubsystemChecked<UZoneGraphSubsystem>();
#endif // WITH_MASSGAMEPLAY_DEBUG
			const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetSubsystemChecked<USmartObjectSubsystem>();

			const int32 NumEntities = Context.GetNumEntities();
			EntitiesToSignal.Reserve(EntitiesToSignal.Num() + NumEntities);

			TConstArrayView<FMassSmartObjectLaneLocationRequestFragment> RequestList = Context.GetFragmentView<FMassSmartObjectLaneLocationRequestFragment>();
			TArrayView<FMassSmartObjectRequestResultFragment> ResultList = Context.GetMutableFragmentView<FMassSmartObjectRequestResultFragment>();
			const TConstArrayView<UE::Mass::SmartObject::FMRUSlotsFragment> MRUSlotsList = Context.GetFragmentView<UE::Mass::SmartObject::FMRUSlotsFragment>();

			// Cache latest used data since request are most of the time on the same zone graph
			FZoneGraphDataHandle LastUsedDataHandle;
			const FSmartObjectAnnotationData* GraphData = nullptr;

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassSmartObjectLaneLocationRequestFragment& RequestFragment = RequestList[EntityIt];
				FMassSmartObjectRequestResultFragment& Result = ResultList[EntityIt];
				
				EntitiesToSignal.Add(RequestFragment.RequestingEntity);

				const FZoneGraphCompactLaneLocation RequestLocation = RequestFragment.CompactLaneLocation;
				const FZoneGraphLaneHandle RequestLaneHandle = RequestLocation.LaneHandle;

				const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
				bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
				FColor DebugColor(FColor::White);
				bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity, &DebugColor);
#endif // WITH_MASSGAMEPLAY_DEBUG

				BeginRequestProcessing(Entity, Context, Result);
				ON_SCOPE_EXIT{ EndRequestProcessing(&SmartObjectSubsystem, Entity, Result); };

				if (!ensureMsgf(RequestLaneHandle.IsValid(), TEXT("Requesting smart objects using an invalid handle")))
				{
					continue;
				}

				if (Annotations == nullptr)
				{
					UE_VLOG(&SmartObjectSubsystem, LogSmartObject, Warning, TEXT("%d lane location based requests failed since SmartObject annotations are not available"), NumEntities);
					return;
				}

				// Fetch smart object data associated to the current graph if different than last used one
				if (LastUsedDataHandle != RequestLaneHandle.DataHandle)
				{
					LastUsedDataHandle = RequestLaneHandle.DataHandle;
					GraphData = Annotations->GetAnnotationData(RequestLaneHandle.DataHandle);
				}

				if (GraphData == nullptr)
				{
					continue;
				}

				// Fetch current annotations for the specified lane and look for the smart object tag
				const FZoneGraphTagMask LaneMask = AnnotationSubsystem.GetAnnotationTags(RequestLaneHandle);
				if (!LaneMask.Contains(SmartObjectTag))
				{
					continue;
				}

				const FSmartObjectLaneLocationIndices* SmartObjectList = GraphData->LaneToLaneLocationIndicesLookup.Find(RequestLaneHandle.Index);
				if (SmartObjectList == nullptr || !ensureMsgf(SmartObjectList->SmartObjectLaneLocationIndices.Num() > 0, TEXT("Lookup table should only contains lanes with one or more associated object(s).")))
				{
					continue;
				}

				Filter.UserTags = RequestFragment.UserTags;
				Filter.ActivityRequirements = RequestFragment.ActivityRequirements;

				for (const int32 Index : SmartObjectList->SmartObjectLaneLocationIndices)
				{
					// Find entry point using FindChecked since all smart objects added to LaneToSmartObjects lookup table
					// were also added to the entry point lookup table
					check(GraphData->SmartObjectLaneLocations.IsValidIndex(Index));
					const FSmartObjectLaneLocation& EntryPoint = GraphData->SmartObjectLaneLocations[Index];
					const FSmartObjectHandle Handle = EntryPoint.ObjectHandle;

					float Cost = 0.f;
					if (ensureMsgf(EntryPoint.LaneIndex == RequestLocation.LaneHandle.Index, TEXT("Must be on same lane to be able to use distance along lane.")))
					{
						// Only consider object ahead
						const float DistAhead = EntryPoint.DistanceAlongLane - RequestLocation.DistanceAlongLane;
						if (DistAhead < 0)
						{
							continue;
						}
						Cost = DistAhead;
					}

					// Make sure that we can use a slot in that object (availability with supported definitions, etc.)
					TArray<FSmartObjectSlotHandle> SlotHandles;
					SmartObjectSubsystem.FindSlots(Handle, Filter, SlotHandles, FConstStructView::Make(FSmartObjectMassEntityUserData(Entity)));

					if (SlotHandles.IsEmpty())
					{
						continue;
					}

					// Gather MRU slots to ignore if the settings are set to use them
					TArray<FSmartObjectSlotHandle> SlotsToIgnore;
					if (MRUSlotsList.Num() > 0)
					{
						MRUSlotsList[EntityIt].Slots.AppendTo(SlotsToIgnore);
					}

					for (FSmartObjectSlotHandle SlotHandle : SlotHandles)
					{
						if (SlotsToIgnore.Contains(SlotHandle))
						{
							continue;
						}

						Result.Candidates.Slots[Result.Candidates.NumSlots++] = FSmartObjectCandidateSlot(FSmartObjectRequestResult(Handle, SlotHandle), Cost);

#if WITH_MASSGAMEPLAY_DEBUG
						if (bDisplayDebug)
						{
							FZoneGraphLaneLocation RequestLaneLocation, EntryPointLaneLocation;
							ZoneGraphSubsystem.CalculateLocationAlongLane(RequestLaneHandle, RequestLocation.DistanceAlongLane, RequestLaneLocation);
							ZoneGraphSubsystem.CalculateLocationAlongLane(RequestLaneHandle, EntryPoint.DistanceAlongLane, EntryPointLaneLocation);

							constexpr uint16 DebugRadius = 10;
							FVector SlotLocation = SmartObjectSubsystem.GetSlotLocation(SlotHandle).Get(EntryPointLaneLocation.Position);
							UE_VLOG_LOCATION(&SmartObjectSubsystem, LogSmartObject, Display, SlotLocation, DebugRadius, DebugColor, TEXT("%s"), *LexToString(SlotHandle));
							UE_VLOG_SEGMENT(&SmartObjectSubsystem, LogSmartObject, Display, SlotLocation, EntryPointLaneLocation.Position, DebugColor, TEXT(""));
							UE_VLOG_SEGMENT(&SmartObjectSubsystem, LogSmartObject, Display, RequestLaneLocation.Position, EntryPointLaneLocation.Position, DebugColor, TEXT(""));
						}
#endif // WITH_MASSGAMEPLAY_DEBUG
						if (Result.Candidates.NumSlots == FMassSmartObjectCandidateSlots::MaxNumCandidates)
						{
							break;
						}
					}

					if (Result.Candidates.NumSlots == FMassSmartObjectCandidateSlots::MaxNumCandidates)
					{
						break;
					}
				}
			}
		});

	// Signal entities that their search results are ready
	if (EntitiesToSignal.Num())
	{
		SignalSubsystem.SignalEntities(UE::Mass::Signals::SmartObjectCandidatesReady, EntitiesToSignal);
	}
}

//----------------------------------------------------------------------//
// UMassSmartObjectTimedBehaviorProcessor
//----------------------------------------------------------------------//
void UMassSmartObjectTimedBehaviorProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassSmartObjectUserFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassSmartObjectTimedBehaviorFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadOnly);

	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
}

UMassSmartObjectTimedBehaviorProcessor::UMassSmartObjectTimedBehaviorProcessor()
	: EntityQuery(*this)
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
}

void UMassSmartObjectTimedBehaviorProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TArray<FMassEntityHandle> ToRelease;

	QUICK_SCOPE_CYCLE_COUNTER(UMassProcessor_SmartObjectTestBehavior_Run);

	EntityQuery.ForEachEntityChunk(Context, [this, &ToRelease](FMassExecutionContext& Context)
	{
		const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetSubsystemChecked<USmartObjectSubsystem>();

		const TArrayView<FMassSmartObjectUserFragment> UserList = Context.GetMutableFragmentView<FMassSmartObjectUserFragment>();
		const TArrayView<FMassSmartObjectTimedBehaviorFragment> TimedBehaviorFragments = Context.GetMutableFragmentView<FMassSmartObjectTimedBehaviorFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassSmartObjectUserFragment& SOUser = UserList[EntityIt];
			FMassSmartObjectTimedBehaviorFragment& TimedBehaviorFragment = TimedBehaviorFragments[EntityIt];
			ensureMsgf(SOUser.InteractionStatus == EMassSmartObjectInteractionStatus::InProgress, TEXT("TimedBehavior fragment should only be present for in-progress interactions: %s"), *Context.GetEntity(EntityIt).DebugGetDescription());

			const float DT = Context.GetDeltaTimeSeconds();
			float& UseTime = TimedBehaviorFragment.UseTime;
			UseTime = FMath::Max(UseTime - DT, 0.0f);
			const bool bMustRelease = UseTime <= 0.f;

#if WITH_MASSGAMEPLAY_DEBUG
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			FColor DebugColor(FColor::White);
			const bool bIsDebuggingEntity = UE::Mass::Debug::IsDebuggingEntity(Entity, &DebugColor);
			if (bIsDebuggingEntity)
			{
				UE_CVLOG(bMustRelease, &SmartObjectSubsystem, LogSmartObject, Log, TEXT("[%s] stops using [%s]"), *Entity.DebugGetDescription(), *LexToString(SOUser.InteractionHandle));
				UE_CVLOG(!bMustRelease, &SmartObjectSubsystem, LogSmartObject, VeryVerbose, TEXT("[%s] using [%s]. Time left: %.1f"), *Entity.DebugGetDescription(), *LexToString(SOUser.InteractionHandle), UseTime);

				const TOptional<FTransform> Transform = SmartObjectSubsystem.GetSlotTransform(SOUser.InteractionHandle);
				if (Transform.IsSet())
				{
					constexpr float Radius = 40.f;
					const FVector HalfHeightOffset(0.f, 0.f, 100.f);
					const FVector Pos = Transform.GetValue().GetLocation();
					const FVector Dir = Transform.GetValue().GetRotation().GetForwardVector();
					UE_VLOG_CYLINDER(&SmartObjectSubsystem, LogSmartObject, Display, Pos - HalfHeightOffset, Pos + HalfHeightOffset, Radius, DebugColor, TEXT(""));
					UE_VLOG_ARROW(&SmartObjectSubsystem, LogSmartObject, Display, Pos, Pos + Dir * 2.0f * Radius, DebugColor, TEXT(""));
				}
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			if (bMustRelease)
			{
				SOUser.InteractionStatus = EMassSmartObjectInteractionStatus::BehaviorCompleted;
				ToRelease.Add(Context.GetEntity(EntityIt));
			}
		}
	});

	if (ToRelease.Num())
	{
		UMassSignalSubsystem& SignalSubsystem = Context.GetMutableSubsystemChecked<UMassSignalSubsystem>();
		SignalSubsystem.SignalEntities(UE::Mass::Signals::SmartObjectInteractionDone, ToRelease);
	}
}


//----------------------------------------------------------------------//
//  UMassSmartObjectUserFragmentDeinitializer
//----------------------------------------------------------------------//
UMassSmartObjectUserFragmentDeinitializer::UMassSmartObjectUserFragmentDeinitializer()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassSmartObjectUserFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void UMassSmartObjectUserFragmentDeinitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassSmartObjectUserFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassSmartObjectUserFragmentDeinitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			USmartObjectSubsystem& SmartObjectSubsystem = Context.GetMutableSubsystemChecked<USmartObjectSubsystem>();
			const TArrayView<FMassSmartObjectUserFragment> SmartObjectUserFragments = Context.GetMutableFragmentView<FMassSmartObjectUserFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				SmartObjectSubsystem.UnregisterSlotInvalidationCallback(SmartObjectUserFragments[EntityIt].InteractionHandle);
			}
		});
}

namespace UE::Mass::SmartObject
{

//----------------------------------------------------------------------//
// UMRUSlotsProcessor
//----------------------------------------------------------------------//
void UMRUSlotsProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassSmartObjectUserFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMRUSlotsFragment>(EMassFragmentAccess::ReadWrite);
}

UMRUSlotsProcessor::UMRUSlotsProcessor()
	: EntityQuery(*this)
{
	ExecutionOrder.ExecuteInGroup = ProcessorGroupNames::SyncWorldToMass;
}

void UMRUSlotsProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UMassProcessor_SmartObjecMRUSlots);

	const UMassSmartObjectSettings* Settings = GetDefault<UMassSmartObjectSettings>();
	if (Settings && Settings->bUseCooldownForMRUSlots == false)
	{
		return;
	}

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const float DT = Context.GetDeltaTimeSeconds();
		const TArrayView<FMRUSlotsFragment> MRUsList = Context.GetMutableFragmentView<FMRUSlotsFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMRUSlotsFragment& MRUs = MRUsList[EntityIt];
			FMRUSlots UpdatedSlots;
			for (int32 Index = MRUs.Slots.NumSlots - 1; Index >= 0; Index--)
			{
				float& Time = MRUs.Slots.Slots[Index].RemainingTime;
				Time = FMath::Max(Time - DT, 0.0f);
				if (Time > 0.f)
				{
					UpdatedSlots.Slots[UpdatedSlots.NumSlots] = MRUs.Slots.Slots[Index];
					UpdatedSlots.NumSlots++;
				}
			}

			MRUs.Slots = MoveTemp(UpdatedSlots);
		}
	});
}

} // UE::Mass::SmartObject