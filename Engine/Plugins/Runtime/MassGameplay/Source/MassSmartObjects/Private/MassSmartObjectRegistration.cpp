// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectRegistration.h"

#include "MassActorSubsystem.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "SmartObjectSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectRegistration)

namespace UE::Mass::Signals
{
	const FName SmartObjectActivationChanged = FName(TEXT("SmartObjectActivated"));
}

//----------------------------------------------------------------------//
// UMassSmartObjectInitializerBase
//----------------------------------------------------------------------//
UMassSmartObjectInitializerBase::UMassSmartObjectInitializerBase()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedOperations = EMassObservedOperationFlags::Add;
}

void UMassSmartObjectInitializerBase::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassInActiveSmartObjectsRangeTag>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FMassActorInstanceFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FSmartObjectRegistrationFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassSmartObjectInitializerBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	UMassSignalSubsystem& SignalSubsystem = ExecutionContext.GetMutableSubsystemChecked<UMassSignalSubsystem>();
	TArray<FMassEntityHandle> EntitiesToSignal;

	EntityQuery.ForEachEntityChunk(ExecutionContext,[&EntitiesToSignal](FMassExecutionContext& Context)
	{
		const TConstArrayView<FMassActorInstanceFragment> InstancedActorFragments = Context.GetFragmentView<FMassActorInstanceFragment>();
		const TArrayView<FSmartObjectRegistrationFragment> RegistrationFragments = Context.GetMutableFragmentView<FSmartObjectRegistrationFragment>();

		const int32 NumEntities = Context.GetNumEntities();
		EntitiesToSignal.Reserve(EntitiesToSignal.Num() + NumEntities);

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FSmartObjectRegistrationFragment& RegistrationFragment = RegistrationFragments[EntityIt];
			const FMassActorInstanceFragment& InstancedActorFragment = InstancedActorFragments[EntityIt];

			checkf(!RegistrationFragment.Handle.IsValid(), TEXT("Should create smartobject only once."))
			if (RegistrationFragment.Asset.Get() == nullptr)
			{
				continue;
			}

			if (!InstancedActorFragment.Handle.IsValid())
			{
				continue;
			}

			EntitiesToSignal.Add(Context.GetEntity(EntityIt));
		}
	});

	// Signal all entities inside the consolidated list
	if (EntitiesToSignal.Num())
	{
		SignalSubsystem.SignalEntities(Signal, EntitiesToSignal);
	}
}

//----------------------------------------------------------------------//
// UMassSmartObjectDeinitializerBase
//----------------------------------------------------------------------//
UMassSmartObjectDeinitializerBase::UMassSmartObjectDeinitializerBase()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedOperations = EMassObservedOperationFlags::Remove;
}

void UMassSmartObjectDeinitializerBase::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassSmartObjectDeinitializerBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	UMassSignalSubsystem& SignalSubsystem = ExecutionContext.GetMutableSubsystemChecked<UMassSignalSubsystem>();
	TArray<FMassEntityHandle> EntitiesToSignal;

	EntityQuery.ForEachEntityChunk(ExecutionContext, [&EntitiesToSignal](FMassExecutionContext& Context)
	{
		const int32 NumEntities = Context.GetNumEntities();
		EntitiesToSignal.Reserve(EntitiesToSignal.Num() + NumEntities);
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			EntitiesToSignal.Add(Context.GetEntity(EntityIt));
		}
	});
	
	// Signal all entities inside the consolidated list
	if (EntitiesToSignal.Num())
	{
		SignalSubsystem.SignalEntities(Signal, EntitiesToSignal);
	}
}

//----------------------------------------------------------------------//
// UMassActiveSmartObjectInitializer
//----------------------------------------------------------------------//
UMassActiveSmartObjectInitializer::UMassActiveSmartObjectInitializer()
{
	ObservedTypes.Add(FMassInActiveSmartObjectsRangeTag::StaticStruct());
	Signal = UE::Mass::Signals::SmartObjectActivationChanged;
}

//----------------------------------------------------------------------//
// UMassActiveSmartObjectDeinitializer
//----------------------------------------------------------------------//
UMassActiveSmartObjectDeinitializer::UMassActiveSmartObjectDeinitializer()
{
	ObservedTypes.Add(FMassInActiveSmartObjectsRangeTag::StaticStruct());
	Signal = UE::Mass::Signals::SmartObjectActivationChanged;
}

//----------------------------------------------------------------------//
// UMassActorInstanceHandleInitializer
//----------------------------------------------------------------------//
UMassActorInstanceHandleInitializer::UMassActorInstanceHandleInitializer()
{
	ObservedTypes.Add(FMassActorInstanceFragment::StaticStruct());
	Signal = UE::Mass::Signals::ActorInstanceHandleChanged;
}

//----------------------------------------------------------------------//
// UMassActorInstanceHandleDeinitializer
//----------------------------------------------------------------------//
UMassActorInstanceHandleDeinitializer::UMassActorInstanceHandleDeinitializer()
{
	ObservedTypes.Add(FMassActorInstanceFragment::StaticStruct());
	Signal = UE::Mass::Signals::ActorInstanceHandleChanged;
}

//-----------------------------------------------------------------------------
// UMassActiveSmartObjectSignalProcessor
//-----------------------------------------------------------------------------
UMassActiveSmartObjectSignalProcessor::UMassActiveSmartObjectSignalProcessor()
	: InsideSmartObjectActiveRangeQuery(*this)
	, OutsideSmartObjectActiveRangeQuery(*this)
{
	// USmartObjectSubsystem CreateSmartObject/DestroySmartObject methods called
	// from this process are not safe to call from other threads.
	bRequiresGameThreadExecution = true;
}

void UMassActiveSmartObjectSignalProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	InsideSmartObjectActiveRangeQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadWrite);
	InsideSmartObjectActiveRangeQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	InsideSmartObjectActiveRangeQuery.AddRequirement<FMassActorInstanceFragment>(EMassFragmentAccess::ReadOnly);
	InsideSmartObjectActiveRangeQuery.AddRequirement<FSmartObjectRegistrationFragment>(EMassFragmentAccess::ReadWrite);
	InsideSmartObjectActiveRangeQuery.AddTagRequirement<FMassInActiveSmartObjectsRangeTag>(EMassFragmentPresence::All);

	OutsideSmartObjectActiveRangeQuery.AddSubsystemRequirement<USmartObjectSubsystem>(EMassFragmentAccess::ReadWrite);
	OutsideSmartObjectActiveRangeQuery.AddRequirement<FSmartObjectRegistrationFragment>(EMassFragmentAccess::ReadWrite);
	OutsideSmartObjectActiveRangeQuery.AddTagRequirement<FMassInActiveSmartObjectsRangeTag>(EMassFragmentPresence::None);
}

void UMassActiveSmartObjectSignalProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::ActorInstanceHandleChanged);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SmartObjectActivationChanged);
}

void UMassActiveSmartObjectSignalProcessor::SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup&)
{
	// Process entities in active range
	InsideSmartObjectActiveRangeQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		USmartObjectSubsystem& Subsystem = Context.GetMutableSubsystemChecked<USmartObjectSubsystem>();
		const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();
		const TConstArrayView<FMassActorInstanceFragment> InstancedActorFragments = Context.GetFragmentView<FMassActorInstanceFragment>();
		const TArrayView<FSmartObjectRegistrationFragment> RegistrationFragments = Context.GetMutableFragmentView<FSmartObjectRegistrationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FSmartObjectRegistrationFragment& RegistrationFragment = RegistrationFragments[EntityIt];
			const FMassActorInstanceFragment& InstancedActorFragment = InstancedActorFragments[EntityIt];
			const FTransformFragment& TransformFragment = TransformFragments[EntityIt];

			// Creation
			if (!RegistrationFragment.Handle.IsValid() && InstancedActorFragment.Handle.IsValid())
			{
				if (const USmartObjectDefinition* Definition = RegistrationFragment.Asset.Get())
				{
					FSmartObjectActorOwnerData InstancedActorOwnerData(InstancedActorFragment.Handle);
					RegistrationFragment.Handle = Subsystem.CreateSmartObject(
						*Definition,
						TransformFragment.GetTransform(),
						FConstStructView::Make(InstancedActorOwnerData));
				}
			}
			// Destruction
			else if (RegistrationFragment.Handle.IsValid() && !InstancedActorFragment.Handle.IsValid())
			{
				Subsystem.DestroySmartObject(RegistrationFragment.Handle);
				RegistrationFragment.Handle.Invalidate();
			}
		}
	});

	// Process out of active range entities
	OutsideSmartObjectActiveRangeQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		USmartObjectSubsystem& Subsystem = Context.GetMutableSubsystemChecked<USmartObjectSubsystem>();
		const TArrayView<FSmartObjectRegistrationFragment> RegistrationFragments = Context.GetMutableFragmentView<FSmartObjectRegistrationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FSmartObjectRegistrationFragment& RegistrationFragment = RegistrationFragments[EntityIt];

			if (RegistrationFragment.Handle.IsValid())
			{
				Subsystem.DestroySmartObject(RegistrationFragment.Handle);
				RegistrationFragment.Handle.Invalidate();
			}
		}
	});
}
