// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectSettings)

#if WITH_EDITOR
void UMassSmartObjectSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty != nullptr && Property != nullptr)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMassSmartObjectSettings, SmartObjectTag))
		{
			OnAnnotationSettingsChanged.Broadcast();
		}
	}
}
#endif // WITH_EDITOR
