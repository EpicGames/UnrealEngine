// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectTypes.h"
#include "MassSmartObjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectTypes)

namespace UE::Mass::SmartObject
{
void FMRUSlots::Push(const FSmartObjectSlotHandle& ClaimedSlot)
{
	int32 InsertIndex = NumSlots;
	if (NumSlots == MaxNumSlots)
	{
		InsertIndex = NumSlots - 1;
		for (int32 Index = 0; Index < InsertIndex; ++Index)
		{
			Slots[Index] = Slots[Index + 1];
		}
	}
	else
	{
		NumSlots++;
	}

	const UMassSmartObjectSettings* Settings = GetDefault<UMassSmartObjectSettings>();
	Slots[InsertIndex].Slot = ClaimedSlot;
	Slots[InsertIndex].RemainingTime = (Settings && Settings->bUseCooldownForMRUSlots)
		? Settings->MRUSlotsCooldown
		: FMRUSlot::NoCoolDown;
}

void FMRUSlots::AppendTo(TArray<FSmartObjectSlotHandle>& OutSlots) const
{
	OutSlots.Reserve(OutSlots.Num() + NumSlots);

	for (int32 SlotIndex = NumSlots - 1; SlotIndex >= 0; SlotIndex--)
	{
		OutSlots.Push(Slots[SlotIndex].Slot);
	}
}
} // UE::Mass::SmartObject
