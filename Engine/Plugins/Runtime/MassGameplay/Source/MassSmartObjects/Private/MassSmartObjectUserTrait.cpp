// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSmartObjectUserTrait.h"

#include "MassSmartObjectFragments.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmartObjectUserTrait)

void UMassSmartObjectUserTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassSmartObjectUserFragment SmartObjectUser;
	SmartObjectUser.UserTags = UserTags;
	
	BuildContext.AddFragment(FConstStructView::Make(SmartObjectUser));
}
