// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmartObjectZoneAnnotations.h"
#include "MassSmartObjectSettings.h"
#include "SmartObjectComponent.h"
#include "SmartObjectSubsystem.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSubsystem.h"
#include "GameFramework/Character.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmartObjectZoneAnnotations)

void USmartObjectZoneAnnotations::PostSubsystemsInitialized()
{
	SmartObjectSubsystem = UWorld::GetSubsystem<USmartObjectSubsystem>(GetWorld());

#if WITH_EDITOR
	// Monitor collection changes to rebuild our lookup data
	if (SmartObjectSubsystem != nullptr)
	{
		OnMainCollectionChangedHandle = SmartObjectSubsystem->OnMainCollectionChanged.AddLambda([this]()
		{
			const UWorld* World = GetWorld();
			if (World != nullptr && !World->IsGameWorld())
			{
				RebuildForAllGraphs();
			}
		});

		OnMainCollectionDirtiedHandle = SmartObjectSubsystem->OnMainCollectionDirtied.AddLambda([this]()
		{
			const UWorld* World = GetWorld();
			if (World != nullptr && !World->IsGameWorld())
			{
				// Simply queue a rebuild request until we serialize the annotations.
				// This is to avoid large amount of rebuild triggered from SmartObjectComponents being constantly
				// unregistered/registered when modifying their properties (e.g. dragging the actor(s) in the level)
				bRebuildAllGraphsRequested = true;
				MarkPackageDirty();
			}
		});
	}

	const UMassSmartObjectSettings* MassSmartObjectSettings = GetDefault<UMassSmartObjectSettings>();
	BehaviorTag = MassSmartObjectSettings->SmartObjectTag;

	// Track density settings changes
	OnAnnotationSettingsChangedHandle = MassSmartObjectSettings->OnAnnotationSettingsChanged.AddLambda([this]()
	{
		BehaviorTag = GetDefault<UMassSmartObjectSettings>()->SmartObjectTag;
	});

	// Monitor zone graph changes to rebuild our lookup data
	OnGraphDataChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.AddLambda([this](const FZoneGraphBuildData& BuildData)
	{
		RebuildForAllGraphs();
	});
#endif // WITH_EDITOR

	// Update our cached members before calling base class since it might call
	// PostZoneGraphDataAdded and we need to be all set.
	Super::PostSubsystemsInitialized();
}

void USmartObjectZoneAnnotations::PostZoneGraphDataAdded(const AZoneGraphData& ZoneGraphData)
{
	const FZoneGraphStorage& Storage = ZoneGraphData.GetStorage();
	const int32 Index = Storage.DataHandle.Index;

#if WITH_EDITOR
	if (Index >= SmartObjectAnnotationDataArray.Num())
	{
		SmartObjectAnnotationDataArray.SetNum(Index + 1);
	}
#endif

	checkf(SmartObjectAnnotationDataArray.IsValidIndex(Index), TEXT("In Editor should always resize when necessary and runtime should always have valid precomputed data."));
	FSmartObjectAnnotationData& Data = SmartObjectAnnotationDataArray[Index];
	if (!Data.IsValid())
	{
		Data.DataHandle = Storage.DataHandle;
	}

#if WITH_EDITOR
	// We don't rebuild for runtime world, we use precomputed data
	if (!ZoneGraphData.GetWorld()->IsGameWorld())
	{
		RebuildForSingleGraph(Data, Storage);
	}
#endif // WITH_EDITOR

	Data.bInitialTaggingCompleted = false;
}

void USmartObjectZoneAnnotations::PreZoneGraphDataRemoved(const AZoneGraphData& ZoneGraphData)
{
	const FZoneGraphStorage& Storage = ZoneGraphData.GetStorage();
	const int32 Index = Storage.DataHandle.Index;

	if (!SmartObjectAnnotationDataArray.IsValidIndex(Index))
	{
		return;
	}

	FSmartObjectAnnotationData& Data = SmartObjectAnnotationDataArray[Index];

	// We use precomputed data for runtime so we only mark it as no longer used
	if (ZoneGraphData.GetWorld()->IsGameWorld())
	{
		Data.DataHandle = {};
	}
	else
	{
		Data.Reset();
	}
}

FZoneGraphTagMask USmartObjectZoneAnnotations::GetAnnotationTags() const
{
	return FZoneGraphTagMask(BehaviorTag);
}

const FSmartObjectAnnotationData* USmartObjectZoneAnnotations::GetAnnotationData(const FZoneGraphDataHandle DataHandle) const
{
	const int32 Index = DataHandle.Index;
	if (!SmartObjectAnnotationDataArray.IsValidIndex(Index))
	{
		return nullptr;
	}

	return &SmartObjectAnnotationDataArray[Index];
}

TOptional<FSmartObjectLaneLocation> USmartObjectZoneAnnotations::GetSmartObjectLaneLocation(const FZoneGraphDataHandle DataHandle, const FSmartObjectHandle SmartObjectHandle) const
{
	TOptional<FSmartObjectLaneLocation> SmartObjectLaneLocation;
	if (const FSmartObjectAnnotationData* AnnotationData = GetAnnotationData(DataHandle))
	{
		const int32 Index = AnnotationData->SmartObjectToLaneLocationIndexLookup.FindChecked(SmartObjectHandle);
		if (AnnotationData->SmartObjectLaneLocations.IsValidIndex(Index))
		{
			SmartObjectLaneLocation = AnnotationData->SmartObjectLaneLocations[Index];
		}
	}
	return SmartObjectLaneLocation;
}

void USmartObjectZoneAnnotations::ApplyComponentInstanceData(FSmartObjectZoneAnnotationsInstanceData* InstanceData)
{
	check(InstanceData);
	SmartObjectAnnotationDataArray = InstanceData->SmartObjectAnnotations;
}

void USmartObjectZoneAnnotations::TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& BehaviorTagContainer)
{
	if (!BehaviorTag.IsValid())
	{
		return;
	}

	for (FSmartObjectAnnotationData& Data : SmartObjectAnnotationDataArray)
	{
		if (Data.bInitialTaggingCompleted || !Data.IsValid() || Data.SmartObjectToLaneLocationIndexLookup.IsEmpty())
		{
			continue;
		}

		// Apply tags
		TArrayView<FZoneGraphTagMask> LaneTags = BehaviorTagContainer.GetMutableAnnotationTagsForData(Data.DataHandle);
		for (const int32 LaneIndex : Data.AffectedLanes)
		{
			LaneTags[LaneIndex].Add(BehaviorTag);
		}

		Data.bInitialTaggingCompleted = true;
	}

#if UE_ENABLE_DEBUG_DRAWING
	MarkRenderStateDirty();
#endif // UE_ENABLE_DEBUG_DRAWING
}

TStructOnScope<FActorComponentInstanceData> USmartObjectZoneAnnotations::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FSmartObjectZoneAnnotationsInstanceData>(this);
}

#if UE_ENABLE_DEBUG_DRAWING
void USmartObjectZoneAnnotations::DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy)
{
	UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	if (ZoneGraph == nullptr || SmartObjectSubsystem == nullptr)
	{
		return;
	}

	const FSmartObjectContainer& SmartObjectContainer = SmartObjectSubsystem->GetSmartObjectContainer();

	for (FSmartObjectAnnotationData& AnnotationData : SmartObjectAnnotationDataArray)
	{
		const FZoneGraphStorage* ZoneStorage = AnnotationData.DataHandle.IsValid() ? ZoneGraph->GetZoneGraphStorage(AnnotationData.DataHandle) : nullptr;
		if (ZoneStorage == nullptr)
		{
			continue;
		}

		for (const FSmartObjectCollectionEntry& Entry : SmartObjectContainer.GetEntries())
		{
			int32* Index = AnnotationData.SmartObjectToLaneLocationIndexLookup.Find(Entry.GetHandle());
			if (Index == nullptr)
			{
				continue;
			}
			const FSmartObjectLaneLocation& SOLaneLocation = AnnotationData.SmartObjectLaneLocations[*Index];

			const FVector& ObjectLocation = Entry.GetTransform().GetLocation();
			FZoneGraphLaneLocation EntryPointLocation;
			UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneStorage, SOLaneLocation.LaneIndex, SOLaneLocation.DistanceAlongLane, EntryPointLocation);
			const FColor Color = FColor::Silver;
			constexpr float SphereRadius = 25.f;
			DebugProxy->Spheres.Emplace(SphereRadius, EntryPointLocation.Position, Color);
			DebugProxy->Spheres.Emplace(SphereRadius, ObjectLocation, Color);
			DebugProxy->DashedLines.Emplace(ObjectLocation, EntryPointLocation.Position, Color, /*dash size*/10.f);
		}
	}
}
#endif // UE_ENABLE_DEBUG_DRAWING

#if WITH_EDITORONLY_DATA
void USmartObjectZoneAnnotations::Serialize(FArchive& Ar)
{
	if (bRebuildAllGraphsRequested
		&& Ar.IsSaving()
		&& Ar.IsPersistent()	// saving archive for persistent storage (package)
		&& !Ar.IsTransacting()	// do not rebuild for transactions (i.e. undo/redo)
		)
	{
		RebuildForAllGraphs();
	}

	Super::Serialize(Ar);
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void USmartObjectZoneAnnotations::OnUnregister()
{
	if (SmartObjectSubsystem != nullptr)
	{
		SmartObjectSubsystem->OnMainCollectionChanged.Remove(OnMainCollectionChangedHandle);
		OnMainCollectionChangedHandle.Reset();

		SmartObjectSubsystem->OnMainCollectionDirtied.Remove(OnMainCollectionDirtiedHandle);
		OnMainCollectionDirtiedHandle.Reset();
	}

	GetDefault<UMassSmartObjectSettings>()->OnAnnotationSettingsChanged.Remove(OnAnnotationSettingsChangedHandle);
	OnAnnotationSettingsChangedHandle.Reset();

	UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.Remove(OnGraphDataChangedHandle);
	OnGraphDataChangedHandle.Reset();

	Super::OnUnregister();
}

void USmartObjectZoneAnnotations::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(USmartObjectZoneAnnotations, AffectedLaneTags))
		{
			RebuildForAllGraphs();
		}
	}
}

void USmartObjectZoneAnnotations::RebuildForSingleGraph(FSmartObjectAnnotationData& Data, const FZoneGraphStorage& Storage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("ZoneGraphSmartObjectBehavior RebuildData")

	if (SmartObjectSubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogSmartObject, Error, TEXT("Attempting to rebuild data while SmartObjectSubsystem is not set. This indicates a problem in the initialization flow."));
		return;
	}

	if (!BehaviorTag.IsValid())
	{
		UE_VLOG_UELOG(this, LogSmartObject, Warning, TEXT("Attempting to rebuild data while BehaviorTag is invalid (e.g. not set in MassSmartObjectSettings)"));
		return;
	}
	
	const FSmartObjectContainer& SmartObjectContainer = SmartObjectSubsystem->GetSmartObjectContainer();

	const FVector SearchExtent(GetDefault<UMassSmartObjectSettings>()->SearchExtents);
	int32 NumAdded = 0;
	int32 NumDiscarded = 0;

	const int32 NumSO = SmartObjectContainer.GetEntries().Num();
	const int32 NumLanes = Storage.Lanes.Num();
	Data.SmartObjectLaneLocations.Empty(NumSO);
	Data.SmartObjectToLaneLocationIndexLookup.Empty(NumSO);
	Data.LaneToLaneLocationIndicesLookup.Empty(NumLanes);
	Data.AffectedLanes.Empty(NumLanes);

	for (const FSmartObjectCollectionEntry& Entry : SmartObjectContainer.GetEntries())
	{
		FSmartObjectHandle Handle = Entry.GetHandle();
		const FVector& ObjectLocation = Entry.GetTransform().GetLocation();
		const FBox QueryBounds(ObjectLocation - SearchExtent, ObjectLocation + SearchExtent);

		FZoneGraphLaneLocation LaneLocation;
		float DistanceSqr = 0.f;
		const bool bFound = UE::ZoneGraph::Query::FindNearestLane(Storage, QueryBounds, AffectedLaneTags, LaneLocation, DistanceSqr);
		if (bFound)
		{
			NumAdded++;

			const int32 LaneIndex = LaneLocation.LaneHandle.Index;

			const int32 SOLaneLocationIndex = Data.SmartObjectLaneLocations.Add(FSmartObjectLaneLocation(Handle, LaneIndex, LaneLocation.DistanceAlongLane));
			Data.SmartObjectToLaneLocationIndexLookup.Add(Handle, SOLaneLocationIndex);
			Data.AffectedLanes.AddUnique(LaneIndex);
			Data.LaneToLaneLocationIndicesLookup.FindOrAdd(LaneIndex).SmartObjectLaneLocationIndices.Add(SOLaneLocationIndex);

			UE_VLOG_UELOG(this, LogSmartObject, Verbose, TEXT("Adding ZG annotation for SmartObject '%s' on lane '%s'"), *LexToString(Handle), *LaneLocation.LaneHandle.ToString());
			UE_VLOG_SEGMENT(this, LogSmartObject, Display, ObjectLocation, LaneLocation.Position, FColor::Green, TEXT(""));
			UE_VLOG_LOCATION(this, LogSmartObject, Display, ObjectLocation, 50 /*radius*/, FColor::Green, TEXT("%s"), *LexToString(Handle));
		}
		else
		{
			NumDiscarded++;
			UE_VLOG_LOCATION(this, LogSmartObject, Display, ObjectLocation, 75 /*radius*/, FColor::Red, TEXT("%s"), *LexToString(Handle));
		}
	}

	// Sort all entry points per distance on lane
	for (auto It(Data.LaneToLaneLocationIndicesLookup.CreateIterator()); It; ++It)
	{
		It.Value().SmartObjectLaneLocationIndices.Sort([Locations = Data.SmartObjectLaneLocations](const int32 FirstIndex, const int32 SecondIndex)
		{
			return Locations[FirstIndex].DistanceAlongLane < Locations[SecondIndex].DistanceAlongLane;
		});
	}

	Data.SmartObjectLaneLocations.Shrink();
	Data.SmartObjectToLaneLocationIndexLookup.Shrink();
	Data.AffectedLanes.Shrink();
	Data.LaneToLaneLocationIndicesLookup.Shrink();
	Data.bInitialTaggingCompleted = false;

	UE_VLOG_UELOG(this, LogSmartObject, Log, TEXT("Summary: %d entry points added, %d discarded%s."), NumAdded, NumDiscarded, NumDiscarded == 0 ? TEXT("") : TEXT(" (too far from any lane)"));
}

void USmartObjectZoneAnnotations::RebuildForAllGraphs()
{
	bRebuildAllGraphsRequested = false;

	UZoneGraphSubsystem* ZoneGraphSubsystem = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	if (!ZoneGraphSubsystem)
	{
		return;
	}

	for (const FRegisteredZoneGraphData& RegisteredZoneGraphData : ZoneGraphSubsystem->GetRegisteredZoneGraphData())
	{
		if (!RegisteredZoneGraphData.bInUse || RegisteredZoneGraphData.ZoneGraphData == nullptr)
		{
			continue;
		}

		const FZoneGraphStorage& Storage = RegisteredZoneGraphData.ZoneGraphData->GetStorage();
		const int32 Index = Storage.DataHandle.Index;

		if (SmartObjectAnnotationDataArray.IsValidIndex(Index))
		{
			FSmartObjectAnnotationData& AnnotationData = SmartObjectAnnotationDataArray[Index];
			AnnotationData.Reset();
			RebuildForSingleGraph(AnnotationData, Storage);
		}
	}
}
#endif // WITH_EDITOR
