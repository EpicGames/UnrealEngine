// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassEntityView.h"
#include "SmartObjectDefinition.h"
#include "MassSmartObjectBehaviorDefinition.generated.h"

#define UE_API MASSSMARTOBJECTS_API

struct FMassEntityManager;
class USmartObjectSubsystem;
struct FMassExecutionContext;
struct FTransformFragment;
struct FMassSmartObjectUserFragment;

/**
 * Struct to pass around the required set of information to activate a mass behavior definition on a given entity.
 * Context must be created on stack and not kept around since EntityView validity is not guaranteed.
 */
struct FMassBehaviorEntityContext
{
	FMassBehaviorEntityContext() = delete;

	FMassBehaviorEntityContext(FMassEntityView&& InEntityView, USmartObjectSubsystem& InSubsystem)
		: EntityView(MoveTemp(InEntityView))
		, SmartObjectSubsystem(InSubsystem)
	{}

	const FMassEntityView EntityView;
	USmartObjectSubsystem& SmartObjectSubsystem;
};

/**
 * Base class for MassAIBehavior definitions. This is the type of definitions that MassEntity queries will look for.
 * Definition subclass can parameterized its associated behavior by overriding method Activate.
 */
UCLASS(MinimalAPI, EditInlineNew)
class USmartObjectMassBehaviorDefinition : public USmartObjectBehaviorDefinition
{
	GENERATED_BODY()

public:
	/** This virtual method allows subclasses to configure the MassEntity based on their parameters (e.g. Add fragments) */
	UE_API virtual void Activate(FMassCommandBuffer& CommandBuffer, const FMassBehaviorEntityContext& EntityContext) const;

	/** This virtual method allows subclasses to update the MassEntity on interaction deactivation (e.g. Remove fragments) */
	UE_API virtual void Deactivate(FMassCommandBuffer& CommandBuffer, const FMassBehaviorEntityContext& EntityContext) const;

	/**
	 * Indicates the amount of time the Massentity
	 * will execute its behavior when reaching the smart object.
	 */
	UPROPERTY(EditDefaultsOnly, Category = SmartObject)
	float UseTime;
};

#undef UE_API
