// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassSmartObjectTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSmartObjectRequest.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SmartObjectRuntime.h"
#include "MassSmartObjectFragments.generated.h"

/** Fragment used by an entity to be able to interact with smart objects */
USTRUCT()
struct FMassSmartObjectUserFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Tags describing the smart object user. */
	UPROPERTY(Transient)
	FGameplayTagContainer UserTags;

	/** Claim handle for the currently active smart object interaction. */
	UPROPERTY(Transient)
	FSmartObjectClaimHandle InteractionHandle;

	/** Status of the current active smart object interaction. */
	UPROPERTY(Transient)
	EMassSmartObjectInteractionStatus InteractionStatus = EMassSmartObjectInteractionStatus::Unset;

	/**
	 * World time in seconds before which the user is considered in cooldown and
	 * won't look for new interactions (value of 0 indicates no cooldown).
	 */
	UPROPERTY(Transient)
	double InteractionCooldownEndTime = 0.;
};

template<>
struct TMassFragmentTraits<FMassSmartObjectUserFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/** Fragment used to process time based smartobject interactions */
USTRUCT()
struct FMassSmartObjectTimedBehaviorFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	float UseTime = 0.f;
};

namespace UE::Mass::SmartObject
{

/** Fragment used to track most recently used slots for a given smart object user */
USTRUCT()
struct FMRUSlotsFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FMRUSlots Slots;
};

} //  UE::Mass::SmartObject
