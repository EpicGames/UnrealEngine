// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSmartObjectRequest.h"
#include "MassSmartObjectTypes.h"

#define UE_API MASSSMARTOBJECTS_API

struct FMassEntityManager;
class UMassSignalSubsystem;
class USmartObjectSubsystem;
struct FMassExecutionContext;
struct FMassEntityHandle;
struct FMassSmartObjectUserFragment;
struct FTransformFragment;
struct FSmartObjectClaimHandle;
struct FSmartObjectHandle;
struct FZoneGraphCompactLaneLocation;
enum class ESmartObjectSlotState : uint8;

/**
 * Mediator struct that encapsulates communication between SmartObjectSubsystem and Mass.
 * This object is meant to be created and used in method scope to guarantee subsystems validity.
 */
struct FMassSmartObjectHandler
{
	/**
	 * FMassSmartObjectHandler constructor
	 * @param InExecutionContext is the current execution context of the entity subsystem
	 * @param InSmartObjectSubsystem is the smart object subsystem
	 * @param InSignalSubsystem is the mass signal subsystem to use to send signal to affected entities
	 */
	FMassSmartObjectHandler(FMassExecutionContext& InExecutionContext, USmartObjectSubsystem& InSmartObjectSubsystem, UMassSignalSubsystem& InSignalSubsystem)
		: ExecutionContext(InExecutionContext)
		, SmartObjectSubsystem(InSmartObjectSubsystem)
		, SignalSubsystem(InSignalSubsystem)
	{
	}

	UE_DEPRECATED(5.6, "Use the other constructor that doesn't require MassEntityManager")
	FMassSmartObjectHandler(FMassEntityManager& InEntityManager, FMassExecutionContext& InExecutionContext, USmartObjectSubsystem& InSmartObjectSubsystem, UMassSignalSubsystem& InSignalSubsystem)
		: FMassSmartObjectHandler(InExecutionContext, InSmartObjectSubsystem, InSignalSubsystem)
	{
	}

	/**
	 * Creates an async request to build a list of compatible smart objects
	 * around the provided location. The caller must poll using the request id
	 * to know when the reservation can be done.
	 * @param RequestingEntity Entity requesting the candidates list
	 * @param Location The center of the query
	 * @return Request identifier that can be used to try claiming a result once available
	 */
	UE_DEPRECATED(5.7, "Use the overload taking FFindCandidatesParameters instead")
	[[nodiscard]] UE_API FMassSmartObjectRequestID FindCandidatesAsync(const FMassEntityHandle RequestingEntity, const FGameplayTagContainer& UserTags, const FGameplayTagQuery& ActivityRequirements, const FVector& Location) const;

	/**
	 * Creates an async request to build a list of compatible smart objects
	 * around the provided lane location. The caller must poll using the request id
	 * to know when the reservation can be done.
	 * @param RequestingEntity Entity requesting the candidates list
	 * @param LaneLocation The lane location as reference for the query
	 * @return Request identifier that can be used to try claiming a result once available
	 */
	UE_DEPRECATED(5.7, "Use the overload taking FFindCandidatesParameters instead")
	[[nodiscard]] UE_API FMassSmartObjectRequestID FindCandidatesAsync(const FMassEntityHandle RequestingEntity, const FGameplayTagContainer& UserTags, const FGameplayTagQuery& ActivityRequirements, const FZoneGraphCompactLaneLocation& LaneLocation) const;

	/**
	 * Creates an async request to build a list of compatible smart objects
	 * using the provided set of parameters. The caller must poll using the request id
	 * to know when the reservation can be done.
	 * @param RequestingEntity Entity requesting the candidates list
	 * @param Parameters All parameters of the query
	 * @return Request identifier that can be used to try claiming a result once available
	 */
	[[nodiscard]] UE_API FMassSmartObjectRequestID FindCandidatesAsync(const FMassEntityHandle RequestingEntity, UE::Mass::SmartObject::FFindCandidatesParameters&& Parameters) const;

	/**
	 * Provides the result of a previously created request from FindCandidatesAsync to indicate if it has been processed
	 * and the results can be used by ClaimCandidate.
	 * @param RequestID A valid request identifier (method will ensure otherwise)
	 * @return The current request's result, nullptr if request not ready yet.
	 */
	[[nodiscard]] UE_API const FMassSmartObjectCandidateSlots* GetRequestCandidates(const FMassSmartObjectRequestID& RequestID) const;

	/**
	 * Deletes the request associated to the specified identifier
	 * @param RequestID A valid request identifier (method will ensure otherwise)
	 */
	UE_API void RemoveRequest(const FMassSmartObjectRequestID& RequestID) const;

	/**
	 * Claims the first available smart object from the provided candidates.
	 * @param Entity MassEntity associated to the user fragment
	 * @param User Fragment of the user claiming
	 * @param Candidates Candidate slots to choose from.
	 * @param ClaimPriority Claim priority, a slot claimed at lower priority can be claimed by higher priority (unless already in use).
	 * @return Whether the slot has been successfully claimed or not
	 */
	[[nodiscard]] UE_API FSmartObjectClaimHandle ClaimCandidate(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FMassSmartObjectCandidateSlots& Candidates, ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal) const;

	/**
	 * Claims the first available slot holding any type of USmartObjectMassBehaviorDefinition in the smart object
	 * associated to the provided identifier.
	 * @param Entity MassEntity associated to the user fragment
	 * @param User Fragment of the user claiming
	 * @param RequestResult A valid smart object request result (method will ensure otherwise)
	 * @param ClaimPriority Claim priority, a slot claimed at lower priority can be claimed by higher priority (unless already in use).
	 * @return Whether the slot has been successfully claimed or not
	 */
	[[nodiscard]] UE_API FSmartObjectClaimHandle ClaimSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FSmartObjectRequestResult& RequestResult, ESmartObjectClaimPriority ClaimPriority = ESmartObjectClaimPriority::Normal) const;

	/**
	 * Activates the mass gameplay behavior associated to the previously claimed smart object.
	 * @param Entity MassEntity associated to the user fragment
	 * @param User Fragment of the user claiming
	 * @param ClaimHandle claimed smart object slot to use.
	 * @param Transform Fragment holding the transform of the user claiming
	 * @return Whether the slot has been successfully claimed or not
	 */
	UE_API bool StartUsingSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FSmartObjectClaimHandle ClaimHandle) const;

	/**
	 * Deactivates the mass gameplay behavior started using StartUsingSmartObject.
	 * @param Entity MassEntity associated to the user fragment
	 * @param User Fragment of the user claiming
	 * @param NewStatus Reason of the deactivation.
	 */
	UE_API void StopUsingSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const EMassSmartObjectInteractionStatus NewStatus) const;

	/**
	 * Releases a claimed/in-use smart object and update user fragment.
	 * @param Entity MassEntity associated to the user fragment
	 * @param User Fragment of the user claiming
	 * @param ClaimHandle claimed smart object slot to release.
	 */
	UE_API void ReleaseSmartObject(const FMassEntityHandle Entity, FMassSmartObjectUserFragment& User, const FSmartObjectClaimHandle ClaimHandle) const;

private:
	FMassExecutionContext& ExecutionContext;
	USmartObjectSubsystem& SmartObjectSubsystem;
	UMassSignalSubsystem& SignalSubsystem;
};

#undef UE_API
