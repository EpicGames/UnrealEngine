// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassSmartObjectProcessor.generated.h"

#define UE_API MASSSMARTOBJECTS_API

class UMassSignalSubsystem;
class UZoneGraphAnnotationSubsystem;

/** Processor that builds a list of candidates objects for each users. */
UCLASS(MinimalAPI)
class UMassSmartObjectCandidatesFinderProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassSmartObjectCandidatesFinderProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** Extents used to perform the spatial query in the octree for world location queries. */
	UPROPERTY(EditDefaultsOnly, Category = SmartObject, config)
	float SearchExtents = 5000.f;

	/** Query to fetch and process requests to find smart objects using spacial query around a given world location. */
	FMassEntityQuery WorldRequestQuery;

	/** Query to fetch and process requests to find smart objects on zone graph lanes. */
	FMassEntityQuery LaneRequestQuery;
};

/** Processor for time based user's behavior that waits x seconds then releases its claim on the object */
UCLASS(MinimalAPI)
class UMassSmartObjectTimedBehaviorProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassSmartObjectTimedBehaviorProcessor();

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	FMassEntityQuery EntityQuery;
};

/** Deinitializer processor to unregister slot invalidation callback when SmartObjectUser fragment gets removed */
UCLASS(MinimalAPI)
class UMassSmartObjectUserFragmentDeinitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassSmartObjectUserFragmentDeinitializer();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};
namespace UE::Mass::SmartObject
{

/** Processor to decay smart object MRU slots */
UCLASS(MinimalAPI)
class UMRUSlotsProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMRUSlotsProcessor();

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	FMassEntityQuery EntityQuery;
};

} // UE::Mass::SmartObject
#undef UE_API
