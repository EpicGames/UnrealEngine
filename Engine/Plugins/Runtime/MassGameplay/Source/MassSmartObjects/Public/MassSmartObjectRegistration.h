// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassSignalProcessorBase.h"
#include "SmartObjectTypes.h"
#include "MassEntityQuery.h"
#include "MassSmartObjectRegistration.generated.h"

class USmartObjectDefinition;

/** Mass Tag applied on entities with FFSmartObjectRegistrationFragment that need to create smart objects */
USTRUCT()
struct FMassInActiveSmartObjectsRangeTag : public FMassTag
{
	GENERATED_BODY()
};

/** Mass Fragment storing the handle associated to the created smart object */
USTRUCT()
struct FSmartObjectRegistrationFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<USmartObjectDefinition> Asset;

	UPROPERTY()
	FSmartObjectHandle Handle;
};

/**
 * Processor that signals entities with FSmartObjectRegistration and FMassActorInstanceFragment fragments
 * when the a given tag or fragment is added to an entity.
 * @see FSmartObjectRegistrationFragment
 * @see FMassActorInstanceFragment
 */
UCLASS(Abstract)
class UMassSmartObjectInitializerBase : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UMassSmartObjectInitializerBase();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
	FName Signal;
};

/**
 * Processor that signals entities with FSmartObjectRegistration and FMassActorInstanceFragment fragments
 * when the a given tag or fragment is removed from an entity.
 * @see FSmartObjectRegistrationFragment
 * @see FMassActorInstanceFragment
 */
UCLASS(Abstract)
class UMassSmartObjectDeinitializerBase : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UMassSmartObjectDeinitializerBase();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
	FName Signal;
};

/**
 * Signals entities with UE::Mass::Signals::SmartObjectActivationChanged when 'FMassInActiveSmartObjectsRangeTag' is added.
 */
UCLASS()
class UMassActiveSmartObjectInitializer : public UMassSmartObjectInitializerBase
{
	GENERATED_BODY()
public:
	UMassActiveSmartObjectInitializer();
};

/**
 * Signals entities with UE::Mass::Signals::SmartObjectActivationChanged when 'FMassInActiveSmartObjectsRangeTag' is removed.
 */
UCLASS()
class UMassActiveSmartObjectDeinitializer : public UMassSmartObjectDeinitializerBase
{
	GENERATED_BODY()

public:
	UMassActiveSmartObjectDeinitializer();
};

/**
 * Signals entities with UE::Mass::Signals::ActorInstanceHandleChanged when 'FMassActorInstanceFragment' is added.
 */
UCLASS()
class UMassActorInstanceHandleInitializer : public UMassSmartObjectInitializerBase
{
	GENERATED_BODY()
public:
	UMassActorInstanceHandleInitializer();
};

/**
 * Signals entities with UE::Mass::Signals::ActorInstanceHandleChanged when 'FMassActorInstanceFragment' is removed.
 */
UCLASS()
class UMassActorInstanceHandleDeinitializer : public UMassSmartObjectDeinitializerBase
{
	GENERATED_BODY()

public:
	UMassActorInstanceHandleDeinitializer();
};

/**
 * Signal based processor that creates and destroys the smart object instance associated to an entity based
 * on valid FSmartObjectRegistration and FMassActorInstance fragments.
 * The registration is processed on the following events:
 * 	 UE::Mass::Signals::ActorInstanceHandleChanged
 * 	 UE::Mass::Signals::SmartObjectActivationChanged
 * @see FFSmartObjectRegistrationFragment
 */
UCLASS()
class UMassActiveSmartObjectSignalProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	UMassActiveSmartObjectSignalProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& /*Unused*/) override;

private:
	FMassEntityQuery InsideSmartObjectActiveRangeQuery;
	FMassEntityQuery OutsideSmartObjectActiveRangeQuery;
};
