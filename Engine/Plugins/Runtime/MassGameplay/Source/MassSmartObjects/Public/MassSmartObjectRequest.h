// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SmartObjectSubsystem.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/StaticArray.h"
#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "MassSmartObjectTypes.h"
#include "SmartObjectRequestTypes.h"
#include "ZoneGraphTypes.h"
#include "MassSmartObjectRequest.generated.h"

/**
 * Structure that represents a potential smart object slot for a MassEntity during the search
 */
USTRUCT()
struct FSmartObjectCandidateSlot
{
	GENERATED_BODY()

	FSmartObjectCandidateSlot() = default;
	FSmartObjectCandidateSlot(const FSmartObjectRequestResult InResult, const float InCost)	: Result(InResult), Cost(InCost) {}

	UPROPERTY(VisibleAnywhere, Category = SmartObject, transient)
	FSmartObjectRequestResult Result;

	UPROPERTY(VisibleAnywhere, Category = SmartObject, transient)
	float Cost = 0.f;
};

/**
 * Identifier associated to a request for smart object candidates. We use a 1:1 match
 * with an FMassEntityHandle since all requests are batched together using the EntitySubsystem.
 */
USTRUCT()
struct FMassSmartObjectRequestID
{
	GENERATED_BODY()

	FMassSmartObjectRequestID() = default;
	explicit FMassSmartObjectRequestID(const FMassEntityHandle InEntity) : Entity(InEntity) {}

	bool IsSet() const { return Entity.IsSet(); }
	void Reset() { Entity.Reset(); }

	explicit operator FMassEntityHandle() const { return Entity; }

private:
	UPROPERTY(Transient)
	FMassEntityHandle Entity;
};

/**
 * Struct that holds status and results of a candidate finder request
 */
USTRUCT(BlueprintType)
struct FMassSmartObjectCandidateSlots
{
	GENERATED_BODY()

	void Reset()
	{
		NumSlots = 0;
	}

	//~ For StructOpsTypeTraits
	bool ExportTextItem(FString& ValueStr, const FMassSmartObjectCandidateSlots& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;

	static constexpr uint32 MaxNumCandidates = 4;
	TStaticArray<FSmartObjectCandidateSlot, MaxNumCandidates> Slots;

	UPROPERTY(Transient, VisibleAnywhere, Category = SmartObject)
	uint8 NumSlots = 0;
};

inline bool FMassSmartObjectCandidateSlots::ExportTextItem(FString& ValueStr, const FMassSmartObjectCandidateSlots& DefaultValue, UObject* Parent, const int32 PortFlags, UObject* ExportRootScope) const
{
	for (int32 SlotIndex = 0; SlotIndex < NumSlots; SlotIndex++)
	{
		const FSmartObjectCandidateSlot& Slot = Slots[SlotIndex];
		FSmartObjectCandidateSlot::StaticStruct()->ExportText(ValueStr, &Slot, &Slot, Parent, PortFlags, ExportRootScope);
	}

	constexpr bool bSkipGenericExport = false;
	return bSkipGenericExport;
}

template<>
struct TStructOpsTypeTraits<FMassSmartObjectCandidateSlots> : TStructOpsTypeTraitsBase2<FMassSmartObjectCandidateSlots>
{
	enum
	{
		WithExportTextItem = true,
	};
};

/**
 * Fragment that holds the result of a request to find candidates.
 */
USTRUCT()
struct FMassSmartObjectRequestResultFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FMassSmartObjectCandidateSlots Candidates;

	UPROPERTY(Transient)
	bool bProcessed = false;
};

/**
 * Fragment used to build a list potential smart objects to use. Once added to an entity
 * this will be processed by the candidates finder processor to fill a SmartObjectCandidates
 * fragment that could then be processed by the reservation processor
 */
USTRUCT()
struct FMassSmartObjectWorldLocationRequestFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FVector SearchOrigin = FVector::ZeroVector;

	UPROPERTY(Transient)
	FMassEntityHandle RequestingEntity;

	UPROPERTY(Transient)
	FGameplayTagContainer UserTags;

	UPROPERTY(Transient)
	FGameplayTagQuery ActivityRequirements;
};

template<>
struct TMassFragmentTraits<FMassSmartObjectWorldLocationRequestFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Fragment used to build a list potential smart objects to use. Once added to an entity
 * this will be processed by the candidates finder processor to fill a SmartObjectCandidates
 * fragment that could then be processed by the reservation processor
 */
USTRUCT()
struct FMassSmartObjectLaneLocationRequestFragment : public FMassFragment
{
	GENERATED_BODY()

	FZoneGraphCompactLaneLocation CompactLaneLocation;

	UPROPERTY(Transient)
	FMassEntityHandle RequestingEntity;

	UPROPERTY(Transient)
	FGameplayTagContainer UserTags;

	UPROPERTY(Transient)
	FGameplayTagQuery ActivityRequirements;
};

template<>
struct TMassFragmentTraits<FMassSmartObjectLaneLocationRequestFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Special tag to mark processed requests
 */
USTRUCT()
struct FMassSmartObjectCompletedRequestTag : public FMassTag
{
	GENERATED_BODY()
};

namespace UE::Mass::SmartObject
{

/**
 * Struct used to store parameters for FindCandidatesAsync requests
 */
USTRUCT()
struct FFindCandidatesParameters
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FGameplayTagContainer UserTags;

	UPROPERTY(Transient)
	FGameplayTagQuery ActivityRequirements;

	UPROPERTY(Transient)
	FZoneGraphCompactLaneLocation LaneLocation;

	UPROPERTY(Transient)
	FVector Location = FVector::ZeroVector;

	UPROPERTY(Transient)
	FMRUSlots MRUSlots;
};

} // UE::Mass::SmartObject