// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZoneGraphTypes.h"
#include "MassSettings.h"
#include "MassSmartObjectSettings.generated.h"

#define UE_API MASSSMARTOBJECTS_API

#if WITH_EDITOR
/** Called when annotation tag settings changed. */
DECLARE_MULTICAST_DELEGATE(FOnAnnotationSettingsChanged);
#endif

/**
 * Settings for the MassSmartObject module.
 */
UCLASS(MinimalAPI, config = Plugins, defaultconfig, DisplayName = "Mass SmartObject")
class UMassSmartObjectSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	mutable FOnAnnotationSettingsChanged OnAnnotationSettingsChanged;
#endif

	/** Tag used to indicate that smart objects are associated to a lane for queries using lanes. */
	UPROPERTY(EditDefaultsOnly, Category = ZoneGraph, config)
	FZoneGraphTag SmartObjectTag;

	/** Extents used to find precomputed entry points to reach a smart object from a zone graph lane. */
	UPROPERTY(EditDefaultsOnly, Category = ZoneGraph, config)
	float SearchExtents = 500.f;

	/**
	 * Can be used to prevent smart object users from reusing the same smart object slots more than once.
	 * The list of most recently used slots can contain up to 4 slots, then a newly used slot will 
	 * replace the least recently used one.
	 * It is also possible to use a cooldown to allow slots to be considered again after a given
	 * period of time.
	 * @see bUseCooldownForMRUSlots
	 * @see MRUSlotsCooldown
	 */
	//~ Max values must stay in sync with FMRUSlots::MaxSlots
	UPROPERTY(EditDefaultsOnly, Category = SmartObject, config, meta = (UIMin = "0", ClampMin = "0", UIMax = "4", ClampMax = "4", DisplayName = ""))
	int32 MRUSlotsMaxCount = 0;

	/**
	 * Set this flag to allow the most recently used slots to be considered again after the period of time specified by MRUSlotsCooldown
	 * This option requires MRUSlotsMaxCount to be a non-zero value
	 * @see MRUSlotsCooldown
	 * @see MRUSlotsMaxCount
	 */
	UPROPERTY(config, EditAnywhere, Category = SmartObject, meta = (EditCondition = "MRUSlotsMaxCount > 0", EditConditionHides))
	bool bUseCooldownForMRUSlots = false;

	/**
	 * Period of time during which a given slot will not be considered after being used
	 * This option requires 'bUseCooldownForMRUSlots' to be true and 'MRUSlotsMaxCount' to be a non-zero value
	 * @see bUseCooldownForMRUSlots
	 * @see MRUSlotsMaxCount
	 */
	UPROPERTY(EditDefaultsOnly, Category = SmartObject, config, meta = (UIMin = "0", ClampMin = "0", EditCondition = "MRUSlotsMaxCount > 0 && bUseCooldownForMRUSlots", EditConditionHides))
	float MRUSlotsCooldown = 0.f;

protected:

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
