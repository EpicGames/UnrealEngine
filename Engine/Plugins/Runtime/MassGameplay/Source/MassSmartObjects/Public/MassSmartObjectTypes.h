// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/EntityHandle.h"
#include "SmartObjectTypes.h"
#include "MassSmartObjectTypes.generated.h"

#define UE_API MASSSMARTOBJECTS_API

namespace UE::Mass::Signals
{
	const FName SmartObjectRequestCandidates = FName(TEXT("SmartObjectRequestCandidates"));
	const FName SmartObjectCandidatesReady = FName(TEXT("SmartObjectCandidatesReady"));
	const FName SmartObjectInteractionDone = FName(TEXT("SmartObjectInteractionDone"));
	const FName SmartObjectInteractionAborted = FName(TEXT("SmartObjectInteractionAborted"));
}

UENUM()
enum class EMassSmartObjectInteractionStatus : uint8
{
	Unset,
	InProgress,			// Claimed and Behavior activated
	BehaviorCompleted,	// Behavior is completed but task still running (not updated yet)
	TaskCompleted,		// Task has been notified that behavior is completed and completes
	Aborted				// Task and Behavior were aborted
};

/**
 * Struct that can be used to pass data to the find or filtering methods.
 * Properties will be used as external data to fill values expected by the world condition schema
 * specified by the smart object definition.
 *		e.g. FilterSlotsBySelectionConditions(SlotHandles, FConstStructView::Make(FSmartObjectMassEntityUserData(Entity)));
 *
 * It can be inherited from to provide additional data to another world condition schema inheriting
 * from USmartObjectWorldConditionSchema.
 *	e.g.
 *		UCLASS()
 *		class USmartObjectWorldConditionExtendedSchema : public USmartObjectWorldConditionSchema
 *		{
 *			...
 *			USmartObjectWorldConditionExtendedSchema(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
 *			{
 *				OtherEntityRef = AddContextDataDesc(TEXT("OtherEntity"), FMassEntityHandle::StaticStruct(), EWorldConditionContextDataType::Dynamic);
 *			}
 *			
 *			FWorldConditionContextDataRef OtherEntityRef;
 *		};
 *
 *		USTRUCT()
 *		struct FSmartObjectMassEntityExtendedUserData : public FSmartObjectMassEntityUserData
 *		{
 *			UPROPERTY()
 *			FMassEntityHandle OtherEntity;
 *		}
 *
 * The struct can also be used to be added to a Smart Object slot when it gets claimed.
 *		e.g. Claim(SlotHandle, FConstStructView::Make(FSmartObjectMassEntityUserData(Entity)));
 */
USTRUCT()
struct FSmartObjectMassEntityUserData
{
	GENERATED_BODY()

	FSmartObjectMassEntityUserData() = default;
	explicit FSmartObjectMassEntityUserData(const FMassEntityHandle InEntityHandle) : UserEntity(InEntityHandle) {}

	UPROPERTY()
	FMassEntityHandle UserEntity;
};

namespace UE::Mass::SmartObject
{

/**
 * Struct holding data related to a single recently used smart object slot.
 */
USTRUCT()
struct FMRUSlot
{
	GENERATED_BODY()

	static constexpr float NoCoolDown = -1.f;

	UPROPERTY(VisibleAnywhere, Category = SmartObject, transient)
	FSmartObjectSlotHandle Slot;

	UPROPERTY(VisibleAnywhere, Category = SmartObject, transient)
	float RemainingTime = NoCoolDown;
};

/**
 * Struct used as a container for the most recently used slots.
 */
USTRUCT()
struct FMRUSlots
{
	GENERATED_BODY()

	void Reset()
	{
		NumSlots = 0;
	}

	UE_API void Push(const FSmartObjectSlotHandle& ClaimedSlot);
	UE_API void AppendTo(TArray<FSmartObjectSlotHandle>& OutSlots) const;

	/** Using a fixed maximum number of slots to use fixed inline allocation in the fragments. */
	static constexpr uint32 MaxNumSlots = 4;

	UPROPERTY(Transient)
	FMRUSlot Slots[MaxNumSlots];

	UPROPERTY(Transient)
	uint8 NumSlots = 0;
};

} // UE::Mass::SmartObject

#undef UE_API
