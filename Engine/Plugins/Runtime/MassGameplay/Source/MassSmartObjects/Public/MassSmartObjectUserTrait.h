// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "GameplayTagContainer.h"
#include "MassSmartObjectUserTrait.generated.h"

#define UE_API MASSSMARTOBJECTS_API

/**
 * Trait to allow an entity to interact with SmartObjects
 */
UCLASS(MinimalAPI, meta = (DisplayName = "SmartObject User"))
class UMassSmartObjectUserTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	/** Tags describing the SmartObject user. Used when searching smart objects. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTagContainer UserTags;
};

#undef UE_API
