// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SmartObjectTypes.h"
#include "ZoneGraphAnnotationComponent.h"
#include "ZoneGraphTypes.h"
#include "SmartObjectZoneAnnotations.generated.h"

#define UE_API MASSSMARTOBJECTS_API

struct FSmartObjectZoneAnnotationsInstanceData;
class AZoneGraphData;
class USmartObjectSubsystem;

/** Struct to keep track of a SmartObject entry point on a given lane. */
USTRUCT()
struct FSmartObjectLaneLocation
{
	GENERATED_BODY()

	FSmartObjectLaneLocation() = default;
	FSmartObjectLaneLocation(const FSmartObjectHandle InObjectHandle, const int32 InLaneIndex, const float InDistanceAlongLane)
        : ObjectHandle(InObjectHandle)
        , LaneIndex(InLaneIndex)
        , DistanceAlongLane(InDistanceAlongLane)
	{
	}

	UPROPERTY()
	FSmartObjectHandle ObjectHandle;

	UPROPERTY()
	int32 LaneIndex = INDEX_NONE;

	UPROPERTY()
	float DistanceAlongLane = 0.0f;
};

/**
 * Struct to store indices to all entry points on a given lane.
 * Used as a container wrapper to be able to use in a TMap.
 */
USTRUCT()
struct FSmartObjectLaneLocationIndices
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TArray<int32> SmartObjectLaneLocationIndices;
};

/** Per ZoneGraphData smart object look up data. */
USTRUCT()
struct FSmartObjectAnnotationData
{
	GENERATED_BODY()

	/** @return True if this entry is valid (associated to a valid zone graph data), false otherwise. */
	bool IsValid() const { return DataHandle.IsValid(); }

	/** Reset all internal data. */
	void Reset()
	{
		DataHandle = {};
		AffectedLanes.Reset();
		SmartObjectLaneLocations.Reset();
		SmartObjectToLaneLocationIndexLookup.Reset();
		LaneToLaneLocationIndicesLookup.Reset();
	}

	/** Handle of the ZoneGraphData that this smart object annotation data is associated to */
	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	FZoneGraphDataHandle DataHandle;

	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TArray<int32> AffectedLanes;

	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TArray<FSmartObjectLaneLocation> SmartObjectLaneLocations;

	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TMap<FSmartObjectHandle, int32> SmartObjectToLaneLocationIndexLookup;

	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TMap<int32, FSmartObjectLaneLocationIndices> LaneToLaneLocationIndicesLookup;

	bool bInitialTaggingCompleted = false;
};

/**
 * ZoneGraph annotations for smart objects
 */
UCLASS(MinimalAPI, ClassGroup = AI, BlueprintType, meta = (BlueprintSpawnableComponent))
class USmartObjectZoneAnnotations : public UZoneGraphAnnotationComponent
{
	GENERATED_BODY()

public:
	UE_API const FSmartObjectAnnotationData* GetAnnotationData(FZoneGraphDataHandle DataHandle) const;
	UE_API TOptional<FSmartObjectLaneLocation> GetSmartObjectLaneLocation(const FZoneGraphDataHandle DataHandle, const FSmartObjectHandle SmartObjectHandle) const;
	UE_API void ApplyComponentInstanceData(FSmartObjectZoneAnnotationsInstanceData* InstanceData);

	TConstArrayView<FSmartObjectAnnotationData> GetSmartObjectAnnotations() const
	{
		return SmartObjectAnnotationDataArray;
	}

protected:
	UE_API virtual void PostSubsystemsInitialized() override;
	UE_API virtual FZoneGraphTagMask GetAnnotationTags() const override;
	UE_API virtual void TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& BehaviorTagContainer) override;

	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;

	UE_API virtual void PostZoneGraphDataAdded(const AZoneGraphData& ZoneGraphData) override;
	UE_API virtual void PreZoneGraphDataRemoved(const AZoneGraphData& ZoneGraphData) override;

#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual void DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy) override;
#endif // UE_ENABLE_DEBUG_DRAWING

	/** Filter specifying which lanes the behavior is applied to. */
	UPROPERTY(EditAnywhere, Category = SmartObject)
	FZoneGraphTagFilter AffectedLaneTags;

	/** Entry points graph for each ZoneGraphData. */
	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	TArray<FSmartObjectAnnotationData> SmartObjectAnnotationDataArray;

	/** Tag to mark the lanes that offers smart objects. */
	UPROPERTY(VisibleAnywhere, Category = SmartObject)
	FZoneGraphTag BehaviorTag;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void OnUnregister() override;

	UE_API void RebuildForSingleGraph(FSmartObjectAnnotationData& Data, const FZoneGraphStorage& Storage);
	UE_API void RebuildForAllGraphs();

	FDelegateHandle OnAnnotationSettingsChangedHandle;
	FDelegateHandle OnGraphDataChangedHandle;
	FDelegateHandle OnMainCollectionChangedHandle;
	FDelegateHandle OnMainCollectionDirtiedHandle;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	UE_API virtual void Serialize(FArchive& Ar) override;
	bool bRebuildAllGraphsRequested = false;
#endif

	/** Cached SmartObjectSubsystem */
	UPROPERTY(Transient)
	TObjectPtr<USmartObjectSubsystem> SmartObjectSubsystem = nullptr;
};

/** Used to store data that is considered modified by the UCS and not generically saved during RerunConstructionScripts */
USTRUCT()
struct FSmartObjectZoneAnnotationsInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()

public:
	FSmartObjectZoneAnnotationsInstanceData() = default;
	FSmartObjectZoneAnnotationsInstanceData(const USmartObjectZoneAnnotations* SourceComponent)
		: FActorComponentInstanceData(SourceComponent)
		, SmartObjectAnnotations(SourceComponent->GetSmartObjectAnnotations())
	{}

	virtual ~FSmartObjectZoneAnnotationsInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return SmartObjectAnnotations.Num() || Super::ContainsData();
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);

		if (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript)
		{
			CastChecked<USmartObjectZoneAnnotations>(Component)->ApplyComponentInstanceData(this);
		}
	}

	UPROPERTY()
	TArray<FSmartObjectAnnotationData> SmartObjectAnnotations;
};

#undef UE_API
