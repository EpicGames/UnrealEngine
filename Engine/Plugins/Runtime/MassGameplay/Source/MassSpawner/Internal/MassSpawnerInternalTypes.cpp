// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpawnerInternalTypes.h"
#include "MassSpawnerTypes.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpawnerInternalTypes)

void UMassCreatedBySpawnerTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassCreatedBySpawner>();
}
