// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Mass/EntityElementTypes.h"
#include "MassEntityTraitBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassSpawnerInternalTypes.generated.h"

class AMassSpawner;

UCLASS(MinimalAPI, Hidden)
class UMassCreatedBySpawnerTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

public:

	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

USTRUCT(meta = (Hidden))
struct FMassCreatedBySpawner : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<AMassSpawner> Spawner;
};

template<>
struct TMassFragmentTraits<FMassCreatedBySpawner> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};
