// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassSpawner : ModuleRules
	{
		public MassSpawner(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassSimulation",
					"ZoneGraph",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"UnrealEd",
						"Slate"
					}
				);

				PublicDependencyModuleNames.Add("MassEntityEditor");
				// here for communication with MassTraitRepository
				PrivateIncludePathModuleNames.Add("MassGameplayEditor");
				DynamicallyLoadedModuleNames.Add("MassGameplayEditor");
			}
		}
	}
}
