// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAssortedFragmentsTrait.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAssortedFragmentsTrait)


void UMassAssortedFragmentsTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	for (const FInstancedStruct& Fragment : Fragments)
	{
		if (Fragment.IsValid())
		{
			const UScriptStruct* Type = Fragment.GetScriptStruct();
			CA_ASSUME(Type);
			if (UE::Mass::IsA<FMassFragment>(Type))
			{
				BuildContext.AddFragment(Fragment);
			}
			else
			{
				UE_LOGF(LogMass, Error, "Struct type %ls is not a child of FMassFragment", *GetPathNameSafe(Type));
			}
		}
	}
	
	for (const FInstancedStruct& Tag : Tags)
	{
		const UScriptStruct* Type = Tag.GetScriptStruct();
		if (UE::Mass::IsA<FMassTag>(Type))
		{
			BuildContext.AddTag(Type);
		}
		else
		{
			UE_LOGF(LogMass, Error, "Struct type %ls is not a child of FMassTag", *GetPathNameSafe(Type));
		}
	}
}
