// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityConfigAsset.h"
#include "Logging/MessageLog.h"
#include "MassEntityTraitBase.h"
#include "MassSpawnerTypes.h"
#include "MassSpawnerSubsystem.h"
#include "MassEntityTemplateRegistry.h"
#include "VisualLogger/VisualLogger.h"
#include "Engine/World.h"
#if WITH_EDITOR
#include "Editor.h"
#include "MassEntityEditor.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityConfigAsset)

#define LOCTEXT_NAMESPACE "Mass"


//-----------------------------------------------------------------------------
// FMassEntityConfig
//-----------------------------------------------------------------------------
FMassEntityConfig::FMassEntityConfig()
{
	ConfigGuid = FGuid::NewGuid();
}

FMassEntityConfig::FMassEntityConfig(UObject& InOwner)
	: ConfigOwner(&InOwner)
{
	ConfigGuid = FGuid::NewDeterministicGuid(InOwner.GetPathName());
}

UMassEntityTraitBase* FMassEntityConfig::FindTraitInternal(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch) const
{
	TArray<const UMassEntityConfigAsset*, TInlineAllocator<8>> Visited;
	for (const FMassEntityConfig* Current = this; Current != nullptr; )
	{
		for (const TObjectPtr<UMassEntityTraitBase>& Trait : Current->Traits)
		{
			if (Trait && (bExactMatch ? Trait->GetClass() == TraitClass : Trait->IsA(TraitClass)))
			{
				return Trait;
			}
		}

		const UMassEntityConfigAsset* NextParent = Current->Parent;
		if (NextParent == nullptr)
		{
			break;
		}
		if (Visited.Contains(NextParent))
		{
			UE_VLOG(Current->ConfigOwner, LogMassSpawner, Error, TEXT("%s: Encountered %s as parent second time (Infinite loop) while searching for trait %s")
				, *GetNameSafe(Current->ConfigOwner), *GetNameSafe(NextParent), *GetNameSafe(TraitClass.Get()));
			break;
		}
		Visited.Add(NextParent);
		Current = &NextParent->GetConfig();
	}
	return nullptr;
}

const FMassEntityTemplate& FMassEntityConfig::GetOrCreateEntityTemplate(const UWorld& World) const
{
	FMassEntityTemplateID TemplateID;
	if (const FMassEntityTemplate* ExistingTemplate = GetEntityTemplateInternal(World, TemplateID))
	{
		return *ExistingTemplate;
	}

	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(&World);
	check(SpawnerSystem);
	FMassEntityTemplateRegistry& TemplateRegistry = SpawnerSystem->GetMutableTemplateRegistryInstance();

	// Build new template
	// TODO: Add methods to FMassEntityTemplateBuildContext to indicate dependency vs setup.
	// Dependency should add a fragment with default values (which later can be overridden),
	// while setup would override values and should be run just once.

	FMassEntityTemplateData TemplateData;
	FMassEntityTemplateBuildContext BuildContext(TemplateData, TemplateID);

	TArray<UMassEntityTraitBase*> CombinedTraits;
	GetCombinedTraits(CombinedTraits);

	BuildContext.BuildFromTraits(CombinedTraits, World);
	BuildContext.SetTemplateName(GetNameSafe(ConfigOwner));

	return TemplateRegistry.FindOrAddTemplate(TemplateID, MoveTemp(TemplateData)).Get();
}

void FMassEntityConfig::DestroyEntityTemplate(const UWorld& World) const
{
	FMassEntityTemplateID TemplateID;
	const FMassEntityTemplate* Template = GetEntityTemplateInternal(World, TemplateID);
	if (Template == nullptr)
	{
		return;
	}

	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(&World);
	check(SpawnerSystem);
	FMassEntityTemplateRegistry& TemplateRegistry = SpawnerSystem->GetMutableTemplateRegistryInstance();

	TArray<UMassEntityTraitBase*> CombinedTraits;
	GetCombinedTraits(CombinedTraits);

	for (const UMassEntityTraitBase* Trait : CombinedTraits)
	{
		check(Trait);
		Trait->DestroyTemplate(World);
	}

	// TODO - The templates are not being torn down completely, resulting in traits that leave data in various subsystems. (Representation system)
	
	TemplateRegistry.DestroyTemplate(TemplateID);
}

const FMassEntityTemplate& FMassEntityConfig::GetEntityTemplateChecked(const UWorld& World) const
{
	FMassEntityTemplateID TemplateID;
	const FMassEntityTemplate* ExistingTemplate = GetEntityTemplateInternal(World, TemplateID);
	check(ExistingTemplate);
	return *ExistingTemplate;
}

const FMassEntityTemplate* FMassEntityConfig::GetEntityTemplateInternal(const UWorld& World, FMassEntityTemplateID& OutTemplateID) const
{
	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(&World);
	check(SpawnerSystem);
	const FMassEntityTemplateRegistry& TemplateRegistry = SpawnerSystem->GetTemplateRegistryInstance();

	// Return existing template if found.
	OutTemplateID = FMassEntityTemplateIDFactory::Make(ConfigGuid);
	const TSharedRef<FMassEntityTemplate>* TemplateFound = TemplateRegistry.FindTemplateFromTemplateID(OutTemplateID);
	return TemplateFound ? &TemplateFound->Get() : nullptr;
}

void FMassEntityConfig::GetCombinedTraits(TArray<UMassEntityTraitBase*>& OutTraits, FMassEntityTemplateBuildContext* BuildContext) const
{
	TArray<const UObject*> Visited;
	OutTraits.Reset();
	Visited.Add(ConfigOwner);
	GetCombinedTraitsInternal(OutTraits, Visited, BuildContext);
}

void FMassEntityConfig::GetCombinedTraitsInternal(TArray<UMassEntityTraitBase*>& OutTraits, TArray<const UObject*>& Visited, FMassEntityTemplateBuildContext* BuildContext) const
{
	if (Parent)
	{
		if (Visited.IndexOfByKey(Parent) != INDEX_NONE)
		{
			// Infinite loop detected.
			FString Path;
			for (const UObject* Object : Visited)
			{
				Path += Object->GetName();
				Path += TEXT("/");
			}
			UE_VLOG(ConfigOwner, LogMassSpawner, Error, TEXT("%s: Encountered %s as parent second time (Infinite loop). %s"), *GetNameSafe(ConfigOwner), *GetNameSafe(Parent), *Path);
		}
		else
		{
			Visited.Add(Parent);
			Parent->GetConfig().GetCombinedTraitsInternal(OutTraits, Visited, nullptr);
		}
	}

	for (UMassEntityTraitBase* Trait : Traits)
	{
		if (!Trait)
		{
			continue;
		}
		// Allow only one feature per type. This is also used to allow child configs override parent features.
		const int32 Index = OutTraits.IndexOfByPredicate([Trait](const UMassEntityTraitBase* ExistingFeature) -> bool
		{
			return Trait->GetClass() == ExistingFeature->GetClass()
				// we allow duplicating trait's that are configured for different targets
				&& static_cast<bool>(Trait->ValidTargetConfig & ExistingFeature->ValidTargetConfig);
		});

		if (Index != INDEX_NONE)
		{
			UE_LOGF(LogMassSpawner, Warning, "%ls: Overriding %ls trait with another instance"
				, *GetNameSafe(ConfigOwner), *Trait->GetName());
			if (BuildContext)
			{
				BuildContext->RegisterOverriddenTrait(Trait);
			}
			OutTraits[Index] = Trait;
		}
		else
		{
			OutTraits.Add(Trait);
		}
	}
}

void FMassEntityConfig::AddTrait(UMassEntityTraitBase& Trait)
{
	Traits.Add(&Trait);
}

bool FMassEntityConfig::ValidateEntityTemplate(const UWorld& World)
{
	FMassEntityTemplateData Template;
	FMassEntityTemplateBuildContext BuildContext(Template);
	TArray<UMassEntityTraitBase*> CombinedTraits;
	GetCombinedTraits(CombinedTraits, &BuildContext);

	return BuildContext.BuildFromTraits(CombinedTraits, World);
}

#if WITH_EDITOR
void FMassEntityConfig::PostDuplicate(const bool bDuplicateForPIE)
{
	if (bDuplicateForPIE == false)
	{
		ForceRegenerateGUID();
	}
}

void FMassEntityConfig::ForceRegenerateGUID()
{
	ConfigGuid = FGuid::NewGuid();
}

UMassEntityTraitBase* FMassEntityConfig::FindMutableTrait(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch)
{
	return FindTraitInternal(TraitClass, bExactMatch);
}
#endif // WITH_EDITOR

//-----------------------------------------------------------------------------
// UMassEntityConfigAsset
//-----------------------------------------------------------------------------
#if WITH_EDITOR
void UMassEntityConfigAsset::PostDuplicate(const bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	
	Config.PostDuplicate(bDuplicateForPIE);
}

void UMassEntityConfigAsset::ValidateEntityConfig()
{
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);
		MessageLog.NewPage(FText::FromName(UE::Mass::Editor::MessageLogPageName));

		if (Config.ValidateEntityTemplate(*EditorWorld))
		{
			FMassEditorNotification Notification;
			Notification.Message = FText::FormatOrdered(LOCTEXT("MassEntityConfigAssetNoErrorsDetected", "There were no errors detected during validation of {0}")
				, FText::FromName(GetFName()));
			Notification.Severity = EMessageSeverity::Info;
			Notification.Show();
		}
	}
}

UMassEntityTraitBase* UMassEntityConfigAsset::AddTrait(TSubclassOf<UMassEntityTraitBase> TraitClass)
{
	check(TraitClass);

	UMassEntityTraitBase* TraitInstance = Config.FindMutableTrait(TraitClass, /*bExactMatch=*/true);
	if (TraitInstance == nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("ProcedurallyAddingTrait", "Adding a trait procedurally"));

		Modify();

		TraitInstance = NewObject<UMassEntityTraitBase>(this, TraitClass, FName(), RF_Transactional);
		check(TraitInstance);
		Config.AddTrait(*TraitInstance);
	}
	return TraitInstance;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE 
