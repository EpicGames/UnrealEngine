// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityEQSSpawnPointsGenerator.h"
#include "MassSpawnerTypes.h"
#include "MassSpawnLocationProcessor.h"
#include "VisualLogger/VisualLogger.h"
#include "MassGameplaySettings.h"
#include "MassCommonUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityEQSSpawnPointsGenerator)


UMassEntityEQSSpawnPointsGenerator::UMassEntityEQSSpawnPointsGenerator()
{
	EQSRequest.RunMode = EEnvQueryRunMode::AllMatching; 
}

void UMassEntityEQSSpawnPointsGenerator::Generate(UObject& QueryOwner, TConstArrayView<FMassSpawnedEntityType> EntityTypes, const int32 Count, FFinishedGeneratingSpawnDataSignature& FinishedGeneratingSpawnPointsDelegate) const
{
	if (Count <= 0)
	{
		FinishedGeneratingSpawnPointsDelegate.Execute(TArray<FMassEntitySpawnDataGeneratorResult>());
		return;
	}
	
	// Need to copy the request as it is called inside a CDO and CDO states cannot be changed.
	FEQSParametrizedQueryExecutionRequest EQSRequestInstanced = EQSRequest;
	if (EQSRequestInstanced.IsValid() == false)
	{
		EQSRequestInstanced.InitForOwnerAndBlackboard(QueryOwner, /*BBAsset=*/nullptr);
		if (!ensureMsgf(EQSRequestInstanced.IsValid(), TEXT("Query request initialization can fail due to missing parameters. See the runtime log for details")))
		{
			return;
		}
	}

	// Build array of entity types to spawn.
	// @todo: I dont like that this get's passed by value to OnEQSQueryFinished, but seemed like the cleanest solution.
	TArray<FMassEntitySpawnDataGeneratorResult> Results;
	BuildResultsFromEntityTypes(Count, EntityTypes, Results);

	FQueryFinishedSignature Delegate = FQueryFinishedSignature::CreateUObject(this, &UMassEntityEQSSpawnPointsGenerator::OnEQSQueryFinished, Results, FinishedGeneratingSpawnPointsDelegate);
	EQSRequestInstanced.Execute(QueryOwner, /*BlackboardComponent=*/nullptr, Delegate);
}

void UMassEntityEQSSpawnPointsGenerator::OnEQSQueryFinished(TSharedPtr<FEnvQueryResult> EQSResult, TArray<FMassEntitySpawnDataGeneratorResult> Results,
															FFinishedGeneratingSpawnDataSignature FinishedGeneratingSpawnPointsDelegate) const
{
	if (EQSResult.IsValid() == false || EQSResult->IsSuccessful() == false)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Error, TEXT("EQS query failed or result is invalid"));
		// Return empty result.
		Results.Reset();
		FinishedGeneratingSpawnPointsDelegate.Execute(Results);
		return;
	}

	TArray<FVector> Locations;
	EQSResult->GetAllAsLocations(Locations);

	// Randomize them
	FRandomStream RandomStream(UE::Mass::Utils::OverrideRandomSeedForTesting(GetRandomSelectionSeed()));
	for (int32 I = 0; I < Locations.Num(); ++I)
	{
		const int32 J = RandomStream.RandHelper(Locations.Num());
		Locations.Swap(I, J);
	}

	const int32 LocationCount = Locations.Num();
	int32 LocationIndex = 0;

	// Distribute points amongst the entities to spawn.
	for (FMassEntitySpawnDataGeneratorResult& Result : Results)
	{
		Result.SpawnDataProcessor = UMassSpawnLocationProcessor::StaticClass();
		Result.SpawnData.InitializeAs<FMassTransformsSpawnData>();
		FMassTransformsSpawnData& Transforms = Result.SpawnData.GetMutable<FMassTransformsSpawnData>();

		Transforms.Transforms.Reserve(Result.NumEntities);
		for (int i = 0; i < Result.NumEntities; i++)
		{
			FTransform& Transform = Transforms.Transforms.AddDefaulted_GetRef();
			Transform.SetLocation(Locations[LocationIndex % LocationCount]);
			LocationIndex++;
		}
	}

#if ENABLE_VISUAL_LOG
	UE_VLOG(this, LogMassSpawner, Log, TEXT("Spawning at %d locations"), LocationIndex);
	if (GetDefault<UMassGameplaySettings>()->bLogSpawnLocations)
	{
		FVisualLogger::Get().ExecuteOnLastEntryForObject(this, [LocationIndex, &Results](FVisualLogEntry& LogEntry)
			{
				FVisualLogShapeElement Element(TEXT(""), FColor::Orange, /*Thickness*/20, LogMassSpawner.GetCategoryName());

				Element.Points.Reserve(LocationIndex);
				for (const FMassEntitySpawnDataGeneratorResult& Result : Results)
				{
					const FMassTransformsSpawnData& Transforms = Result.SpawnData.Get<FMassTransformsSpawnData>();
					for (int i = 0; i < Result.NumEntities; i++)
					{
						Element.Points.Add(Transforms.Transforms[i].GetLocation());
					}
				}

				Element.Type = EVisualLoggerShapeElement::SinglePoint;
				Element.Verbosity = ELogVerbosity::Display;
				LogEntry.AddElement(Element);
			});
	}
#endif // ENABLE_VISUAL_LOG

	FinishedGeneratingSpawnPointsDelegate.Execute(Results);
}

#if WITH_EDITOR
void UMassEntityEQSSpawnPointsGenerator::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName EQSRequestName = GET_MEMBER_NAME_CHECKED(UMassEntityEQSSpawnPointsGenerator, EQSRequest);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == EQSRequestName)
	{
		EQSRequest.PostEditChangeProperty(*this, PropertyChangedEvent);
	}
}
#endif
