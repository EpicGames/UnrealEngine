// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntitySpawnDataGeneratorBase.h"
#include "VisualLogger/VisualLogger.h"
#include "MassSpawnerTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntitySpawnDataGeneratorBase)

void UMassEntitySpawnDataGeneratorBase::BuildResultsFromEntityTypes(const int32 SpawnCount, TConstArrayView<FMassSpawnedEntityType> EntityTypes, TArray<FMassEntitySpawnDataGeneratorResult>& OutResults) const
{
	float TotalProportion = 0.0f;
	for (const FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		TotalProportion += EntityType.Proportion;
	}

	if (TotalProportion <= 0)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Error, TEXT("The total combined proportion of all the entity types needs to be greater than 0."));
		return;
	}

	float CumulativeProportion = 0.0f;
	int TotalSpawned = 0;

	for (int32 i = 0; i < EntityTypes.Num(); i++)
	{
		const FMassSpawnedEntityType& EntityType = EntityTypes[i];

		CumulativeProportion += EntityType.Proportion;
		const int32 CumulativeEntityCount = FMath::RoundToInt(static_cast<float>(SpawnCount) * CumulativeProportion / TotalProportion);

		const int32 EntityCount = CumulativeEntityCount - TotalSpawned;

		if (EntityCount > 0 && EntityType.GetEntityConfig() != nullptr)
		{
			FMassEntitySpawnDataGeneratorResult& Res = OutResults.AddDefaulted_GetRef();
			Res.NumEntities = EntityCount;
			Res.EntityConfigIndex = i;
		}

		TotalSpawned = CumulativeEntityCount;
	}
}
