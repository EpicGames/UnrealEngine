// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTemplate.h"
#include "MassCommonTypes.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "MassDebugger.h"
#include "Algo/Find.h"
#include "MassEntityBuilder.h"

//----------------------------------------------------------------------//
//  FMassEntityTemplateID
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityTemplate)
FString FMassEntityTemplateID::ToString() const
{
	return IsValid() ? FString::Printf(TEXT("[%s:%d]"), *ConfigGuid.ToString(EGuidFormats::DigitsLower), FlavorHash) 
			: FString::Printf(TEXT("[Invalid]"));
}

//----------------------------------------------------------------------//
//  FMassEntityTemplateData
//----------------------------------------------------------------------//
FMassEntityTemplateData::FMassEntityTemplateData(const FMassEntityTemplate& InFinalizedTemplate)
	: FMassEntityTemplateData(InFinalizedTemplate.GetTemplateData())
{

}

bool FMassEntityTemplateData::SlowIsEquivalent(const FMassEntityTemplateData& Other) const
{
	if (Composition.IsEquivalent(Other.GetCompositionDescriptor()) == false)
	{
		return false;
	}
	else if (SharedFragmentValues.IsEquivalent(Other.GetSharedFragmentValues()) == false)
	{
		return false;
	}
	
	TConstArrayView<FInstancedStruct> OtherInitialFragmentValues = Other.GetInitialFragmentValues();
	
	if (OtherInitialFragmentValues.Num() != InitialFragmentValues.Num())
	{
		return false;
	}

	for (const FInstancedStruct& InitialValue : InitialFragmentValues)
	{
		const FInstancedStruct* FoundElement = Algo::FindByPredicate(OtherInitialFragmentValues, [&InitialValue](const FInstancedStruct& Element)
		{
			return InitialValue == Element;
		});

		if (FoundElement == nullptr)
		{
			return false;
		}
	}
	return true;
}

UE::Mass::FEntityBuilder FMassEntityTemplateData::CreateEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager) const
{
	UE::Mass::FEntityBuilder Builder = UE::Mass::FEntityBuilder::Make(InEntityManager, Composition
		, InitialFragmentValues
		, SharedFragmentValues.GetConstSharedFragments()
		, SharedFragmentValues.GetSharedFragments());

	Builder.ConfigureArchetypeCreation(CreationParams);

	ensureMsgf(ObjectInitializers.Num() == 0, TEXT("%hs: EntityBuilder doesn't support object initializers (yet)."), __FUNCTION__);

	return Builder;
}

uint32 GetTypeHash(const FMassEntityTemplateData& Template)
{
	// @todo: using the template name is temporary solution to allow to tell two templates apart based on something 
	// else than composition. Ideally we would hash the fragment values instead.
	const int32 NameHash = GetTypeHash(Template.GetTemplateName());
	const uint32 CompositionHash = Template.GetCompositionDescriptor().CalculateHash();
	// @todo shared fragments hash is based on pointers - this needs to be changed as well
	const uint32 SharedFragmentValuesHash = GetTypeHash(Template.GetSharedFragmentValues());
	
	uint32 InitialValuesHash = 0;
	// Initial fragment values, this is not part of the archetype as it is the spawner job to set them.
	for (const FInstancedStruct& Struct : Template.GetInitialFragmentValues())
	{
		InitialValuesHash = UE::StructUtils::GetStructInstanceCrc32(Struct, InitialValuesHash);
	}

	// These functions will be called to initialize entity's UObject-based fragments
	// @todo this is very bad for hash creation - no way to make this consistent between client server. Might need some "initializer index"
	uint32 InitializersHash = 0;
	/* left here on purpose, will address in following CLs
	for (const FMassEntityTemplateData::FObjectFragmentInitializerFunction& Initializer : Template.ObjectInitializers)
	{
		InitializersHash = HashCombine(InitializersHash, GetTypeHash(Initializer));
	}*/

	// @todo maybe better to have two separate hashes - one for composition and maybe shared fragments, and then the other one for the rest, and use multimap for look up
	return HashCombine(NameHash, HashCombine(HashCombine(CompositionHash, SharedFragmentValuesHash), HashCombine(InitialValuesHash, InitializersHash)));
}

//-----------------------------------------------------------------------------
// FMassEntityTemplate
//-----------------------------------------------------------------------------
TSharedRef<FMassEntityTemplate> FMassEntityTemplate::MakeFinalTemplate(FMassEntityManager& EntityManager, FMassEntityTemplateData&& TempTemplateData, FMassEntityTemplateID InTemplateID)
{
	return MakeShared<FMassEntityTemplate>(MoveTemp(TempTemplateData), EntityManager, InTemplateID);
}

FMassEntityTemplate::FMassEntityTemplate(const FMassEntityTemplateData& InData, FMassEntityManager& EntityManager, FMassEntityTemplateID InTemplateID)
	: TemplateData(InData)
	, TemplateID(InTemplateID)
{
	// Sort anything there is to sort for later comparison purposes
	TemplateData.Sort();

	TemplateData.GetArchetypeCreationParams().DebugName = FName(GetTemplateName());
	const FMassArchetypeHandle ArchetypeHandle = EntityManager.CreateArchetype(GetCompositionDescriptor(), TemplateData.GetArchetypeCreationParams());
	SetArchetype(ArchetypeHandle);
}

FMassEntityTemplate::FMassEntityTemplate(FMassEntityTemplateData&& InData, FMassEntityManager& EntityManager, FMassEntityTemplateID InTemplateID)
	: TemplateData(InData)
	, TemplateID(InTemplateID)
{
	// Sort anything there is to sort for later comparison purposes
	TemplateData.Sort();

	TemplateData.GetArchetypeCreationParams().DebugName = FName(GetTemplateName());
	const FMassArchetypeHandle ArchetypeHandle = EntityManager.CreateArchetype(GetCompositionDescriptor(), TemplateData.GetArchetypeCreationParams());
	SetArchetype(ArchetypeHandle);
}

void FMassEntityTemplate::SetArchetype(const FMassArchetypeHandle& InArchetype)
{
	check(InArchetype.IsValid());
	Archetype = InArchetype;
}

FString FMassEntityTemplate::DebugGetArchetypeDescription(FMassEntityManager& EntityManager) const
{
	FStringOutputDevice OutDescription;
#if WITH_MASSGAMEPLAY_DEBUG
	FMassDebugger::OutputArchetypeDescription(OutDescription, Archetype);
#endif // WITH_MASSGAMEPLAY_DEBUG
	return MoveTemp(OutDescription);
}

FString FMassEntityTemplate::DebugGetDescription(FMassEntityManager* EntityManager) const
{
	FStringOutputDevice Ar;
#if WITH_MASSGAMEPLAY_DEBUG
	Ar.SetAutoEmitLineTerminator(true);

	if (EntityManager)
	{
		Ar += TEXT("Archetype details:\n");
		Ar += DebugGetArchetypeDescription(*EntityManager);
	}
	else
	{
		Ar += TEXT("Composition:\n");
		GetCompositionDescriptor().DebugOutputDescription(Ar);
	}

#endif // WITH_MASSGAMEPLAY_DEBUG
	return MoveTemp(Ar);
}

UE::Mass::FEntityBuilder FMassEntityTemplate::CreateEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager) const
{
	return TemplateData.CreateEntityBuilder(InEntityManager);
}

//-----------------------------------------------------------------------------
// FMassEntityTemplateIDFactory
//-----------------------------------------------------------------------------
FMassEntityTemplateID FMassEntityTemplateIDFactory::Make(const FGuid& ConfigGuid)
{
	return FMassEntityTemplateID(ConfigGuid);
}

FMassEntityTemplateID FMassEntityTemplateIDFactory::MakeFlavor(const FMassEntityTemplateID& SourceTemplateID, const int32 Flavor)
{
	return FMassEntityTemplateID(SourceTemplateID.ConfigGuid, Flavor);
}
