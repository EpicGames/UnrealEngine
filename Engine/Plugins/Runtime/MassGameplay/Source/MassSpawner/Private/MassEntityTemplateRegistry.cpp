// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTemplateRegistry.h"
#include "MassCommonTypes.h"
#include "MassSpawnerTypes.h"
#include "MassEntityManager.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityTypes.h"
#include "MassEntityTraitBase.h"

#if WITH_EDITOR
#include "Editor.h"
#include "MassDebugger.h"
#include "MassEntityEditor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityTemplateRegistry)

#define LOCTEXT_NAMESPACE "Mass"

namespace UE::Mass::Debug
{
	const FName TraitFailedValidation(TEXT("TraitFailedValidation"));
	const FName TraitIgnored(TEXT("TraitIgnored"));
	const FName TraitOverridden(TEXT("TraitOverridden"));

	static bool bReportDuplicatedFragmentsAsWarnings = false;
	// anonymous namespace to force CVars's uniqueness - we use the same name in many places, sometimes withing same namespaces
	namespace 
	{
		FAutoConsoleVariableRef CVars[] =
		{
			{ TEXT("mass.template.DuplicateElementsAsWarnings")
				, bReportDuplicatedFragmentsAsWarnings
				, TEXT("Whether to report a detection of a given element type being added by multiple traits as a Warning. Otherise we print the information out as `Info`")
				, ECVF_Cheat}
		};
	}
}

//----------------------------------------------------------------------//
// FMassEntityTemplateRegistry 
//----------------------------------------------------------------------//
TMap<const UScriptStruct*, FMassEntityTemplateRegistry::FStructToTemplateBuilderDelegate> FMassEntityTemplateRegistry::StructBasedBuilders;

FMassEntityTemplateRegistry::FMassEntityTemplateRegistry(UObject* InOwner)
	: Owner(InOwner)
{
}

void FMassEntityTemplateRegistry::ShutDown()
{
	TemplateIDToTemplateMap.Reset();
	EntityManager = nullptr;
}

UWorld* FMassEntityTemplateRegistry::GetWorld() const 
{
	return Owner.IsValid() ? Owner->GetWorld() : nullptr;
}

FMassEntityTemplateRegistry::FStructToTemplateBuilderDelegate& FMassEntityTemplateRegistry::FindOrAdd(const UScriptStruct& DataType)
{
	return StructBasedBuilders.FindOrAdd(&DataType);
}

void FMassEntityTemplateRegistry::Initialize(const TSharedPtr<FMassEntityManager>& InEntityManager)
{
	if (EntityManager)
	{
		ensureMsgf(EntityManager == InEntityManager, TEXT("Attempting to store a different EntityManager then the previously stored one - this indicated a set up issue, attempting to use multiple EntityManager instances"));
		return;
	}

	EntityManager = InEntityManager;
}

void FMassEntityTemplateRegistry::DebugReset()
{
#if WITH_MASSGAMEPLAY_DEBUG
	TemplateIDToTemplateMap.Reset();
#endif // WITH_MASSGAMEPLAY_DEBUG
}

const TSharedRef<FMassEntityTemplate>* FMassEntityTemplateRegistry::FindTemplateFromTemplateID(FMassEntityTemplateID TemplateID) const
{
	return TemplateIDToTemplateMap.Find(TemplateID);
}

const TSharedRef<FMassEntityTemplate>& FMassEntityTemplateRegistry::FindOrAddTemplate(FMassEntityTemplateID TemplateID, FMassEntityTemplateData&& TemplateData)
{
	check(EntityManager);
	const TSharedRef<FMassEntityTemplate>* ExistingTemplate = FindTemplateFromTemplateID(TemplateID);
	if (ExistingTemplate != nullptr)
	{
		return *ExistingTemplate;
	}

	return TemplateIDToTemplateMap.Add(TemplateID, FMassEntityTemplate::MakeFinalTemplate(*EntityManager, MoveTemp(TemplateData), TemplateID));
}

void FMassEntityTemplateRegistry::DestroyTemplate(FMassEntityTemplateID TemplateID)
{
	TemplateIDToTemplateMap.Remove(TemplateID);
}

//----------------------------------------------------------------------//
// FMassEntityTemplateBuildContext 
//----------------------------------------------------------------------//
bool FMassEntityTemplateBuildContext::BuildFromTraits(TConstArrayView<UMassEntityTraitBase*> Traits, const UWorld& World)
{
	ensureMsgf(bBuildInProgress == false, TEXT("Unexpected occurrence - it suggests FMassEntityTemplateBuildContext::BuildFromTraits "
		"has been called as a consequence of some UMassEntityTraitBase::BuildTemplate call. Check the callstack."));

	const EProcessorExecutionFlags WorldProcessingFlags = World.IsGameWorld()
		? UE::Mass::Utils::DetermineProcessorExecutionFlags(&World)
		: EProcessorExecutionFlags::All;

	bBuildInProgress = true;
	for (const UMassEntityTraitBase* Trait : Traits)
	{
		check(Trait);
		if (static_cast<bool>(Trait->ValidTargetConfig & WorldProcessingFlags)
			&& SetTraitBeingProcessed(Trait))
		{
			Trait->BuildTemplate(*this, World);
		}
	}
	// now remove all that has been requested to be removed
	// those are only tags for now, thus the shortcut of going directly for tags
	for (FRemovedType& Removed : RemovedTypes)
	{
		check(Removed.TypeRemoved);
		TemplateData.RemoveTag(CastChecked<UScriptStruct>(Removed.TypeRemoved));
	}

	bBuildInProgress = false;

	const bool bTemplateValid = ValidateBuildContext(World);
	
	ResetBuildTimeData();

	return bTemplateValid;
}

bool FMassEntityTemplateBuildContext::SetTraitBeingProcessed(const UMassEntityTraitBase* Trait)
{
	if (Trait == nullptr || TraitsProcessed.Contains(Trait) == false)
	{
		TraitsData.Add({Trait});
		return true;
	}

	UE_LOGF(LogMass, Warning, "Attempting to add %ls to FMassEntityTemplateBuildContext while this or another instance of the trait class has already been added."
		, *GetNameSafe(Trait));

	IgnoredTraits.Add(Trait);
	return false;
}

bool FMassEntityTemplateBuildContext::ValidateBuildContext(const UWorld& World)
{
#if WITH_UNREAL_DEVELOPER_TOOLS && WITH_EDITOR && WITH_EDITORONLY_DATA && WITH_MASSENTITY_DEBUG
#define IF_MESSAGES(Message) if (GEditor) { Message }
#else
#define IF_MESSAGES(_)
#endif

	int32 ErrorCount = 0;
	int32 WarningCount = 0;
	int32 InfosCount = 0;

	// Doing the trait-specific validation first since it can add required elements to the build context
	for (FTraitData& TraitData : TraitsData)
	{
		UMassEntityTraitBase::FAdditionalTraitRequirements TraitRequirementsWrapper(TraitData.TypesRequired);
		if (LIKELY(TraitData.Trait) && TraitData.Trait->ValidateTemplate(*this, World, TraitRequirementsWrapper) == false)
		{
			++ErrorCount;
			IF_MESSAGES(
				FMassDebugger::DebugEvent(UE::Mass::Debug::TraitFailedValidation, FConstStructView::Make(FMassGenericDebugEvent{TraitData.Trait}));
			);
		}
	}

	TMap<const UStruct*, const UMassEntityTraitBase*> TypesAlreadyAdded;

	// these are non-critical warnings, we want to report these to the users as a potential configuration issue,
	// but it won't affect the final entity template composition (for example adding the same fragment is fine since 
	// the entity template handles that gracefully).
	for (const FTraitData& TraitData : TraitsData)
	{
		for (const UStruct* TypeAdded : TraitData.TypesAdded)
		{
			const UMassEntityTraitBase*& SourceTrait = TypesAlreadyAdded.FindOrAdd(TypeAdded);
			if (SourceTrait != nullptr)
			{
				if (UE::Mass::Debug::bReportDuplicatedFragmentsAsWarnings)
				{
					// we report this only if it wasn't added twice by the same trait, the one we're processing right now
					UE_CLOGF(SourceTrait != TraitData.Trait
						, LogMass, Warning, "%ls: Fragment %ls already added by %ls. Check the entity config for conflicting traits"
						, *GetNameSafe(TraitData.Trait), *GetNameSafe(TypeAdded), *SourceTrait->GetName());
					++WarningCount;
				}
				IF_MESSAGES(
					FMassDebugger::DebugEvent(FMassDuplicateElementsMessage::StaticStruct()->GetFName()
						, FConstStructView::Make(FMassDuplicateElementsMessage{TraitData.Trait, SourceTrait, TypeAdded})
						, UE::Mass::Debug::bReportDuplicatedFragmentsAsWarnings ? EMassDebugMessageSeverity::Warning : EMassDebugMessageSeverity::Info);
				);
			}
			else
			{
				SourceTrait = TraitData.Trait;
			}
		}
	}

	// now to properly test if something required was removed we need to filter TypesAlreadyAdded first
	for (const FRemovedType& RemovedElement : RemovedTypes)
	{
		if (RemovedElement.TypeRemoved)
		{
			TypesAlreadyAdded.Remove(RemovedElement.TypeRemoved);
		}
	}

	// these are critical, we're going to fail the validation if anything here fails
	for (const FTraitData& TraitData : TraitsData)
	{
		for (const UStruct* TypeRequired : TraitData.TypesRequired)
		{
			if (TypesAlreadyAdded.Contains(TypeRequired) == false)
			{
				UE_LOGF(LogMass, Error, "%ls: Missing required element of type %ls"
					, *GetNameSafe(TraitData.Trait), *GetNameSafe(TypeRequired));
				++ErrorCount;
				IF_MESSAGES(
				{
					// check if it was removed
					const UMassEntityTraitBase* RemovedByTrait = nullptr;
					const int32 RemoverIndex = RemovedTypes.Find(FRemovedType({TypeRequired}));
					if (RemoverIndex != INDEX_NONE)
					{
						RemovedByTrait = RemovedTypes[RemoverIndex].Remover;
					}

					FMassDebugger::DebugEvent<FMassMissingTraitMessage>(TraitData.Trait, TypeRequired, RemovedByTrait);
				});
			}
		}
	}

	for (const UMassEntityTraitBase* IgnoredTrait : IgnoredTraits)
	{
		IF_MESSAGES(
			FMassDebugger::DebugEvent(UE::Mass::Debug::TraitIgnored, FConstStructView::Make(FMassGenericDebugEvent{IgnoredTrait}));
		);
		++WarningCount;
	}

	for (const UMassEntityTraitBase* Trait : OverriddenTraits)
	{
		IF_MESSAGES(
			FMassDebugger::DebugEvent(UE::Mass::Debug::TraitOverridden, FConstStructView::Make(FMassGenericDebugEvent{Trait}));
		);
		++InfosCount;
	}
	
	// @todo add dependencies on trait classes? might be hard if traits are unrelated, like requiring UMassLODCollectorTrait 
	// or UMassDistanceLODCollectorTrait - both supply alternative implementations of a given functionality, but are unrelated.
	// Could be done with a complex requirements system (similar to entity queries - "all of X", "any of Y", etc) - probably 
	// not worth it since we don't even have a use case for it right now.

#if WITH_UNREAL_DEVELOPER_TOOLS && WITH_EDITOR
	if (GEditor && (ErrorCount || WarningCount || InfosCount))
	{
		FMassEditorNotification Notification;
		Notification.Message = FText::FormatOrdered(LOCTEXT("TraitResult", "Mass Entity Template validation:\n{0} errors and {1} warnings found. {2} notes."), ErrorCount, WarningCount, InfosCount);
		Notification.Severity = ErrorCount ? EMessageSeverity::Error : EMessageSeverity::Warning;
		Notification.bIncludeSeeOutputLogForDetails = true;
		Notification.Show();
	}
#endif // WITH_UNREAL_DEVELOPER_TOOLS && WITH_EDITOR

#undef IF_MESSAGES

	// only the Errors render the template invalid, Warnings just warn about stuff not being set up quite right, but we can recover.
	return (ErrorCount == 0);
}

#undef LOCTEXT_NAMESPACE 
