// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTraitBase.h"

//----------------------------------------------------------------------//
//  UMassEntityTraitBase
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityTraitBase)

bool UMassEntityTraitBase::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	return true;
}

#if WITH_EDITOR
UMassEntityTraitBase::FOnNewTraitType UMassEntityTraitBase::OnNewTraitTypeEvent;

void UMassEntityTraitBase::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		OnNewTraitTypeEvent.Broadcast(*this);
	}
}

#endif // WITH_EDITOR
