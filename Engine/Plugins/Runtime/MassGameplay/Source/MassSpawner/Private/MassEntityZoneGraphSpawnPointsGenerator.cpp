// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityZoneGraphSpawnPointsGenerator.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphQuery.h"
#include "MassSpawnerTypes.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "MassGameplaySettings.h"
#include "MassSpawnLocationProcessor.h"
#include "Engine/World.h"
#include "MassCommonUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityZoneGraphSpawnPointsGenerator)

void UMassEntityZoneGraphSpawnPointsGenerator::Generate(UObject& QueryOwner, TConstArrayView<FMassSpawnedEntityType> EntityTypes, int32 Count, FFinishedGeneratingSpawnDataSignature& FinishedGeneratingSpawnPointsDelegate) const
{
	if (Count <= 0)
	{
		FinishedGeneratingSpawnPointsDelegate.Execute(TArray<FMassEntitySpawnDataGeneratorResult>());
		return;
	}
	
	const UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(QueryOwner.GetWorld());
	if (ZoneGraph == nullptr)
	{
		UE_VLOG_UELOG(&QueryOwner, LogMassSpawner, Error, TEXT("No zone graph subsystem found in world"));
		return;
	}

	TArray<FVector> Locations;
	
	const FRandomStream RandomStream(UE::Mass::Utils::OverrideRandomSeedForTesting(GetRandomSelectionSeed()));
	const TConstArrayView<FRegisteredZoneGraphData> RegisteredZoneGraphs = ZoneGraph->GetRegisteredZoneGraphData();
	if (RegisteredZoneGraphs.IsEmpty())
	{
		UE_VLOG_UELOG(&QueryOwner, LogMassSpawner, Error, TEXT("No zone graphs found"));
		return;
	}

	for (const FRegisteredZoneGraphData& Registered : RegisteredZoneGraphs)
	{
		if (Registered.bInUse && Registered.ZoneGraphData)
		{
			GeneratePointsForZoneGraphData(*Registered.ZoneGraphData, Locations, RandomStream);
		}
	}

	if (Locations.IsEmpty())
	{
		UE_VLOG_UELOG(&QueryOwner, LogMassSpawner, Error, TEXT("No locations found on zone graphs"));
		return;
	}

	// Randomize them
	for (int32 I = 0; I < Locations.Num(); ++I)
	{
		const int32 J = RandomStream.RandHelper(Locations.Num());
		Locations.Swap(I, J);
	}

	// If we generated too many, shrink it.
	if (Locations.Num() > Count)
	{
		Locations.SetNum(Count);
	}

	// Build array of entity types to spawn.
	TArray<FMassEntitySpawnDataGeneratorResult> Results;
	BuildResultsFromEntityTypes(Count, EntityTypes, Results);

	const int32 LocationCount = Locations.Num();
	int32 LocationIndex = 0;

	// Distribute points amongst the entities to spawn.
	for (FMassEntitySpawnDataGeneratorResult& Result : Results)
	{
		// @todo: Make separate processors and pass the ZoneGraph locations directly.
		Result.SpawnDataProcessor = UMassSpawnLocationProcessor::StaticClass();
		Result.SpawnData.InitializeAs<FMassTransformsSpawnData>();
		FMassTransformsSpawnData& Transforms = Result.SpawnData.GetMutable<FMassTransformsSpawnData>();

		Transforms.Transforms.Reserve(Result.NumEntities);
		for (int i = 0; i < Result.NumEntities; i++)
		{
			FTransform& Transform = Transforms.Transforms.AddDefaulted_GetRef();
			Transform.SetLocation(Locations[LocationIndex % LocationCount]);
			LocationIndex++;
		}
	}

#if ENABLE_VISUAL_LOG
	UE_VLOG(this, LogMassSpawner, Log, TEXT("Spawning at %d locations"), LocationIndex);
	if (GetDefault<UMassGameplaySettings>()->bLogSpawnLocations)
	{
		FVisualLogger::Get().ExecuteOnLastEntryForObject(this, [LocationIndex, &Results](FVisualLogEntry& LogEntry)
		{
			FVisualLogShapeElement Element(TEXT(""), FColor::Orange, /*Thickness*/20, LogMassSpawner.GetCategoryName());

			Element.Points.Reserve(LocationIndex);
			for (const FMassEntitySpawnDataGeneratorResult& Result : Results)
			{
				const FMassTransformsSpawnData& Transforms = Result.SpawnData.Get<FMassTransformsSpawnData>();
				for (int i = 0; i < Result.NumEntities; i++)
				{
					Element.Points.Add(Transforms.Transforms[i].GetLocation());
				}
			}
			
			Element.Type = EVisualLoggerShapeElement::SinglePoint;
			Element.Verbosity = ELogVerbosity::Display;
			LogEntry.AddElement(Element);
		});
	}
#endif // ENABLE_VISUAL_LOG

	FinishedGeneratingSpawnPointsDelegate.Execute(Results);
}

void UMassEntityZoneGraphSpawnPointsGenerator::GeneratePointsForZoneGraphData(const ::AZoneGraphData& ZoneGraphData, TArray<FVector>& Locations, const FRandomStream& RandomStream) const
{
	// Avoid an infinite loop.
	if (MinGap == 0.0f && MaxGap == 0.0f)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Error, TEXT("You cannot set both Min Gap and Max Gap to 0.0f"));
		return;						
	}

	const FZoneGraphStorage &ZoneGraphStorage = ZoneGraphData.GetStorage();
	
	// Loop through all lanes
	for (int32 LaneIndex = 0; LaneIndex < ZoneGraphStorage.Lanes.Num(); ++LaneIndex)
	{
		const FZoneLaneData& Lane = ZoneGraphStorage.Lanes[LaneIndex];
		const float LaneHalfWidth = Lane.Width / 2.0f;
		if (TagFilter.Pass(Lane.Tags))
		{
			float LaneLength = 0.0f;
			UE::ZoneGraph::Query::GetLaneLength(ZoneGraphStorage, LaneIndex, LaneLength);

			float Distance = static_cast<float>(RandomStream.FRandRange(MinGap, MaxGap)); // ..initially
			while (Distance <= LaneLength)
			{
				// Add location at the center of this space.
				FZoneGraphLaneLocation LaneLocation;
				UE::ZoneGraph::Query::CalculateLocationAlongLane(ZoneGraphStorage, LaneIndex, Distance, LaneLocation);
				const FVector Perp = LaneLocation.Direction ^ LaneLocation.Up;
				Locations.Add(LaneLocation.Position + Perp * RandomStream.FRandRange(-LaneHalfWidth, LaneHalfWidth));

				// Advance ahead past the space we just consumed, plus a random gap.
				Distance += static_cast<float>(RandomStream.FRandRange(MinGap, MaxGap));
			}
		}
	}
}
