// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpawnLocationProcessor.h"
#include "MassCommonFragments.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassSpawnerTypes.h"
#include "MassCommonUtils.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"

//----------------------------------------------------------------------//
// UMassSpawnLocationProcessor 
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpawnLocationProcessor)
UMassSpawnLocationProcessor::UMassSpawnLocationProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = false;
	RandomStream.Initialize(UE::Mass::Utils::GenerateRandomSeed());
}

void UMassSpawnLocationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassSpawnLocationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	if (!ensure(ExecutionContext.ValidateAuxDataType<FMassTransformsSpawnData>()))
	{
		UE_VLOG_UELOG(this, LogMass, Log, TEXT("Execution context has invalid AuxData or it's not FMassSpawnAuxData. Entity transforms won't be initialized."));
		return;
	}

	const UWorld* World = EntityManager.GetWorld();
	check(World);

	const ENetMode NetMode = World->GetNetMode();

	FMassTransformsSpawnData& AuxData = ExecutionContext.GetMutableAuxData().GetMutable<FMassTransformsSpawnData>();
	TArray<FTransform>& Transforms = AuxData.Transforms;

	const int32 NumSpawnTransforms = Transforms.Num();
	if (NumSpawnTransforms == 0)
	{
		UE_VLOG_UELOG(this, LogMass, Error, TEXT("No spawn transforms provided. Entity transforms won't be initialized."));
		return;
	}

	int32 NumRequiredSpawnTransforms = 0;
	EntityQuery.ForEachEntityChunk(ExecutionContext, [&NumRequiredSpawnTransforms](const FMassExecutionContext& Context)
		{
			NumRequiredSpawnTransforms += Context.GetNumEntities();
		});

	const int32 NumToAdd = NumRequiredSpawnTransforms - NumSpawnTransforms;
	if (NumToAdd > 0)
	{
		UE_VLOG_UELOG(this, LogMass, Warning,
			TEXT("Not enough spawn locations provided (%d) for all entities (%d). Existing locations will be reused randomly to fill the %d missing positions."),
			NumSpawnTransforms, NumRequiredSpawnTransforms, NumToAdd);

		Transforms.AddUninitialized(NumToAdd);
		for (int EntityIndex = 0; EntityIndex < NumToAdd; ++EntityIndex)
		{
			Transforms[NumSpawnTransforms + EntityIndex] = Transforms[RandomStream.RandRange(0, NumSpawnTransforms - 1)];
		}
	}

	if (AuxData.bRandomize && !UE::Mass::Utils::IsDeterministic())
	{
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&Transforms, this](FMassExecutionContext& Context)
			{
				const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const int32 AuxIndex = RandomStream.RandRange(0, Transforms.Num() - 1);
					LocationList[EntityIt].GetMutableTransform() = Transforms[AuxIndex];
					Transforms.RemoveAtSwap(AuxIndex, EAllowShrinking::No);
				}
			});
	}
	else
	{
		int32 NextTransformIndex = 0;
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&Transforms, &NextTransformIndex, this](FMassExecutionContext& Context)
			{
				const int32 NumEntities = Context.GetNumEntities();
				TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
				check(NextTransformIndex + NumEntities <= Transforms.Num());
	
				FMemory::Memcpy(LocationList.GetData(), &Transforms[NextTransformIndex], NumEntities * LocationList.GetTypeSize());
				NextTransformIndex += NumEntities;
			});
	}
}
