// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpawner.h"

#include "Components/BillboardComponent.h"
#include "Components/SceneComponent.h"
#include "Engine/AssetManager.h"
#include "Engine/Engine.h"
#include "Engine/StreamableManager.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "MassEntityConfigAsset.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassEntityUtils.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"
#include "MassSimulationSubsystem.h"
#include "MassSpawnerInternalTypes.h"
#include "MassSpawnerSubsystem.h"
#include "MassSpawnerTypes.h"
#include "UObject/ConstructorHelpers.h"
#include "VisualLogger/VisualLogger.h"
#if WITH_EDITOR
#include "Engine/Texture2D.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpawner)


namespace UE::MassSpawner
{
	float ScalabilitySpawnDensityMultiplier = 1.f;
	FAutoConsoleVariableRef CVarScalabilitySpawnDensityMultiplier(TEXT("ai.mass.scalability.SpawnDensityMultiplier"), ScalabilitySpawnDensityMultiplier, TEXT("Spawn Density Multiplier, must be set before Mass Spawn Init"), ECVF_Scalability);

#if WITH_EDITOR
	static FAutoConsoleCommandWithWorld ForceSpawningCommand(
	TEXT("ai.mass.ForceSpawn"),
	TEXT("Command to Force Spawn all mass entities generated by MassSpawners"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
	{
		for (TActorIterator<AActor> It(World, AMassSpawner::StaticClass()); It; ++It)
		{
			if (AMassSpawner* Spawner = Cast<AMassSpawner>(*It))
			{
				Spawner->DoSpawning();
			}
		}
	}));

	static FAutoConsoleCommandWithWorld ForceDespawningCommand(
	TEXT("ai.mass.ForceDespawn"),
	TEXT("Command to Force Despawn all mass entities generated by MassSpawners"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
	{
		for (TActorIterator<AActor> It(World, AMassSpawner::StaticClass()); It; ++It)
		{
			if (AMassSpawner* Spawner = Cast<AMassSpawner>(*It))
			{
				Spawner->DoDespawning();
			}
		}
	}));

	static FAutoConsoleCommandWithWorld ResetSpawningCommand(
	TEXT("ai.mass.ResetSpawning"),
	TEXT("Command to Force Despawn and Respawn all mass entities generated by MassSpawners"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
	{
		for (TActorIterator<AActor> It(World, AMassSpawner::StaticClass()); It; ++It)
		{
			if (AMassSpawner* Spawner = Cast<AMassSpawner>(*It))
			{
				Spawner->DoDespawning();
				Spawner->DoSpawning();
			}
		}
	}));
#endif // WITH_EDITOR
}

AMassSpawner::AMassSpawner()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITOR
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	// SpriteComponent can be null for editor builds running "-game|server"
	if (!IsRunningCommandlet() && SpriteComponent != nullptr)
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> IconTextureObject;
			FName MassSpawnerID;
			FText MassSpawnerName;
			FConstructorStatics()
				: IconTextureObject(TEXT("/MassGameplay/S_MassCrowd"))
				, MassSpawnerID(TEXT("MassSpawner"))
				, MassSpawnerName(NSLOCTEXT("SpriteCategory", "MassSpawner", "MassSpawner"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;
		
		SpriteComponent->Sprite = ConstructorStatics.IconTextureObject.Get();
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.MassSpawnerID;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.MassSpawnerName;
		SpriteComponent->SetupAttachment(RootComponent);
		SpriteComponent->Mobility = EComponentMobility::Static;
	}
#endif // WITH_EDITOR

	SetCanBeDamaged(false);

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	SetActorHiddenInGame(true);
#endif

	bAutoSpawnOnBeginPlay = true;
	bOverrideSchematics = false;
}

void AMassSpawner::PostLoad()
{
	Super::PostLoad();

	for (FMassSpawnDataGenerator& SpawnPointsGenerator : SpawnDataGenerators)
	{
		if (SpawnPointsGenerator.GeneratorClass)
		{
			SpawnPointsGenerator.GeneratorInstance = NewObject<UMassEntitySpawnDataGeneratorBase>(this, SpawnPointsGenerator.GeneratorClass);
			SpawnPointsGenerator.GeneratorClass = nullptr;
			MarkPackageDirty();
		}
	}
}

void AMassSpawner::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UWorld* World = GetWorld();
		check(World);


		// This is a temp fix for streaming levels async loading MassSpawners after UMassSpawnerSubsystem::OnPostWorldInit, 
		// in the long run we are going to need a better system for making sure all the entity templates are registered
		// on the clients before replication of Agents occurs. This is only required to be done for clients.
		if (GEngine->GetNetMode(GetWorld()) == NM_Client)
		{
			UMassSpawnerSubsystem* MassSpawnerSubsystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(World);
			if (MassSpawnerSubsystem)
			{
				RegisterEntityTemplates();
			}
			else
			{
				FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &AMassSpawner::OnPostWorldInit);
			}
		}
	}
}

void AMassSpawner::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World == GetWorld())
	{
		UMassSpawnerSubsystem* MassSpawnerSubsystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(World);
		check(MassSpawnerSubsystem);

		RegisterEntityTemplates();

		FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitDelegateHandle);
	}
}

void AMassSpawner::BeginDestroy()
{
	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitDelegateHandle);

	DoDespawning();
	
	if (StreamingHandle.IsValid() && StreamingHandle->IsActive())
	{
		StreamingHandle->CancelHandle();
	}

	Super::BeginDestroy();
}

void AMassSpawner::BeginPlay()
{
	check(GEngine);

	Super::BeginPlay();

	const ENetMode NetMode = GEngine->GetNetMode(GetWorld());

	if (bAutoSpawnOnBeginPlay && NetMode != NM_Client)
	{
		const UMassSimulationSubsystem* MassSimulationSubsystem = UWorld::GetSubsystem<UMassSimulationSubsystem>(GetWorld());
		if (MassSimulationSubsystem == nullptr || MassSimulationSubsystem->IsSimulationStarted())
		{
			DoSpawning();
		}
		else
		{
			
			SimulationStartedHandle = UMassSimulationSubsystem::GetOnSimulationStarted().AddLambda([this](UWorld* InWorld)
				{
					UWorld* World = GetWorld();

					if (World == InWorld)
					{
						DoSpawning();
					}
				});
		}
	}
}

void AMassSpawner::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	UMassSimulationSubsystem::GetOnSimulationStarted().Remove(SimulationStartedHandle);

	if (StreamingHandle.IsValid() && StreamingHandle->IsActive())
	{
		StreamingHandle->CancelHandle();
	}

	DoDespawning();

	Super::EndPlay(EndPlayReason);
}

#if WITH_EDITOR
void AMassSpawner::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName EntityTypesName = GET_MEMBER_NAME_CHECKED(AMassSpawner, EntityTypes);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == EntityTypesName)
		{
			// TODO: Should optimize this, i.e. set a dirty flag and update only when needed.
			UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(GetWorld());
			if (SpawnerSystem)
			{
				RegisterEntityTemplates();
			}
		}
	}
}

void AMassSpawner::DEBUG_Spawn()
{
	DoSpawning();
}

void AMassSpawner::DEBUG_Clear()
{
	DoDespawning();
}

#endif // WITH_EDITOR

void AMassSpawner::RegisterEntityTemplates()
{
	UWorld* World = GetWorld();
	check(World);
	for (FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		if (const UMassEntityConfigAsset* EntityConfig = EntityType.GetEntityConfig())
		{
			EntityConfig->GetOrCreateEntityTemplate(*World);
		}
	}
}

void AMassSpawner::DoSpawning()
{
	// no spawn point generators configured. Let user know and fall back to the spawner's location
	if (SpawnDataGenerators.Num() == 0)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Warning, TEXT("No Spawn Data Generators configured."));
		return;
	}
	
	if (EntityTypes.Num() == 0)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Warning, TEXT("No EntityTypes configured."));
		return;
	}

	AllGeneratedResults.Reset();
	
	float TotalProportion = 0.0f;
	for (FMassSpawnDataGenerator& Generator : SpawnDataGenerators)
	{
		if (Generator.GeneratorInstance)
		{
			Generator.bDataGenerated = false;
			TotalProportion += Generator.Proportion;
		}
	}

	if (TotalProportion <= 0.0f)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Error, TEXT("The total combined proportion of all the generator needs to be greater than 0."));
		return;
	}
	
	// Check if it needs loading
	if (StreamingHandle.IsValid() && StreamingHandle->IsActive())
	{
		// @todo, instead of blindly canceling, we should remember what was asked to load with that handle and compare if more is needed?
		StreamingHandle->CancelHandle();
	}
	TArray<FSoftObjectPath> AssetsToLoad;
	for (const FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		if (!EntityType.IsLoaded())
		{
			AssetsToLoad.Add(EntityType.EntityConfig.ToSoftObjectPath());
		}
	}
	
	const int32 TotalSpawnCount = GetSpawnCount();
	
	auto GenerateSpawningPoints = [this, TotalSpawnCount, TotalProportion]()
	{
		int32 SpawnCountRemaining = TotalSpawnCount;
		float ProportionRemaining = TotalProportion;
		for (FMassSpawnDataGenerator& Generator : SpawnDataGenerators)
		{
			if (Generator.Proportion == 0.0f || ProportionRemaining <= 0.0f)
			{
				// If there's nothing to spawn, mark the generator done as OnSpawnDataGenerationFinished() will wait for all generators to complete before the actual spawning.
				Generator.bDataGenerated = true;
				continue;
			}

			if (Generator.GeneratorInstance)
			{
				const float ProportionRatio = FMath::Min(Generator.Proportion / ProportionRemaining, 1.0f);
				const int32 SpawnCount = FMath::CeilToInt(static_cast<float>(SpawnCountRemaining) * ProportionRatio);
				
				FFinishedGeneratingSpawnDataSignature Delegate = FFinishedGeneratingSpawnDataSignature::CreateUObject(this, &AMassSpawner::OnSpawnDataGenerationFinished, &Generator);
				Generator.GeneratorInstance->Generate(*this, EntityTypes, SpawnCount, Delegate);
				SpawnCountRemaining -= SpawnCount;
				ProportionRemaining -= Generator.Proportion;
			}
		}
	};
	
	if (AssetsToLoad.Num())
	{
		FStreamableManager& StreamableManager = UAssetManager::GetStreamableManager();
		StreamingHandle = StreamableManager.RequestAsyncLoad(AssetsToLoad, FStreamableDelegate::CreateWeakLambda(this, GenerateSpawningPoints));
	}
	else
	{
		GenerateSpawningPoints();
	}
}

void AMassSpawner::OnSpawnDataGenerationFinished(TConstArrayView<FMassEntitySpawnDataGeneratorResult> Results, FMassSpawnDataGenerator* FinishedGenerator)
{
	if (UWorld* World = GetWorld())
	{
		if (World->bIsTearingDown || World->IsCleanedUp())
		{
			return;
		}
	}

	// @todo: this can be potentially expensive copy for the instanced structs, could there be a way to use move here instead?
	AllGeneratedResults.Append(Results.GetData(), Results.Num());

	bool bAllSpawnPointsGenerated = true;
	bool bFoundFinishedGenerator = false;
	for (FMassSpawnDataGenerator& Generator : SpawnDataGenerators)
	{
		if (&Generator == FinishedGenerator)
		{
			Generator.bDataGenerated = true;
			bFoundFinishedGenerator = true;
		}

		bAllSpawnPointsGenerated &= Generator.bDataGenerated;
	}

	checkf(bFoundFinishedGenerator, TEXT("Something went wrong, we are receiving a callback on an unknow spawn point generator"));
	
	if (bAllSpawnPointsGenerated)
	{
		SpawnGeneratedEntities(AllGeneratedResults);
		AllGeneratedResults.Reset();
	}
}

int32 AMassSpawner::GetSpawnCount() const
{
	const float FinalSpawningCountScale = SpawningCountScale * UE::MassSpawner::ScalabilitySpawnDensityMultiplier;
	return static_cast<int32>(FinalSpawningCountScale * static_cast<float>(Count));
}

UMassProcessor* AMassSpawner::GetPostSpawnProcessor(TSubclassOf<UMassProcessor> ProcessorClass)
{
	if (!ProcessorClass)
	{
		return nullptr;
	}

	TObjectPtr<UMassProcessor>* const Initializer = PostSpawnProcessors.FindByPredicate([ProcessorClass](const UMassProcessor* Processor)
		{
			return Processor && Processor->GetClass() == ProcessorClass;
		}
	);

	if (Initializer)
	{
		return *Initializer;
	}

	UMassProcessor* NewInitializer = NewObject<UMassProcessor>(this, ProcessorClass);
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(this);
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		NewInitializer->CallInitialize(this, EntityManager->AsShared());
		PostSpawnProcessors.Add(NewInitializer);
		return NewInitializer;
	}

	return nullptr;
}

void AMassSpawner::SpawnGeneratedEntities(TConstArrayView<FMassEntitySpawnDataGeneratorResult> Results)
{
	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(GetWorld());
	if (SpawnerSystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassSpawner, Error, TEXT("UMassSpawnerSubsystem missing while trying to spawn entities"));
		return;
	}

#if WITH_EDITOR
	for (FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		if (UMassEntityConfigAsset* EntityConfig = EntityType.GetEntityConfig())
		{
			EntityConfig->AddTrait(UMassCreatedBySpawnerTrait::StaticClass());
		}
	}
#endif

	UWorld* World = GetWorld();
	check(World);

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystemChecked<UMassEntitySubsystem>();
	FMassEntityManager&   EntityManager   = EntitySubsystem->GetMutableEntityManager();

	int32 TotalNum = 0;
	const int32 StartIndex = AllSpawnedEntities.Num();

	for (const FMassEntitySpawnDataGeneratorResult& Result : Results)
	{
		if (Result.NumEntities <= 0)
		{
			continue;
		}
		
		check(EntityTypes.IsValidIndex(Result.EntityConfigIndex));
		check(Result.SpawnDataProcessor != nullptr);
		
		const FMassSpawnedEntityType& EntityType = EntityTypes[Result.EntityConfigIndex];

		if (const UMassEntityConfigAsset* EntityConfig = EntityType.GetEntityConfig())
		{
			const FMassEntityTemplate& EntityTemplate = EntityConfig->GetOrCreateEntityTemplate(*World);
			if (EntityTemplate.IsValid())
			{
				FSpawnedEntities& SpawnedEntities = AllSpawnedEntities.AddDefaulted_GetRef();
				SpawnedEntities.TemplateID = EntityTemplate.GetTemplateID();
				SpawnerSystem->SpawnEntities(EntityTemplate.GetTemplateID(), Result.NumEntities, Result.SpawnData, Result.SpawnDataProcessor, SpawnedEntities.Entities);
				TotalNum += SpawnedEntities.Entities.Num();

#if WITH_EDITOR
				for (FMassEntityHandle Entity : SpawnedEntities.Entities)
				{
					if (FMassCreatedBySpawner* CreatedBy = EntityManager.GetFragmentDataPtr<FMassCreatedBySpawner>(Entity))
					{
						CreatedBy->Spawner = this;
					}
				}
#endif
			}
		}
	}

	// Run post spawn processors only on the freshly spawned entities.
	if (TotalNum)
	{
		TArray<UMassProcessor*> Processors;
		TSet<TSubclassOf<UMassProcessor>> AddedProcessorClasses;

		for (const FMassEntitySpawnDataGeneratorResult& Result : Results)
		{
			for (const TSubclassOf<UMassProcessor>& ProcessorClass : Result.PostSpawnProcessors)
			{
				if (AddedProcessorClasses.Contains(ProcessorClass) == false)
				{
					if (UMassProcessor* Processor = GetPostSpawnProcessor(ProcessorClass))
					{
						Processors.Add(Processor);
					}
					AddedProcessorClasses.Add(ProcessorClass);
				}
			}
		}

		if (Processors.Num() > 0)
		{
			FMassProcessingContext ProcessingContext(EntityManager, /*TimeDelta=*/0.0f);

			// gather freshly spawned entities
			TArray<FMassEntityHandle> AllEntities;
			AllEntities.Reserve(TotalNum);
			
			for (int32 Index = StartIndex; Index < AllSpawnedEntities.Num(); ++Index)
			{
				AllEntities.Append(AllSpawnedEntities[Index].Entities);
			}

			// create entity collections and run Processors on them. 
			TArray<FMassArchetypeEntityCollection> EntityCollections;
			UE::Mass::Utils::CreateEntityCollections(EntityManager, AllEntities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
			UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext, EntityCollections);
		}
	}

	OnSpawningFinishedEvent.Broadcast();
}

void AMassSpawner::DoDespawning()
{
	if (AllSpawnedEntities.IsEmpty())
	{
		return;
	}
	
	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(GetWorld());
	if (SpawnerSystem == nullptr)
	{
		UE_LOGF(LogMassSpawner, Error, "UMassSpawnerSubsystem missing while trying to despawn entities");
		return;
	}

	for (const FSpawnedEntities& SpawnedEntities : AllSpawnedEntities)
	{
		SpawnerSystem->DestroyEntities(SpawnedEntities.Entities);
	}
	AllSpawnedEntities.Reset();

	OnDespawningFinishedEvent.Broadcast();
}

void AMassSpawner::DoDespawning(TConstArrayView<FMassEntityHandle> EntitiesToIgnore)
{
	// Remove EntitiesToIgnore from SpawnedEntities so they get skipped by DoDespawning() and add to AllEntitiesToKeep
	// to restore after.
	TArray<FSpawnedEntities> AllEntitiesToKeep;
	for (FSpawnedEntities& SpawnedEntities : AllSpawnedEntities)
	{
		FSpawnedEntities* EntitiesToKeep = nullptr;
		for (const FMassEntityHandle& EntityToIgnore : EntitiesToIgnore)
		{
			if (SpawnedEntities.Entities.RemoveSingleSwap(EntityToIgnore, EAllowShrinking::No))
			{
				if (!EntitiesToKeep)
				{
					EntitiesToKeep = &AllEntitiesToKeep.AddDefaulted_GetRef();
					EntitiesToKeep->TemplateID = SpawnedEntities.TemplateID;
				}
				
				EntitiesToKeep->Entities.Add(EntityToIgnore);
			}
		}
	}

	// Despawn the remaining entities in AllSpawnedEntities
	DoDespawning();

	// Restore AllEntitiesToKeep to AllSpawnedEntities so they remain tracked
	AllSpawnedEntities = AllEntitiesToKeep;
}

bool AMassSpawner::DespawnEntity(const FMassEntityHandle Entity)
{
	UMassSpawnerSubsystem* SpawnerSystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(GetWorld());
	if (SpawnerSystem == nullptr)
	{
		UE_LOGF(LogMassSpawner, Error, "UMassSpawnerSubsystem missing while trying to despawn a single entity");
		return false;
	}

	for (FSpawnedEntities& SpawnedEntities : AllSpawnedEntities)
	{
		const int32 Index = SpawnedEntities.Entities.Find(Entity);
		if (Index != INDEX_NONE)
		{
			SpawnerSystem->DestroyEntities(MakeArrayView(&Entity, 1));
			SpawnedEntities.Entities.RemoveAtSwap(Index, EAllowShrinking::No);
			return true;
		}
	}
	return false;
}

int32 AMassSpawner::GetCount() const
{
	return Count;
}

float AMassSpawner::GetSpawningCountScale() const
{
	return SpawningCountScale;
}

void AMassSpawner::ClearTemplates()
{
	UWorld* World = GetWorld();
	check(World);

	for (FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		if (const UMassEntityConfigAsset* EntityConfig = EntityType.GetEntityConfig())
		{
			EntityConfig->DestroyEntityTemplate(*World);
		}
	}
}

void AMassSpawner::UnloadConfig()
{
	// Clear all templates that were created by the config
	ClearTemplates();
	for (FMassSpawnedEntityType& EntityType : EntityTypes)
	{
		EntityType.UnloadEntityConfig();
	}

	if (StreamingHandle.IsValid() && StreamingHandle->IsActive())
	{
		StreamingHandle->CancelHandle();
	}
}
