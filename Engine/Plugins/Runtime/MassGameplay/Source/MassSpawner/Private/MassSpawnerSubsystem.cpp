// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpawnerSubsystem.h"
#include "MassSpawnerTypes.h"
#include "MassEntityTemplate.h"
#include "MassEntityManager.h"
#include "MassEntityTemplateRegistry.h"
#include "Engine/World.h"
#include "MassExecutor.h"
#include "StructUtils/InstancedStruct.h"
#include "VisualLogger/VisualLogger.h"
#include "MassSpawner.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "MassSimulationSubsystem.h"
#include "MassProcessor.h"
#include "MassEntityUtils.h"
#include "MassProcessingContext.h"
#include "MassObserverNotificationTypes.h"

//----------------------------------------------------------------------//
//  UMassSpawnerSubsystem
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpawnerSubsystem)
UMassSpawnerSubsystem::UMassSpawnerSubsystem()
	: TemplateRegistryInstance(this)
{

}

void UMassSpawnerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{	
	Super::Initialize(Collection);

	// making sure UMassSimulationSubsystem gets created before the MassSpawnerSubsystem, since UMassSimulationSubsystem
	// is where the EntityManager gets created for the runtime MassGameplay simulation
	Collection.InitializeDependency<UMassSimulationSubsystem>();

	UWorld* World = GetWorld();
	check(World);
	EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World).AsShared();
	TemplateRegistryInstance.Initialize(EntityManager);
}

void UMassSpawnerSubsystem::Deinitialize() 
{
	EntityManager.Reset();
	TemplateRegistryInstance.ShutDown();

	Super::Deinitialize();
}

TSharedPtr<FMassEntityManager::FEntityCreationContext> UMassSpawnerSubsystem::SpawnEntities(const FMassEntityTemplate& EntityTemplate, const uint32 NumberToSpawn, TArray<FMassEntityHandle>& OutEntities)
{
	check(EntityManager);
	check(EntityTemplate.IsValid());

	if (NumberToSpawn == 0)
	{
		UE_VLOG(this, LogMassSpawner, Warning, TEXT("Trying to spawn 0 entities. This would cause inefficiency. Bailing out with result FALSE."));
		return {};
	}

	return DoSpawning(EntityTemplate, NumberToSpawn, FStructView(), TSubclassOf<UMassProcessor>(), OutEntities);
}

TSharedPtr<FMassEntityManager::FEntityCreationContext> UMassSpawnerSubsystem::SpawnEntities(FMassEntityTemplateID TemplateID, const uint32 NumberToSpawn, FConstStructView SpawnData, TSubclassOf<UMassProcessor> InitializerClass, TArray<FMassEntityHandle>& OutEntities)
{
	check(TemplateID.IsValid());

	const TSharedRef<FMassEntityTemplate>* EntityTemplate = TemplateRegistryInstance.FindTemplateFromTemplateID(TemplateID);
	checkf(EntityTemplate, TEXT("SpawnEntities: TemplateID must have been registered!"));

	return DoSpawning(EntityTemplate->Get(), NumberToSpawn, SpawnData, InitializerClass, OutEntities);
}

void UMassSpawnerSubsystem::DestroyEntities(TConstArrayView<FMassEntityHandle> Entities)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassSpawnerSubsystem_DestroyEntities")

	check(EntityManager);
	checkf(!EntityManager->IsProcessing()
		, TEXT("%s called while MassEntity processing in progress. This is unsupported and dangerous!"), ANSI_TO_TCHAR(__FUNCTION__));

	UWorld* World = GetWorld();
	check(World);


	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
	EntityManager->BatchDestroyEntityChunks(EntityCollections);
}

UMassProcessor* UMassSpawnerSubsystem::GetSpawnDataInitializer(TSubclassOf<UMassProcessor> InitializerClass)
{	
	if (!InitializerClass || !EntityManager)
	{
		return nullptr;
	}

	TObjectPtr<UMassProcessor>* const Initializer = SpawnDataInitializers.FindByPredicate([InitializerClass](const UMassProcessor* Processor)
		{
			return Processor && Processor->GetClass() == InitializerClass;
		}
	);

	if (Initializer == nullptr)
	{
		UMassProcessor* NewInitializer = NewObject<UMassProcessor>(this, InitializerClass);
		NewInitializer->CallInitialize(this, EntityManager.ToSharedRef());
		SpawnDataInitializers.Add(NewInitializer);
		return NewInitializer;
	}

	return *Initializer;
}

TSharedPtr<FMassEntityManager::FEntityCreationContext> UMassSpawnerSubsystem::DoSpawning(const FMassEntityTemplate& EntityTemplate, const int32 NumToSpawn, FConstStructView SpawnData, TSubclassOf<UMassProcessor> InitializerClass, TArray<FMassEntityHandle>& OutEntities)
{
	check(EntityManager);
	check(EntityTemplate.GetArchetype().IsValid());
	UE_VLOG(this, LogMassSpawner, Log, TEXT("Spawning with EntityTemplate:\n%s"), *EntityTemplate.DebugGetDescription(EntityManager.Get()));

	if (NumToSpawn <= 0)
	{
		UE_VLOG(this, LogMassSpawner, Warning, TEXT("%s: Trying to spawn %d entities. Ignoring."), ANSI_TO_TCHAR(__FUNCTION__), NumToSpawn);
		return {};
	}

	LLM_SCOPE_BYNAME(TEXT("Mass/Spawner"))
	//TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassSpawnerSubsystem DoSpawning");

	// 1. Create required number of entities with EntityTemplate.Archetype
	TArray<FMassEntityHandle> SpawnedEntities;
	TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext
		= EntityManager->BatchCreateEntities(EntityTemplate.GetArchetype(), EntityTemplate.GetSharedFragmentValues(), NumToSpawn, SpawnedEntities);

	if (SpawnedEntities.Num())
	{
		// 2. Copy data from FMassEntityTemplate.Fragments.
		//		a. @todo, could be done as part of creation?
		TConstArrayView<FInstancedStruct> FragmentInstances = EntityTemplate.GetInitialFragmentValues();

		// note: using UE::Mass::Utils::CreateEntityCollections rather than CreationContext->GetEntityCollections since the creation context could have been 
		// created outside of this function and reused in a loop to call DoSpawning, in which case CreationContext->GetEntityCollections
		// would contain more than just the freshly created entities
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager, SpawnedEntities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchSetEntityFragmentValues(EntityCollections, FragmentInstances);

		// 3. Run SpawnDataInitializer, if set. This is a special type of processor that operates on the entities to initialize them.
		// e.g., will run UInstancedActorsInitializerProcessor for Mass InstancedActors
		UMassProcessor* SpawnDataInitializer = SpawnData.IsValid()
			? GetSpawnDataInitializer(InitializerClass)
			: nullptr;

		if (SpawnDataInitializer)
		{
			FMassProcessingContext ProcessingContext(EntityManager, /*TimeDelta=*/0.0f, /*bInFlushCommandBuffer=*/false);
			ProcessingContext.AuxData = SpawnData;
			UE::Mass::Executor::RunProcessorsView(MakeArrayView(&SpawnDataInitializer, 1), ProcessingContext, EntityCollections);
		}

		OutEntities.Append(MoveTemp(SpawnedEntities));

		// 4. "OnEntitiesCreated" notifies will be sent out once the CreationContext gets destroyed (via its destructor).
		// The caller can postpone this moment keeping the returned CreationContext alive as long as needed.
	}

	return CreationContext;
}

const FMassEntityTemplate* UMassSpawnerSubsystem::GetMassEntityTemplate(FMassEntityTemplateID TemplateID) const
{
	check(TemplateID.IsValid());
	const TSharedRef<FMassEntityTemplate>* TemplateFound = TemplateRegistryInstance.FindTemplateFromTemplateID(TemplateID);
	return TemplateFound ? &TemplateFound->Get() : nullptr;
}
