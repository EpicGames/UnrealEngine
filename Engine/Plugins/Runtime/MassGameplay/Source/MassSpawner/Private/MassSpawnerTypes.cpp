// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSpawnerTypes.h"
#include "MassEntityConfigAsset.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "MassEntityTraitBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSpawnerTypes)

namespace UE::MassSpawner
{
	uint32 HashTraits(TConstArrayView<UMassEntityTraitBase*> CombinedTraits)
	{
		class FArchiveObjectCRC32AgentConfig : public FArchiveObjectCrc32
		{
		public:
			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				check(InProperty);
				return FArchiveObjectCrc32::ShouldSkipProperty(InProperty) || InProperty->HasAllPropertyFlags(CPF_Transient);
			}
		};

		uint32 CRC = 0;
		for (UMassEntityTraitBase* Trait : CombinedTraits)
		{
			FArchiveObjectCRC32AgentConfig Archive;
			CRC = Archive.Crc32(Trait, CRC);
			// @todo this piece is here to avoid an easy to repro hash class - all one needs to do is to add a 
			// trait subclass that sets different default values
			check(Trait && Trait->GetClass());
			CRC = HashCombine(CRC, GetTypeHash(Trait->GetClass()->GetName()));
		}
		return CRC;
	}
} // UE::MassSpawner

//-----------------------------------------------------------------------------
// FMassSpawnedEntityType
//-----------------------------------------------------------------------------
const UMassEntityConfigAsset* FMassSpawnedEntityType::GetEntityConfig() const
{
	if (EntityConfigPtr == nullptr)
	{
		EntityConfigPtr = EntityConfig.LoadSynchronous();
	}
	return EntityConfigPtr;
}

UMassEntityConfigAsset* FMassSpawnedEntityType::GetEntityConfig()
{
	if (EntityConfigPtr == nullptr)
	{
		EntityConfigPtr = EntityConfig.LoadSynchronous();
	}
	return EntityConfigPtr;
}
