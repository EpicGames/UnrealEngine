// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassTranslator.h"
#include "MassCommonTypes.h"
#include "MassEntityQuery.h"

//----------------------------------------------------------------------//
//  UMassTranslator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassTranslator)
UMassTranslator::UMassTranslator()
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = (int32)EProcessorExecutionFlags::All;
}

void UMassTranslator::AddRequiredTagsToQuery(FMassEntityQuery& EntityQuery)
{
	EntityQuery.AddTagRequirements<EMassFragmentPresence::All>(RequiredTags);
}
