// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTraitBase.h"
#include "StructUtils/InstancedStruct.h"
#include "MassAssortedFragmentsTrait.generated.h"

/**
* Mass Agent Feature which appends a list of specified fragments.  
*/
UCLASS(MinimalAPI, meta=(DisplayName="Assorted Fragments"))
class UMassAssortedFragmentsTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:

	MASSSPAWNER_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category="Mass", EditAnywhere, meta = (BaseStruct = "/Script/MassEntity.MassFragment", ExcludeBaseStruct))
	TArray<FInstancedStruct> Fragments;

	UPROPERTY(Category="Mass", EditAnywhere, meta = (BaseStruct = "/Script/MassEntity.MassTag", ExcludeBaseStruct))
	TArray<FInstancedStruct> Tags;
};
