// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Engine/DataAsset.h"
#include "MassEntityTemplate.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTemplateRegistry.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityConfigAsset.generated.h"

class UMassEntityConfigAsset;
class UMassEntityTraitBase;
struct FMassEntityTemplateBuildContext;

/**
 * Describes a Mass agent to spawn. The struct can be embedded to allow last minute changes to the agent (i.e. for debugging).
 * The agent config describes a unique list of features which are used to create an entity template.
 * Derived configs can override parent features.
 */
USTRUCT()
struct FMassEntityConfig
{
	friend class UMassTemplateRegistry;

	GENERATED_BODY()

	MASSSPAWNER_API FMassEntityConfig();
	MASSSPAWNER_API FMassEntityConfig(UObject& InOwner);

	/** Create entity template based on the features included in this config.
	 *  @param World in which we are creating the template.
	 */
	MASSSPAWNER_API const FMassEntityTemplate& GetOrCreateEntityTemplate(const UWorld& World) const;

	MASSSPAWNER_API void DestroyEntityTemplate(const UWorld& World) const;

	/** 
	 * Fetches the EntityTemplate for given World, fails a check if one cannot be found.
	 */
	MASSSPAWNER_API const FMassEntityTemplate& GetEntityTemplateChecked(const UWorld& World) const;

	/** @return Parent config */
	const UMassEntityConfigAsset* GetParent() const { return Parent; }

	void SetParentAsset(const UMassEntityConfigAsset& InParent) { Parent = &InParent; }
	
	/** @return View to the array of features defined on this config */
	TConstArrayView<UMassEntityTraitBase*> GetTraits() const { return Traits; }

	/** Looks for a trait of the indicated type, accepting all child classes as well, unless bExactMatch == true */
	const UMassEntityTraitBase* FindTrait(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch = false) const
	{
		return FindTraitInternal(TraitClass, bExactMatch);
	}

	/** Adds Trait to the collection of traits hosted by this FMassEntityConfig instance */
	MASSSPAWNER_API void AddTrait(UMassEntityTraitBase& Trait);

	/** Validates if the entity template is well built */
	MASSSPAWNER_API bool ValidateEntityTemplate(const UWorld& World);

	void SetOwner(UObject& InOwner) { ConfigOwner = &InOwner; }

	bool IsEmpty() const { return Parent == nullptr && Traits.Num() == 0; }

	const FGuid& GetGuid() const { return ConfigGuid; }
	
#if WITH_EDITOR
	/** Needs to be called when the given config is being duplicated - ensured the ConfigGuid remains unique */
	MASSSPAWNER_API void PostDuplicate(bool bDuplicateForPIE);
	MASSSPAWNER_API void ForceRegenerateGUID();

	MASSSPAWNER_API UMassEntityTraitBase* FindMutableTrait(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch = false);
#endif // WITH_EDITOR
	
protected:
	/**
	 * Combines traits based on the config hierarchy and returns list of unique traits
	 * @param BuildContext if provided, will be used to store additional information for reporting back to the user
	 */
	MASSSPAWNER_API void GetCombinedTraits(TArray<UMassEntityTraitBase*>& OutTraits, FMassEntityTemplateBuildContext* BuildContext = nullptr) const;

	/** Looks for a trait of the indicated type, accepting all child classes as well, unless bExactMatch == true.
	 *  Walks the parent chain iteratively and detects cycles - logs an error and returns nullptr if a parent is reached twice. */
	MASSSPAWNER_API UMassEntityTraitBase* FindTraitInternal(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch = false) const;

	/** Reference to parent config asset */
	UPROPERTY(Category = "Derived Traits", EditAnywhere)
	TObjectPtr<const UMassEntityConfigAsset> Parent = nullptr;

	/** Array of unique traits of this config */
	UPROPERTY(Category = "Traits", EditAnywhere, Instanced)
	TArray<TObjectPtr<UMassEntityTraitBase>> Traits;

	UPROPERTY(transient)
	TObjectPtr<UObject> ConfigOwner = nullptr;

private:
	UPROPERTY(VisibleAnywhere, Category="Mass", meta = (IgnoreForMemberInitializationTest))
	FGuid ConfigGuid;

private:
	MASSSPAWNER_API void GetCombinedTraitsInternal(TArray<UMassEntityTraitBase*>& OutTraits, TArray<const UObject*>& Visited, FMassEntityTemplateBuildContext* BuildContext) const;
	MASSSPAWNER_API const FMassEntityTemplate* GetEntityTemplateInternal(const UWorld& World, FMassEntityTemplateID& TemplateIDOut) const;
};

/**
 * Agent Config asset allows to create shared configs that can be used as base for derived configs.
 * The asset can be used as is i.e. on a spawner, or you can use FMassEntityConfig to allow last minute changes at use site.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMassEntityConfigAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	UMassEntityConfigAsset()
		: Config(*this)
	{}

	/** @return Agent config stored in this asset */
	const FMassEntityConfig& GetConfig() const { return Config; }

	/** @return Mutable agent config stored in this asset */
	FMassEntityConfig& GetMutableConfig() { return Config; }

	const FMassEntityTemplate& GetOrCreateEntityTemplate(const UWorld& World) const
	{
		return Config.GetOrCreateEntityTemplate(World);
	}

	void DestroyEntityTemplate(const UWorld& World) const
	{
		Config.DestroyEntityTemplate(World);
	}

	/** Looks for a trait of the indicated type, accepting all child classes as well, unless bExactMatch == true */
	const UMassEntityTraitBase* FindTrait(TSubclassOf<UMassEntityTraitBase> TraitClass, const bool bExactMatch = false) const
	{
		return Config.FindTrait(TraitClass, bExactMatch);
	}

#if WITH_EDITOR
	/** Called upon asset's dupllication. Ensured the underlying config's ConfigGuid remains unique */
	MASSSPAWNER_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	UFUNCTION(CallInEditor, Category = "Entity Config")
	MASSSPAWNER_API void ValidateEntityConfig();

	/**
	 * Returns a mutable instance of given trait class. If an instance of the given class can be found in
	 * the hosted FMassEntityConfig then that instance is returned. Otherwise one will be created, added to
	 * the FMassEntityConfig and returned.
	 */
	MASSSPAWNER_API UMassEntityTraitBase* AddTrait(TSubclassOf<UMassEntityTraitBase> TraitClass);
#endif // WITH_EDITOR

protected:
	/** The config described in this asset. */
	UPROPERTY(Category = "Entity Config", EditAnywhere)
	FMassEntityConfig Config;
};
