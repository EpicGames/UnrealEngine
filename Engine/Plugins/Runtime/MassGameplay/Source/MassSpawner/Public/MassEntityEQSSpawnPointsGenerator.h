// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntitySpawnDataGeneratorBase.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "MassEntityEQSSpawnPointsGenerator.generated.h"

/**
 * Describes the SpawnPoints Generator when we want to leverage the points given by an EQS Query
 */
UCLASS(MinimalAPI, BlueprintType, meta=(DisplayName="EQS SpawnPoints Generator"))
class UMassEntityEQSSpawnPointsGenerator : public UMassEntitySpawnDataGeneratorBase
{	
	GENERATED_BODY()

public:
	MASSSPAWNER_API UMassEntityEQSSpawnPointsGenerator();
	
	MASSSPAWNER_API virtual void Generate(UObject& QueryOwner, TConstArrayView<FMassSpawnedEntityType> EntityTypes, int32 Count, FFinishedGeneratingSpawnDataSignature& FinishedGeneratingSpawnPointsDelegate) const override;

protected:

#if WITH_EDITOR
	MASSSPAWNER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	MASSSPAWNER_API void OnEQSQueryFinished(TSharedPtr<FEnvQueryResult> EQSResult, TArray<FMassEntitySpawnDataGeneratorResult> Results, FFinishedGeneratingSpawnDataSignature FinishedGeneratingSpawnPointsDelegate) const;

	UPROPERTY(Category = "Query", EditAnywhere)
	FEQSParametrizedQueryExecutionRequest EQSRequest;
};
