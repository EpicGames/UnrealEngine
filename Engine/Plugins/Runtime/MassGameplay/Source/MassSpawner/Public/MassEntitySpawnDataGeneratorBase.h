// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/InstancedStruct.h"
#include "MassSpawnerTypes.h"
#include "MassEntitySpawnDataGeneratorBase.generated.h"

class UMassEntityConfigAsset;
class UMassProcessor;

/**
 * The result of the spawn point generator.
 */
USTRUCT()
struct FMassEntitySpawnDataGeneratorResult
{
	GENERATED_BODY()

	// Spawn data that is passed to the InitSpawnDataProcessor. E.g. the data could contain array of locations, one for each entity.
	UPROPERTY()
	FInstancedStruct SpawnData;
	
	// Processor that understands how to apply SpawnData to the spawned entities. 
	UPROPERTY()
	TSubclassOf<UMassProcessor> SpawnDataProcessor;

	// Processors that are run for all entities after they entities for a MassSpawner are initialized.
	UPROPERTY()
	TArray<TSubclassOf<UMassProcessor>> PostSpawnProcessors;

	// Index in the EntityTypes array passed to Generate().
	UPROPERTY()
	int32 EntityConfigIndex = INDEX_NONE;

	// Number of entities to spawn.
	UPROPERTY()
	int32 NumEntities = 0;
};


DECLARE_DELEGATE_OneParam(FFinishedGeneratingSpawnDataSignature, TConstArrayView<FMassEntitySpawnDataGeneratorResult>);

/**
 * Base class for Mass Entity Spawn Points Generator.
 * A Mass Spawn Points Generator can be of several type (EQS, ZoneGraph, Volume, Area, etc.)
 * The concept is to override the GenerateSpawnPoints() method and requesting a certain number of Spawn Point Locations to the method.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew)
class UMassEntitySpawnDataGeneratorBase : public UObject
{
	GENERATED_BODY()

public:

	/** Generate "Count" number of SpawnPoints and return as a list of position
	 * @param Count of point to generate
	 * @param FinishedGeneratingSpawnPointsDelegate is the callback to call once the generation is done
	 */
	virtual void Generate(UObject& QueryOwner, TConstArrayView<FMassSpawnedEntityType> EntityTypes, int32 Count, FFinishedGeneratingSpawnDataSignature& FinishedGeneratingSpawnPointsDelegate) const PURE_VIRTUAL(UMassEntitySpawnDataGeneratorBase::GenerateSpawnPoints, );

	/** Populates empty generator results from EntityTypes based on the provided proportions. 
	 * @param SpawnCount How many entities to distributed among the EntityTypes
	 * @param EntityTypes Types of entities to generate data for.
	 * @param OutResults Generator result for each entity type that had > 0 entities assigned to it.
	 */
	MASSSPAWNER_API void BuildResultsFromEntityTypes(const int32 SpawnCount, TConstArrayView<FMassSpawnedEntityType> EntityTypes, TArray<FMassEntitySpawnDataGeneratorResult>& OutResults) const;

protected:
	int32 GetRandomSelectionSeed() const { return bUseCustomSeed ? RandomSelectionSeed : GFrameNumber; }

	UPROPERTY(Category = "Query", EditAnywhere, meta=(EditCondition="bUseCustomSeed"))
	int32 RandomSelectionSeed = 0;

	UPROPERTY()
	uint8 bUseCustomSeed : 1;
};
