// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassTranslator.h"
#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"
#include "Hash/CityHash.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#include "MassProcessingTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTemplate.generated.h"


class UMassEntityTraitBase;
struct FMassEntityView;
struct FMassEntityTemplateIDFactory;
struct FMassEntityTemplate;
struct FMassEntityManager;
namespace UE::Mass
{
	struct FEntityBuilder;
}

//ID of the template an entity is using
USTRUCT()
struct FMassEntityTemplateID
{
	GENERATED_BODY()

	FMassEntityTemplateID() 
		: FlavorHash(0), TotalHash(InvalidHash)
	{}

private:
	friend FMassEntityTemplateIDFactory;
	// use FMassEntityTemplateIDFactory to access this constructor flavor
	explicit FMassEntityTemplateID(const FGuid& InGuid, const int32 InFlavorHash = 0)
		: ConfigGuid(InGuid), FlavorHash(InFlavorHash)
	{
		 const uint64 GuidHash = CityHash64((char*)&ConfigGuid, sizeof(FGuid));
		 TotalHash = CityHash128to64({GuidHash, (uint64)InFlavorHash});
	}

public:
	uint64 GetHash64() const 
	{
		return TotalHash;
	}
	
	void Invalidate()
	{
		TotalHash = InvalidHash;
	}

	bool operator==(const FMassEntityTemplateID& Other) const
	{
		return (TotalHash == Other.TotalHash);
	}
	
	bool operator!=(const FMassEntityTemplateID& Other) const
	{
		return !(*this == Other);
	}

	/** 
	 * Note that since the function is 32-hashing a 64-bit value it's not guaranteed to produce globally unique values.
	 * But also note that it's still fine to use FMassEntityTemplateID as a TMap key type, since TMap is using 32bit hash
	 * to assign buckets rather than identify individual values.
	 */
	friend uint32 GetTypeHash(const FMassEntityTemplateID& TemplateID)
	{
		return GetTypeHash(TemplateID.TotalHash);
	}

	bool IsValid() const
	{
		return (TotalHash != InvalidHash);
	}

	MASSSPAWNER_API FString ToString() const;

protected:
	UPROPERTY(VisibleAnywhere, Category="Mass")
	FGuid ConfigGuid;

	UPROPERTY()
	uint32 FlavorHash;

	UPROPERTY()
	uint64 TotalHash;

private:
	static constexpr uint64 InvalidHash = 0;
};


/** 
 * Serves as data used to define and build finalized FMassEntityTemplate instances. Describes composition and initial
 * values of fragments for entities created with this data, and lets users modify and extend the data. Once finalized as 
 * FMassEntityTemplate the data will become immutable. 
 */
USTRUCT()
struct FMassEntityTemplateData
{
	GENERATED_BODY()

	typedef TFunction<void(UObject& /*Owner*/, FMassEntityView& /*EntityView*/, const EMassTranslationDirection /*CurrentDirection*/)> FObjectFragmentInitializerFunction;

	FMassEntityTemplateData() = default;
	MASSSPAWNER_API explicit FMassEntityTemplateData(const FMassEntityTemplate& InFinalizedTemplate);

	bool IsEmpty() const;

	TConstArrayView<FObjectFragmentInitializerFunction> GetObjectFragmentInitializers() const;
	const FString& GetTemplateName() const;
	const FMassArchetypeCompositionDescriptor& GetCompositionDescriptor() const;
	const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues() const;
	TConstArrayView<FInstancedStruct> GetInitialFragmentValues() const;

	TArray<FMassEntityTemplateData::FObjectFragmentInitializerFunction>& GetMutableObjectFragmentInitializers();

	void SetTemplateName(const FString& Name);
	
	template<typename T>
	void AddFragment()
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		Composition.GetElementsBitSet().Add<T>();
	}

	void AddFragment(TNotNull<const UScriptStruct*> FragmentType)

	{
		checkf(UE::Mass::IsA<FMassFragment>(FragmentType), TEXT(MASS_INVALID_FRAGMENT_MSG_F), *FragmentType->GetName());
		Composition.GetElementsBitSet().Add(FragmentType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void AddFragment(const UScriptStruct& FragmentType)
	{
		AddFragment(TNotNull<const UScriptStruct*>(&FragmentType));
	}

	// @todo this function is doing nothing if a given fragment's initial value has already been created. This seems inconsistent with the other AddFragment functions (especially AddFragment_GetRef).
	void AddFragment(FConstStructView Fragment)
	{
		const UScriptStruct* FragmentType = Fragment.GetScriptStruct();
		checkf(UE::Mass::IsA<FMassFragment>(FragmentType), TEXT(MASS_INVALID_FRAGMENT_MSG_F), *GetNameSafe(FragmentType));

		if (!Composition.GetElementsBitSet().Contains(FragmentType))
		{
			Composition.GetElementsBitSet().Add(FragmentType);
			InitialFragmentValues.Emplace(Fragment);
		}
		else if (!InitialFragmentValues.ContainsByPredicate(FStructTypeEqualOperator(FragmentType)))
		{
			InitialFragmentValues.Emplace(Fragment);
		}
	}

	template<typename T>
	T& AddFragment_GetRef()
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		if (!Composition.GetElementsBitSet().Contains<T>())
		{
			Composition.GetElementsBitSet().Add<T>();
		}
		else if (FInstancedStruct* Fragment = InitialFragmentValues.FindByPredicate(FStructTypeEqualOperator(T::StaticStruct())))
		{
			return Fragment->template GetMutable<T>();
		}

		// Add a default initial fragment value
		return InitialFragmentValues.Emplace_GetRef(T::StaticStruct()).template GetMutable<T>();
	}

	template<typename T>
	T* GetMutableFragment()
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		FInstancedStruct* Fragment = InitialFragmentValues.FindByPredicate(FStructTypeEqualOperator(T::StaticStruct()));
		return Fragment ? &Fragment->template GetMutable<T>() : (T*)nullptr;
	}

	template<typename T>
	void AddTag()
	{
		static_assert(UE::Mass::CTag<T>, "Given struct doesn't represent a valid mass tag type. Make sure to inherit from FMassTag or one of its child-types.");
		Composition.GetElementsBitSet().Add<T>();
	}
	
	void AddTag(TNotNull<const UScriptStruct*> TagType)
	{
		checkf(UE::Mass::IsA<FMassTag>(TagType), TEXT("Given struct doesn't represent a valid mass tag type. Make sure to inherit from FMassTag or one of its child-types."));
		Composition.GetElementsBitSet().Add(TagType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void AddTag(const UScriptStruct& TagType)
	{
		AddTag(&TagType);
	}

	template<typename T>
	void RemoveTag()
	{
		static_assert(UE::Mass::CTag<T>, "Given struct doesn't represent a valid mass tag type. Make sure to inherit from FMassTag or one of its child-types.");
		Composition.GetElementsBitSet().Remove<T>();
	}

	void RemoveTag(TNotNull<const UScriptStruct*> TagType)
	{
		checkf(UE::Mass::IsA<FMassTag>(TagType), TEXT("Given struct doesn't represent a valid mass tag type. Make sure to inherit from FMassTag or one of its child-types."));
		Composition.GetElementsBitSet().Remove(TagType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void RemoveTag(const UScriptStruct& TagType)
	{
		RemoveTag(&TagType);
	}

	FMassTagBitSet GetTags() const;
	FMassElementBitSet& GetMutableTags();

	template<typename T>
	void AddChunkFragment()
	{
		static_assert(UE::Mass::CChunkFragment<T>, "Given struct doesn't represent a valid chunk fragment type. Make sure to inherit from FMassChunkFragment or one of its child-types.");
		Composition.GetElementsBitSet().Add<T>();
	}

	void AddConstSharedFragment(const FConstSharedStruct& SharedFragment)
	{
		const UScriptStruct* FragmentType = SharedFragment.GetScriptStruct();
		if(ensureMsgf(UE::Mass::IsA<FMassConstSharedFragment>(FragmentType), TEXT("Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.")))
		{
			if (!Composition.GetElementsBitSet().Contains(FragmentType))
			{
				Composition.GetElementsBitSet().Add(FragmentType);
				SharedFragmentValues.Add(SharedFragment);
			}
#if DO_ENSURE
			else
			{
				const FConstSharedStruct* Struct = SharedFragmentValues.GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(SharedFragment));
				ensureMsgf(Struct && *Struct == SharedFragment, TEXT("Adding 2 different const shared fragment of the same type is not allowed"));

			}
#endif // DO_ENSURE
		}
	}

	void AddSharedFragment(const FSharedStruct& SharedFragment)
	{
		const UScriptStruct* FragmentType = SharedFragment.GetScriptStruct();
		if(ensureMsgf(UE::Mass::IsA<FMassSharedFragment>(FragmentType), TEXT("Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.")))
		{
			if (!Composition.GetElementsBitSet().Contains(FragmentType))
			{
				Composition.GetElementsBitSet().Add(FragmentType);
				SharedFragmentValues.Add(SharedFragment);
			}
	#if DO_ENSURE
			else
			{
				const FSharedStruct* Struct = SharedFragmentValues.GetSharedFragments().FindByPredicate(FStructTypeEqualOperator(SharedFragment));
				ensureMsgf(Struct && *Struct == SharedFragment, TEXT("Adding 2 different shared fragment of the same type is not allowed"));

			}
	#endif // DO_ENSURE
		}
	}

	template<UE::Mass::CFragment T>
	bool HasFragment() const
	{
		return Composition.GetElementsBitSet().Contains<T>();
	}
	
	bool HasFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		check(UE::Mass::IsA<FMassFragment>(ScriptStruct))
		return Composition.GetElementsBitSet().Contains(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasFragment(const UScriptStruct& ScriptStruct)
	{
		HasFragment(&ScriptStruct);
	}

	template<UE::Mass::CTag T>
	bool HasTag() const
	{
		return Composition.GetElementsBitSet().Contains<T>();
	}

	template<UE::Mass::CChunkFragment T>
	bool HasChunkFragment() const
	{
		return Composition.GetElementsBitSet().Contains<T>();
	}

	template<UE::Mass::CSharedFragment T>
	bool HasSharedFragment() const
	{
		return Composition.GetElementsBitSet().Contains<T>();
	}

	bool HasSharedFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		check(UE::Mass::IsA<FMassSharedFragment>(ScriptStruct))
		return Composition.GetElementsBitSet().Contains(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasSharedFragment(const UScriptStruct& ScriptStruct)
	{
		HasSharedFragment(&ScriptStruct);
	}

	template<UE::Mass::CConstSharedFragment T>
	bool HasConstSharedFragment() const
	{
		return Composition.GetElementsBitSet().Contains<T>();
	}

	bool HasConstSharedFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		check(UE::Mass::IsA<FMassConstSharedFragment>(ScriptStruct))
		return Composition.GetElementsBitSet().Contains(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasConstSharedFragment(const UScriptStruct& ScriptStruct)
	{
		HasConstSharedFragment(&ScriptStruct);
	}

	void Sort()
	{
		SharedFragmentValues.Sort();
	}

	/** Compares contents of two archetypes (this and Other). Returns whether both are equivalent.
	 *  @Note that the function can be slow, depending on how elaborate the template is. This function is meant for debugging purposes. */
	MASSSPAWNER_API bool SlowIsEquivalent(const FMassEntityTemplateData& Other) const;

	FMassArchetypeCreationParams& GetArchetypeCreationParams();

	MASSSPAWNER_API UE::Mass::FEntityBuilder CreateEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager) const;
	
protected:
	FMassArchetypeCompositionDescriptor Composition;
	FMassArchetypeSharedFragmentValues SharedFragmentValues;

	// Initial fragment values, this is not part of the archetype as it is the spawner job to set them.
	TArray<FInstancedStruct> InitialFragmentValues;

	// These functions will be called to initialize entity's UObject-based fragments
	TArray<FObjectFragmentInitializerFunction> ObjectInitializers;

	FMassArchetypeCreationParams CreationParams;

	FString TemplateName;
};

/**
 * A finalized and const wrapper for FMassEntityTemplateData, associated with a Mass archetype and template ID. 
 * Designed to never be changed. If a change is needed a copy of the hosted FMassEntityTemplateData needs to be made and 
 * used to create another finalized FMassEntityTemplate (via FMassEntityTemplateManager).
 */
struct FMassEntityTemplate final : public TSharedFromThis<FMassEntityTemplate> 
{
	friend TSharedFromThis<FMassEntityTemplate>;

	FMassEntityTemplate() = default;
	MASSSPAWNER_API FMassEntityTemplate(const FMassEntityTemplateData& InData, FMassEntityManager& EntityManager, FMassEntityTemplateID InTemplateID);
	MASSSPAWNER_API FMassEntityTemplate(FMassEntityTemplateData&& InData, FMassEntityManager& EntityManager, FMassEntityTemplateID InTemplateID);

	/** InArchetype is expected to be valid. The function will crash-check it. */
	MASSSPAWNER_API void SetArchetype(const FMassArchetypeHandle& InArchetype);
	const FMassArchetypeHandle& GetArchetype() const;

	bool IsValid() const;

	void SetTemplateID(FMassEntityTemplateID InTemplateID);
	FMassEntityTemplateID GetTemplateID() const;

	MASSSPAWNER_API FString DebugGetDescription(FMassEntityManager* EntityManager = nullptr) const;
	MASSSPAWNER_API FString DebugGetArchetypeDescription(FMassEntityManager& EntityManager) const;

	static MASSSPAWNER_API TSharedRef<FMassEntityTemplate> MakeFinalTemplate(FMassEntityManager& EntityManager, FMassEntityTemplateData&& TempTemplateData, FMassEntityTemplateID InTemplateID);

	//-----------------------------------------------------------------------------
	// FMassEntityTemplateData getters
	//-----------------------------------------------------------------------------
	TConstArrayView<FMassEntityTemplateData::FObjectFragmentInitializerFunction> GetObjectFragmentInitializers() const;
	const FString& GetTemplateName() const;
	const FMassArchetypeCompositionDescriptor& GetCompositionDescriptor() const;
	const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues() const;
	TConstArrayView<FInstancedStruct> GetInitialFragmentValues() const;
	const FMassEntityTemplateData& GetTemplateData() const;

	MASSSPAWNER_API UE::Mass::FEntityBuilder CreateEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager) const;

private:
	FMassEntityTemplateData TemplateData;
	FMassArchetypeHandle Archetype;
	FMassEntityTemplateID TemplateID;
};


struct FMassEntityTemplateIDFactory
{
	static MASSSPAWNER_API FMassEntityTemplateID Make(const FGuid& ConfigGuid);
	static MASSSPAWNER_API FMassEntityTemplateID MakeFlavor(const FMassEntityTemplateID& SourceTemplateID, const int32 Flavor);
};


//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------

inline bool FMassEntityTemplateData::IsEmpty() const
{
	return Composition.IsEmpty(); 
}

inline TConstArrayView<FMassEntityTemplateData::FObjectFragmentInitializerFunction> FMassEntityTemplateData::GetObjectFragmentInitializers() const
{ 
	return ObjectInitializers; 
}

inline const FString& FMassEntityTemplateData::GetTemplateName() const
{ 
	return TemplateName; 
}

inline const FMassArchetypeCompositionDescriptor& FMassEntityTemplateData::GetCompositionDescriptor() const
{ 
	return Composition; 
}

inline const FMassArchetypeSharedFragmentValues& FMassEntityTemplateData::GetSharedFragmentValues() const
{ 
	return SharedFragmentValues; 
}

inline TConstArrayView<FInstancedStruct> FMassEntityTemplateData::GetInitialFragmentValues() const
{ 
	return InitialFragmentValues; 
}

inline TArray<FMassEntityTemplateData::FObjectFragmentInitializerFunction>& FMassEntityTemplateData::GetMutableObjectFragmentInitializers()
{ 
	return ObjectInitializers; 
}

inline void FMassEntityTemplateData::SetTemplateName(const FString& Name)
{
	TemplateName = Name; 
}

inline FMassTagBitSet FMassEntityTemplateData::GetTags() const
{
	return Composition.GetElementsBitSet().Get<FMassTagBitSet>();
}

inline FMassElementBitSet& FMassEntityTemplateData::GetMutableTags()
{
	return Composition.GetElementsBitSet();
}

inline FMassArchetypeCreationParams& FMassEntityTemplateData::GetArchetypeCreationParams()
{
	return CreationParams;
}

inline const FMassArchetypeHandle& FMassEntityTemplate::GetArchetype() const
{
	return Archetype;
}

inline bool FMassEntityTemplate::IsValid() const
{
	return Archetype.IsValid();
}

inline void FMassEntityTemplate::SetTemplateID(FMassEntityTemplateID InTemplateID)
{
	TemplateID = InTemplateID;
}

inline FMassEntityTemplateID FMassEntityTemplate::GetTemplateID() const
{
	return TemplateID;
}

inline TConstArrayView<FMassEntityTemplateData::FObjectFragmentInitializerFunction> FMassEntityTemplate::GetObjectFragmentInitializers() const
{
	return TemplateData.GetObjectFragmentInitializers();
}

inline const FString& FMassEntityTemplate::GetTemplateName() const
{
	return TemplateData.GetTemplateName();
}

inline const FMassArchetypeCompositionDescriptor& FMassEntityTemplate::GetCompositionDescriptor() const
{
	return TemplateData.GetCompositionDescriptor();
}

inline const FMassArchetypeSharedFragmentValues& FMassEntityTemplate::GetSharedFragmentValues() const
{
	return TemplateData.GetSharedFragmentValues();
}

inline TConstArrayView<FInstancedStruct> FMassEntityTemplate::GetInitialFragmentValues() const
{
	return TemplateData.GetInitialFragmentValues();
}

inline const FMassEntityTemplateData& FMassEntityTemplate::GetTemplateData() const
{
	return TemplateData;
}

