// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/Object.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTemplate.h"
#include "MassTranslator.h"
#include "MassEntityTemplateRegistry.generated.h"

#define ENSURE_SUPPORTED_TRAIT_OPERATION() ensureMsgf(bBuildInProgress == false, TEXT("This method is not expected to be called as "\
	"part of trait's BuildTemplate call. Traits are not supposed to add elements based on other traits due to arbitrary trait ordering."));

class UWorld;
class UMassEntityTraitBase;

USTRUCT()
struct FMassMissingTraitMessage
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	explicit FMassMissingTraitMessage(const UMassEntityTraitBase* InRequestingTrait = nullptr, const UStruct* InMissingType = nullptr, const UMassEntityTraitBase* InRemovedByTrait = nullptr)
		: RequestingTrait(InRequestingTrait), MissingType(InMissingType), RemovedByTrait(InRemovedByTrait)
	{}

	const UMassEntityTraitBase* RequestingTrait = nullptr;
	const UStruct* MissingType = nullptr;
	// if set indicates that the missing type has been explicitly removed by given trait.
	const UMassEntityTraitBase* RemovedByTrait = nullptr;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FMassDuplicateElementsMessage
{
	GENERATED_BODY()
#if WITH_EDITORONLY_DATA
	const UMassEntityTraitBase* DuplicatingTrait = nullptr;
	const UMassEntityTraitBase* OriginalTrait = nullptr;
	const UStruct* Element = nullptr;
#endif // WITH_EDITORONLY_DATA
};

#if WITH_EDITORONLY_DATA
namespace UE::Mass::Debug
{
	extern MASSSPAWNER_API const FName TraitFailedValidation;
	extern MASSSPAWNER_API const FName TraitIgnored;
	extern MASSSPAWNER_API const FName TraitOverridden;
}
#endif // WITH_EDITORONLY_DATA

enum class EFragmentInitialization : uint8
{
	DefaultInitializer,
	NoInitializer
};

struct FMassEntityTemplateBuildContext
{
	explicit FMassEntityTemplateBuildContext(FMassEntityTemplateData& InTemplate, FMassEntityTemplateID InTemplateID = FMassEntityTemplateID())
		: TemplateData(InTemplate)
		, TemplateID(InTemplateID)
	{}

	void SetTemplateName(const FString& Name)
	{
		TemplateData.SetTemplateName(Name);
	}

	//----------------------------------------------------------------------//
	// Fragments 
	//----------------------------------------------------------------------//
	template<typename T>
	T& AddFragment_GetRef()
	{
		TypeAdded(T::StaticStruct());
		return TemplateData.AddFragment_GetRef<T>();
	}

	template<typename T>
	void AddFragment()
	{
		TypeAdded(T::StaticStruct());
		TemplateData.AddFragment<T>();
	}

	void AddFragment(FConstStructView InFragment)
	{ 
		checkf(InFragment.GetScriptStruct(), TEXT("Expecting a valid fragment type"));
		TypeAdded(InFragment.GetScriptStruct());
		TemplateData.AddFragment(InFragment);
	}

	template<typename T>
	void AddTag()
	{
		// Tags can be added by multiple traits, so they do not follow the same rules as fragments
		TemplateData.AddTag<T>();
		TypeAdded(T::StaticStruct());
	}

	void AddTag(TNotNull<const UScriptStruct*> TagType)
	{
		// Tags can be added by multiple traits, so they do not follow the same rules as fragments
		TemplateData.AddTag(TagType);
		TypeAdded(TagType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void AddTag(const UScriptStruct& TagType)
	{
		AddTag(&TagType);
	}

	template<typename T>
	void AddChunkFragment()
	{
		TypeAdded(T::StaticStruct());
		TemplateData.AddChunkFragment<T>();
	}

	void AddConstSharedFragment(const FConstSharedStruct& InSharedFragment)
	{
		checkf(InSharedFragment.GetScriptStruct(), TEXT("Expecting a valid shared fragment type"));
		TypeAdded(InSharedFragment.GetScriptStruct());
		TemplateData.AddConstSharedFragment(InSharedFragment);
	}

	void AddSharedFragment(const FSharedStruct& InSharedFragment)
	{
		checkf(InSharedFragment.GetScriptStruct(), TEXT("Expecting a valid shared fragment type"));
		TypeAdded(InSharedFragment.GetScriptStruct());
		TemplateData.AddSharedFragment(InSharedFragment);
	}

	/**
	 * Removes given tag from collected data. More precisely: it will store the information and apply upon template creation (an optimization). 
	 * WARNING: use with caution and only in cases where you know for certain what the given tag does and which processors rely on it.
	 *		Using this functionality makes most sense for removing tags that specifically mean that entities having it are to be
	 *		processed by a given processor.
	 */
	void RemoveTag(TNotNull<const UScriptStruct*> TagType)
	{
		checkf(UE::Mass::IsA<FMassTag>(TagType), TEXT("Given struct doesn't represent a valid mass tag type. Make sure to inherit from FMassTag or one of its child-types."));
		RemovedTypes.Add({TagType
#if WITH_EDITORONLY_DATA
			, TraitsData.Last().Trait
#endif // WITH_EDITORONLY_DATA
		});
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void RemoveTag(const UScriptStruct& TagType)
	{
		RemoveTag(&TagType);
	}

	template<typename T>
	void RemoveTag()
	{
		RemoveTag(T::StaticStruct());
	}

	template<typename T>
	T* GetFragment()
	{
		return TemplateData.GetMutableFragment<T>();
	}

	template<typename T>
	bool HasFragment() const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasFragment<T>();
	}
	
	bool HasFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasFragment(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasFragment(const UScriptStruct& ScriptStruct)
	{
		HasFragment(&ScriptStruct);
	}

	template<typename T>
	bool HasTag() const
	{
		return TemplateData.HasTag<T>();
	}

	template<typename T>
	bool HasChunkFragment() const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasChunkFragment<T>();
	}

	template<typename T>
	bool HasSharedFragment() const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasSharedFragment<T>();
	}

	bool HasSharedFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasSharedFragment(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasSharedFragment(const UScriptStruct& ScriptStruct)
	{
		HasSharedFragment(&ScriptStruct);
	}

	template<typename T>
	bool HasConstSharedFragment() const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasConstSharedFragment<T>();
	}

	bool HasConstSharedFragment(TNotNull<const UScriptStruct*> ScriptStruct) const
	{
		ENSURE_SUPPORTED_TRAIT_OPERATION();
		return TemplateData.HasConstSharedFragment(ScriptStruct);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void HasConstSharedFragment(const UScriptStruct& ScriptStruct)
	{
		HasConstSharedFragment(&ScriptStruct);
	}

	//----------------------------------------------------------------------//
	// Translators
	//----------------------------------------------------------------------//
	template<typename T>
	void AddTranslator()
	{
		TypeAdded(T::StaticClass());
		TemplateData.GetMutableTags() += GetDefault<T>()->GetRequiredTags();
	}

	//----------------------------------------------------------------------//
	// Dependencies
	//----------------------------------------------------------------------//
	template<typename T>
	void RequireFragment()
	{
		static_assert(UE::Mass::CTag<T> == false, "Given struct type is a valid fragment type.");
		AddDependency(T::StaticStruct());
	}

	template<typename T>
	void RequireTag()
	{
		static_assert(UE::Mass::CTag<T>, "Given struct type is not a valid tag type.");
		AddDependency(T::StaticStruct());
	}

	void AddDependency(const UStruct* Dependency)
	{
		TraitsData.Last().TypesRequired.Add(Dependency);
	}

	//----------------------------------------------------------------------//
	// Template access
	//----------------------------------------------------------------------//
	FMassEntityTemplateID GetTemplateID() const { return TemplateID; }
	TArray<FMassEntityTemplateData::FObjectFragmentInitializerFunction>& GetMutableObjectFragmentInitializers() { return TemplateData.GetMutableObjectFragmentInitializers(); }

	//----------------------------------------------------------------------//
	// Build methods
	//----------------------------------------------------------------------//

	/**
	 * Builds context from a list of traits
	 * @param Traits is the list of all the traits to build an entity
	 * @param World owning the MassEntitySubsystem for which the entity template is built
	 * @return true if there were no validation errors
	 */
	bool BuildFromTraits(TConstArrayView<UMassEntityTraitBase*> Traits, const UWorld& World);

	/** 
	 * The method that allows to distinguish between regular context use (using traits to build templates) and 
	 * the "data investigation" mode (used for debugging and authoring purposes). Utilize this function to 
	 * avoid UWorld-specific operations (like getting subsystems). This method should also be used when a trait 
	 * contains conditional logic - in that case it's required for the trait to add all the types that are potentially
	 * added at runtime (even if seemingly conflicting information will be added). 
	 * 
	 * @return whether this context is in data inspection mode.
	 */
#if WITH_EDITORONLY_DATA
	bool IsInspectingData() const
	{
		return bIsInspectingData;
	}
#else
	constexpr bool IsInspectingData() const
	{
		return false;
	}
#endif

#if WITH_EDITORONLY_DATA
	void EnableDataInvestigationMode()
	{
		checkf(TemplateData.IsEmpty(), TEXT("Marking a FMassEntityTemplateBuildContext as being in 'investigation mode` is only supported before the context is first used."));
		bIsInspectingData = true;
	}
#endif // WITH_EDITORONLY_DATA

	void RegisterOverriddenTrait(TNotNull<UMassEntityTraitBase*> Trait)
	{
		OverriddenTraits.Add(Trait);
	}

protected:

	/**
	 * Validate the build context for fragment trait ownership and trait fragment missing dependency
	 * @param World owning the MassEntitySubsystem for which the entity template is validated against
	 * @return true if there were no validation errors
	 */
	bool ValidateBuildContext(const UWorld& World);

	void TypeAdded(TNotNull<const UStruct*> Type)
	{
		checkf(TraitsData.Num(), TEXT("Adding elements to the build context before BuildFromTraits or SetTraitBeingProcessed was called is unsupported"));
		TraitsData.Last().TypesAdded.Add(Type);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void TypeAdded(const UScriptStruct& Type)
	{
		TypeAdded(&Type);
	}

	/** 
	 * Return true if the given trait can be used. The function will fail if a trait instance of the given class has already 
	 * been processed. The function will also fail the very same trait instance is used multiple times.
	 * Note that it's ok for Trait to be nullptr to indicate the subsequent additions to the build context are procedural
	 * in nature and are not associated with any traits. In that case it's ok to have multiple SetTraitBeingProcessed(nullptr)
	 * calls.
	 */
	MASSSPAWNER_API bool SetTraitBeingProcessed(const UMassEntityTraitBase* Trait);

	void ResetBuildTimeData()
	{
		TraitsData.Reset();
		TraitsProcessed.Reset();
		IgnoredTraits.Reset();
		RemovedTypes.Reset();
		bBuildInProgress = false;
	}

	struct FTraitData
	{
		const UMassEntityTraitBase* Trait = nullptr;
		TArray<const UStruct*> TypesAdded;
		TArray<const UStruct*> TypesRequired;
	};
	TArray<FTraitData> TraitsData;
	TSet<const UMassEntityTraitBase*> TraitsProcessed;
	TSet<const UMassEntityTraitBase*> IgnoredTraits;
	TSet<const UMassEntityTraitBase*> OverriddenTraits;

	struct FRemovedType
	{
		const UStruct* TypeRemoved = nullptr;
#if WITH_EDITOR
		const UMassEntityTraitBase* Remover = nullptr;
#endif // WITH_EDITOR
		bool operator==(const FRemovedType& Other) const
		{
			return TypeRemoved == Other.TypeRemoved;
		}
	};
	/**
	 * These tags will be removed from the resulting entity template
	 * @see RemoveTag for more details
	 */
	TArray<FRemovedType> RemovedTypes;

	bool bBuildInProgress = false;

	FMassEntityTemplateData& TemplateData;
	FMassEntityTemplateID TemplateID;

#if WITH_EDITORONLY_DATA
private:
	/**
	 * This being set to `true` indicates that the context is being used to gather information, not to create actual
	 * entity templates.
	 */
	bool bIsInspectingData = false;
#endif // WITH_EDITORONLY_DATA
};

/** 
 * Represents a repository storing all the FMassEntityTemplate that have been created and registered as part of FMassEntityConfig
 * processing or via custom code (like we do in InstancedActors plugin).
 */
struct FMassEntityTemplateRegistry
{
	// @todo consider TFunction instead
	DECLARE_DELEGATE_ThreeParams(FStructToTemplateBuilderDelegate, const UWorld* /*World*/, const FConstStructView /*InStructInstance*/, FMassEntityTemplateBuildContext& /*BuildContext*/);

	MASSSPAWNER_API explicit FMassEntityTemplateRegistry(UObject* InOwner = nullptr);

	/** Initializes and stores the EntityManager the templates will be associated with. Needs to be called before any template operations.
	 *  Note that the function will only let users set the EntityManager once. Once it's set the subsequent calls will
	 *  have no effect. If attempting to set a different EntityManaget an ensure will trigger. */
	MASSSPAWNER_API void Initialize(const TSharedPtr<FMassEntityManager>& InEntityManager);

	MASSSPAWNER_API void ShutDown();

	MASSSPAWNER_API UWorld* GetWorld() const;

	static MASSSPAWNER_API FStructToTemplateBuilderDelegate& FindOrAdd(const UScriptStruct& DataType);

	/** Removes all the cached template instances */
	MASSSPAWNER_API void DebugReset();

	MASSSPAWNER_API const TSharedRef<FMassEntityTemplate>* FindTemplateFromTemplateID(FMassEntityTemplateID TemplateID) const;

	/**
	 * Adds a template based on TemplateData
	 */
	MASSSPAWNER_API const TSharedRef<FMassEntityTemplate>& FindOrAddTemplate(FMassEntityTemplateID TemplateID, FMassEntityTemplateData&& TemplateData);

	MASSSPAWNER_API void DestroyTemplate(FMassEntityTemplateID TemplateID);

	FMassEntityManager& GetEntityManagerChecked() { check(EntityManager); return *EntityManager; }

protected:
	static MASSSPAWNER_API TMap<const UScriptStruct*, FStructToTemplateBuilderDelegate> StructBasedBuilders;

	TMap<FMassEntityTemplateID, TSharedRef<FMassEntityTemplate>> TemplateIDToTemplateMap;

	/** 
	 * EntityManager the hosted templates are associated with. Storing instead of fetching at runtime to ensure all 
	 *	templates are tied to the same EntityManager
	 */
	TSharedPtr<FMassEntityManager> EntityManager;

	TWeakObjectPtr<UObject> Owner;
};

#undef ENSURE_SUPPORTED_TRAIT_OPERATION
