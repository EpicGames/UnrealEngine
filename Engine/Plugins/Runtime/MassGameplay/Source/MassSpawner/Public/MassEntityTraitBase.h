// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassProcessingTypes.h"
#include "MassEntityTraitBase.generated.h"

struct FMassEntityTemplateBuildContext;

/**
 * Base class for Mass Entity Traits.
 * An entity trait is a set of fragments that create a logical trait tha makes sense to end use (i.e. replication, visualization).
 * The template building method allows to configure some fragments based on properties or cached values.
 * For example, a fragment can be added based on a referenced asset, or some memory hungry settings can be
 * cached and just and index stored on a fragment.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, EditInlineNew, CollapseCategories, config = Mass, defaultconfig)
class UMassEntityTraitBase : public UObject
{
	GENERATED_BODY()

public:
	/** 
	 * This is a type wrapping an existing TArray to limit how users can interact with the contained data. 
	 * We essentially limit users to just adding elements, no other operations.
	 */
	struct FAdditionalTraitRequirements
	{
		explicit FAdditionalTraitRequirements(TArray<const UStruct*>& InTargetContainer)
			: TargetContainer(InTargetContainer)
		{
		}
		
		// Copying constructor and assignment deleted to prevent users storing copies of the type, 
		// which wouldn't be safe due to this type hosting a reference to an array that can go out of scope.
		FAdditionalTraitRequirements(const FAdditionalTraitRequirements&) = delete;
		FAdditionalTraitRequirements& operator=(const FAdditionalTraitRequirements&) = delete;

		FAdditionalTraitRequirements& Add(const UScriptStruct* RequiredType)
		{
			TargetContainer.Add(RequiredType);
			return *this;
		}

	private:
		TArray<const UStruct*>& TargetContainer;
	};


	/** Appends items into the entity template required for the trait. */
	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const PURE_VIRTUAL(UMassEntityTraitBase::BuildTemplate, return; );

	virtual void DestroyTemplate(const UWorld& World) const {}

	/**
	 * Called once all traits have been processed and fragment requirements have been checked. Override this function
	 * to perform additional Trait's configuration validation. Returning `false` will indicate that the trait instance
	 * is not happy with the validation results - this result will be treated as an error.
	 * @param OutTraitRequirements contains requirements declared by this trait and gives ValidateTemplate a chance to add 
	 *		to the dependencies based on the state of BuildContext, which by this point should contain all the elements 
	 *		added by all the relevant traits.
	 * @return whether the validation was successful
	 */
	MASSSPAWNER_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewTraitType, UMassEntityTraitBase&);
	static FOnNewTraitType& GetOnNewTraitTypeEvent()
	{
		return OnNewTraitTypeEvent;
	}

protected:
	MASSSPAWNER_API virtual void PostInitProperties() override;

private:
	static MASSSPAWNER_API FOnNewTraitType OnNewTraitTypeEvent;
#endif // WITH_EDITOR

public:
	/** Configures which configurations are targeted by this trait. Use it to have different configurations for client and server, for example.*/
	UPROPERTY(EditAnywhere, Category=Mass)
	EProcessorExecutionFlags ValidTargetConfig = EProcessorExecutionFlags::AllNetModes;
};
