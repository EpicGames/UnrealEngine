// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntitySpawnDataGeneratorBase.h"
#include "ZoneGraphTypes.h"
#include "MassEntityZoneGraphSpawnPointsGenerator.generated.h"

class AZoneGraphData;

/**
 * Describes the SpawnPoints Generator when we want to spawn directly on Zone Graph
 */
UCLASS(MinimalAPI, BlueprintType, meta=(DisplayName="ZoneGraph SpawnPoints Generator"))
class UMassEntityZoneGraphSpawnPointsGenerator : public UMassEntitySpawnDataGeneratorBase
{
	GENERATED_BODY()

public:
	MASSSPAWNER_API virtual void Generate(UObject& QueryOwner, TConstArrayView<FMassSpawnedEntityType> EntityTypes, int32 Count, FFinishedGeneratingSpawnDataSignature& FinishedGeneratingSpawnPointsDelegate) const override;

protected:
	MASSSPAWNER_API void GeneratePointsForZoneGraphData(const ::AZoneGraphData& ZoneGraphData, TArray<FVector>& Locations, const FRandomStream& RandomStream) const;

	/** Tags to filter which lane to use to generate points on */
	UPROPERTY(EditAnywhere, Category = "ZoneGraph Generator Config")
	FZoneGraphTagFilter TagFilter;

	/** Minimum gap for spawning entities on a given lanes */
	UPROPERTY(EditAnywhere, Category = "ZoneGraph Generator Config")
	float MinGap = 100;

	/** Maximum gap for spawning entities on a given lanes */
	UPROPERTY(EditAnywhere, Category = "ZoneGraph Generator Config")
	float MaxGap = 300;
};
