// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassSpawnLocationProcessor.generated.h"

#define UE_API MASSSPAWNER_API

UCLASS(MinimalAPI)
class UMassSpawnLocationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassSpawnLocationProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
	FRandomStream RandomStream;
};

#undef UE_API
