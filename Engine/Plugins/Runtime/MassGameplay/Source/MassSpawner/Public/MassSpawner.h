// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "MassSpawnerTypes.h"
#include "MassEntitySpawnDataGeneratorBase.h"
#include "MassSpawner.generated.h"

#define UE_API MASSSPAWNER_API

class UBillboardComponent;
struct FStreamableHandle;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMassSpawnerOnSpawningFinishedEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMassSpawnerOnDespawningFinishedEvent);

/** A spawner you can put on a map and configure it to spawn different things */
UCLASS(MinimalAPI, hidecategories = (Object, Actor, Input, Rendering, LOD, Cooking, Collision, HLOD, Partition))
class AMassSpawner : public AActor
{
	GENERATED_BODY()
public:
	UE_API AMassSpawner();

protected:
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void BeginDestroy() override;

public:
#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Debug")
	UE_API void DEBUG_Spawn();

	/** Remove all the entities */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Debug")
	UE_API void DEBUG_Clear();
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UE_API void RegisterEntityTemplates();

public:
	/**
	 * Starts the spawning of all the agent types of this spawner
	 */
	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API void DoSpawning();

	/**
	 * Despawn all mass agent that was spawned by this spawner
	 */
	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API void DoDespawning();

	/**
	 * Despawn all mass agent that was spawned by this spawner, except EntitiesToIgnore.
	 *
	 * Any EntitiesToIgnore previously spawned by this spawner will remain spawned and tracked by this spawner.   
	 */
	UE_API void DoDespawning(TConstArrayView<FMassEntityHandle> EntitiesToIgnore);

	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API void ClearTemplates();

	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API void UnloadConfig();

	/**
	 * If given entity has been spawned by this MassSpawner instance then it will get destroyed and all the book keeping 
	 * updated. Otherwise the call has no effect.
	 * @return true if the entity got removed. False otherwise.
	 */
	UE_API bool DespawnEntity(const FMassEntityHandle Entity);

	/**
	 * Scales the spawning counts
	 * @param Scale is the number to multiply the all counts of each agent types 
	 */
	UFUNCTION(BlueprintCallable, Category = "Spawning")
	void ScaleSpawningCount(float Scale) { SpawningCountScale = Scale; }

	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API int32 GetCount() const;

	UFUNCTION(BlueprintCallable, Category = "Spawning")
	UE_API float GetSpawningCountScale() const;

	/** Called once DoSpawning completes and all entities have been spawned. */
	UPROPERTY(BlueprintAssignable)
	FMassSpawnerOnSpawningFinishedEvent OnSpawningFinishedEvent;

	/** Called once DoDespawning completes and all mass agents spawned by this spawner have been despawned. */
	UPROPERTY(BlueprintAssignable)
	FMassSpawnerOnDespawningFinishedEvent OnDespawningFinishedEvent; 

protected:
	UE_API void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues);
	UE_API void SpawnGeneratedEntities(TConstArrayView<FMassEntitySpawnDataGeneratorResult> Results);
	UE_API void OnSpawnDataGenerationFinished(TConstArrayView<FMassEntitySpawnDataGeneratorResult> Results, FMassSpawnDataGenerator* FinishedGenerator);

	UE_API int32 GetSpawnCount() const;
	UE_API UMassProcessor* GetPostSpawnProcessor(TSubclassOf<UMassProcessor> ProcessorClass);

protected:

	struct FSpawnedEntities
	{
		FMassEntityTemplateID TemplateID;
		TArray<FMassEntityHandle> Entities;
	};

	UPROPERTY(EditAnywhere, Category = "Mass|Spawn")
	int32 Count;

	/** Array of entity types to spawn. These define which entities to spawn. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mass|Spawn")
	TArray<FMassSpawnedEntityType> EntityTypes;

	/** Array of entity spawn generators. These define where to spawn entities. */
	UPROPERTY(EditAnywhere, Category = "Mass|Spawn")
	TArray<FMassSpawnDataGenerator> SpawnDataGenerators;

	UPROPERTY(Category = "Mass|Spawn", EditAnywhere)
	uint32 bAutoSpawnOnBeginPlay : 1;

	/** By default TickSchematics will be appended to the simulation's schematics. If this property is set to true the
	 *  TickSchematics will override the original simulation schematics */
	UPROPERTY(Category = "Mass|Simulation", EditAnywhere)
	uint32 bOverrideSchematics : 1;

	UPROPERTY()
	TArray<TObjectPtr<UMassProcessor>> PostSpawnProcessors;

	/** Scale of the spawning count */
	UPROPERTY(EditAnywhere, Category = "Mass|Spawn")
	float SpawningCountScale = 1.0f;

	FDelegateHandle SimulationStartedHandle;

	FDelegateHandle OnPostWorldInitDelegateHandle;

	TArray<FSpawnedEntities> AllSpawnedEntities;
	
	TArray<FMassEntitySpawnDataGeneratorResult> AllGeneratedResults;
	
	TSharedPtr<FStreamableHandle> StreamingHandle;

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif // WITH_EDITORONLY_DATA
 };

namespace UE::MassSpawner
{
	MASSSPAWNER_API extern float ScalabilitySpawnDensityMultiplier;
}

#undef UE_API
