// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityManager.h"
#include "MassSubsystemBase.h"
#include "MassEntityTemplateRegistry.h"
#include "MassSpawnerSubsystem.generated.h"

struct FMassEntityManager;
struct FEntityCreationContext;
struct FMassEntityTemplate;
struct FInstancedStruct;
struct FStructView;
struct FMassEntityTemplateID;
class UMassSimulationSubsystem;

UCLASS(MinimalAPI)
class UMassSpawnerSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:
	MASSSPAWNER_API UMassSpawnerSubsystem();

	/** Spawns entities of the kind described by the given EntityTemplate. The spawned entities are fully initialized
	 *  meaning the EntityTemplate.InitializationPipeline gets run for all spawned entities.
	 *  @param EntityTemplate template to use for spawning entities
	 *  @param NumberToSpawn number of entities to spawn
	 *  @param OutEntities where the IDs of created entities get added. Note that the contents of OutEntities get overridden by the function.
	 *  @return shared pointer to the entity creation context, that, one released, will cause all the accumulated observers and commands executed
	 */ 
	MASSSPAWNER_API TSharedPtr<FMassEntityManager::FEntityCreationContext> SpawnEntities(const FMassEntityTemplate& EntityTemplate, const uint32 NumberToSpawn, TArray<FMassEntityHandle>& OutEntities);

	MASSSPAWNER_API TSharedPtr<FMassEntityManager::FEntityCreationContext> SpawnEntities(FMassEntityTemplateID TemplateID, const uint32 NumberToSpawn, FConstStructView SpawnData, TSubclassOf<UMassProcessor> InitializerClass, TArray<FMassEntityHandle>& OutEntities);

	MASSSPAWNER_API void DestroyEntities(TConstArrayView<FMassEntityHandle> Entities);

	const FMassEntityTemplateRegistry& GetTemplateRegistryInstance() const { return TemplateRegistryInstance; }
	FMassEntityTemplateRegistry& GetMutableTemplateRegistryInstance() { return TemplateRegistryInstance; }

	MASSSPAWNER_API const FMassEntityTemplate* GetMassEntityTemplate(FMassEntityTemplateID TemplateID) const;

	FMassEntityManager& GetEntityManagerChecked()
	{
		check(EntityManager.IsValid());
		return *EntityManager.Get();
	}

protected:
	// UWorldSubsystem BEGIN
	MASSSPAWNER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSSPAWNER_API virtual void Deinitialize() override;
	// UWorldSubsystem END

	MASSSPAWNER_API TSharedPtr<FMassEntityManager::FEntityCreationContext> DoSpawning(const FMassEntityTemplate& EntityTemplate, const int32 NumToSpawn, FConstStructView SpawnData, TSubclassOf<UMassProcessor> InitializerClass, TArray<FMassEntityHandle>& OutEntities);

	MASSSPAWNER_API UMassProcessor* GetSpawnDataInitializer(TSubclassOf<UMassProcessor> InitializerClass);

	UPROPERTY()
	TArray<TObjectPtr<UMassProcessor>> SpawnDataInitializers;

	TSharedPtr<FMassEntityManager> EntityManager;

	FMassEntityTemplateRegistry TemplateRegistryInstance;
};

