// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Logging/LogMacros.h"
#include "MassEntityTemplate.h"
#include "MassSpawnerTypes.generated.h"


DECLARE_LOG_CATEGORY_EXTERN(LogMassSpawner, Warning, All);

class APawn;
class UCurveFloat;
class UMassEntityConfigAsset;
class UMassEntitySpawnDataGeneratorBase;
class UMassEntityTraitBase;

namespace UE::MassSpawner
{
	MASSSPAWNER_API uint32 HashTraits(TConstArrayView<UMassEntityTraitBase*> CombinedTraits);
}

USTRUCT()
struct FMassTransformsSpawnData
{
	GENERATED_BODY()

	FMassTransformsSpawnData()
	: bRandomize(true)
	{}

	// declaring the type used to be able to statically test it against other types 
	using FTransformsContainerType = TArray<FTransform>;
	FTransformsContainerType Transforms;
	
	// When true, Transforms will be assigned to entities as if pre-shuffled. 
	// If Transforms >= Entities, this provides the best chance for a good spread of entity transforms 
	// and transforms choices will be guaranteed unique. If Transforms < Entities, the same 
	// transform will be used for multiple entities and a warning logged.
	//
	// When false, Transforms will be assigned in order to spawned entities.
	bool bRandomize : 1;
};

/**
 * Describes an entity type to spawn.
 */
USTRUCT(BlueprintType)
struct FMassSpawnedEntityType
{
	GENERATED_BODY()

	/** Asset that describes the entity */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mass|Spawn")
	TSoftObjectPtr<UMassEntityConfigAsset> EntityConfig;

	/** Proportion of the count that should be this agent type, (the proportions will be normalized with other sibling agent types) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mass|Spawn", meta = (ClampMin = 0.0f, UIMin = 0.0f))
	float Proportion = 1.0f;
	
	bool IsLoaded() const { return EntityConfigPtr != nullptr; }

	void UnloadEntityConfig()
	{
		EntityConfigPtr = nullptr;
		EntityConfig.ResetWeakPtr();
	}

	MASSSPAWNER_API const UMassEntityConfigAsset* GetEntityConfig() const;
	MASSSPAWNER_API UMassEntityConfigAsset* GetEntityConfig();

private:
	UPROPERTY(Transient)
	mutable TObjectPtr<UMassEntityConfigAsset> EntityConfigPtr = nullptr;
};

USTRUCT(BlueprintType)
struct FMassSpawnDataGenerator
{
	GENERATED_BODY()

	/** The Generator to use to generate the spawn the points */
	UPROPERTY()
	TSubclassOf<UMassEntitySpawnDataGeneratorBase> GeneratorClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Instanced, Category = "Mass|Generator")
	TObjectPtr<UMassEntitySpawnDataGeneratorBase> GeneratorInstance = nullptr;

	/** Proportion of the spawn points that should be generated by this generator, (the proportions will be normalized with other sibling generators) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mass|Generator", meta = (ClampMin = 0.0f, UIMin = 0.0f))
	float Proportion = 1.0f;

	/** Runtime value to know if we received the generated data for this generator */
	bool bDataGenerated = false;
};

USTRUCT()
struct FReplicationTemplateIDFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FMassEntityTemplateID ID;
};
