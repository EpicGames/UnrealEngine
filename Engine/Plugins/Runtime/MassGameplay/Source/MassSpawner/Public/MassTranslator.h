// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityTypes.h"
#include "MassTranslator.generated.h"

#define UE_API MASSSPAWNER_API


UENUM()
enum class EMassTranslationDirection : uint8
{
	None = 0,
	InitializationOnly = 1,
	ActorToMass = 1 << 1,
	MassToActor = 1 << 2,
	BothWays = ActorToMass | MassToActor
};
ENUM_CLASS_FLAGS(EMassTranslationDirection);

/** 
 *  A class that's responsible for translation between UObjects and Mass. A translator knows how to initialize 
 *  fragments related to the UClass that the given translator cares about. It can also be used at runtime to 
 *  copy values from UObjects to fragments and back.
 */
UCLASS(MinimalAPI, abstract)
class UMassTranslator : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UE_API UMassTranslator();
public:
	/** Fetches the FMassTag-derived types required by this Translator. And entity needs these tags to be 
	 *  processed by this Translator instance. 
	 * 
	 *  @todo might want this function on the MassProcessor level. TBD
	 * 
	 *  @param InOutTags - this translator's expected tags will be added to this parameter */
	UE_DEPRECATED(5.8, "This function is deprecated, please fetch the tags directly with GetRequiredTags and append then wherever you want.")
	void AppendRequiredTags(FMassTagBitSet& InOutTags) const 
	{
		InOutTags += RequiredTags; 
	}

	FMassTagBitSet GetRequiredTags() const 
	{
		return RequiredTags; 
	}

protected:
	UE_API void AddRequiredTagsToQuery(FMassEntityQuery& EntityQuery);

protected:
	/** These are the tag fragments expected by this translator that let other code (like entity traits) hint to 
	 *  the system which translators they'd want their entity to be processed by. */
	FMassTagBitSet RequiredTags;
};

#undef UE_API
